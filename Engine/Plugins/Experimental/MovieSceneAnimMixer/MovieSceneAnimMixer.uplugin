{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Sequencer Anim Mixer",
	"Description" : "System for mixing layered animation in sequences",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFMirroring",
			"Enabled": true
		},
		{
			"Name": "SequencerAnimTools",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	],
	"Modules" :
	[
		{
			"Name" : "MovieSceneAnimMixer",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "MovieSceneAnimMixerEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "MovieSceneAnimMixerScripting",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		}
	]
}
