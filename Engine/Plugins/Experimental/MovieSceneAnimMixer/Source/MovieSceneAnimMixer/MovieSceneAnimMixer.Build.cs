// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieSceneAnimMixer : ModuleRules
	{
		public MovieSceneAnimMixer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings",
					"MovieScene",
					"MovieSceneTracks",
					"AnimGraphRuntime",
					"UAF",
					"UAFAnimGraph",
					"SlateCore",
					"UAFMirroring"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MovieSceneTracks",
				}
			);

			if (Target.bBuildDeveloperTools)
			{
				PrivateDependencyModuleNames.Add("Settings");
			}

			if (Target.bBuildWithEditorOnlyData && Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
					{
						"EditorFramework",
						"UnrealEd",
					});
			}
		}
	}
}