// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerBakeEvaluation.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "Evaluation/MovieScenePlayback.h"
#include "HAL/ThreadSafeCounter.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Systems/MovieSceneMixerBakeTargetSystem.h"
#include "StructUtils/InstancedStruct.h"

namespace UE::MovieScene::AnimMixerBakeEvaluation
{

// Scope that saves the mixer track's target and bake system state on construction,
// and restores them + re-evaluates at the original time on destruction.
//
// The bake filter flows through the entity system via FMovieSceneMixerBakeTarget,
// so the mixer system applies it during evaluation without a separate lookup.

static FThreadSafeCounter GBakeScopeIDCounter;

struct FBakeEvaluationScope
{
	UMovieSceneEntitySystemLinker* Linker;
	FInstanceHandle InstanceHandle;
	UMovieSceneAnimationMixerTrack* MixerTrack;
	UMovieSceneMixerBakeTargetSystem* BakeSystem;
	FMovieSceneEntitySystemRunner* Runner;
	FMovieSceneContext OriginalContext;
	TInstancedStruct<FMovieSceneMixedAnimationTarget> OriginalTarget;
	TSharedPtr<FTransform> CaptureTarget;

	FBakeEvaluationScope(
		UMovieSceneEntitySystemLinker* InLinker,
		FInstanceHandle InInstanceHandle,
		UMovieSceneAnimationMixerTrack* InMixerTrack,
		const FBakeFilter& Filter)
		: Linker(InLinker)
		, InstanceHandle(InInstanceHandle)
		, MixerTrack(InMixerTrack)
	{
		Runner = &Linker->GetRunner().Get();
		check(Runner);

		// Save original context
		FInstanceRegistry* Registry = Linker->GetInstanceRegistry();
		OriginalContext = Registry->GetContext(InstanceHandle);

		// Save original target, then swap to a bake target carrying the filter.
		// The filter fields on FMovieSceneMixerBakeTarget flow through entity
		// import into the mixer system's evaluation.
		OriginalTarget = MixerTrack->MixedAnimationTarget;

		FMovieSceneMixerBakeTarget BakeTarget;
		BakeTarget.ScopeID = GBakeScopeIDCounter.Increment();
		BakeTarget.Filter = Filter;

		// When the bake includes root motion conversion, capture the
		// animation-space root alongside the world-space result.
		if (!Filter.bSkipRootMotionConversion)
		{
			CaptureTarget = MakeShared<FTransform>(FTransform::Identity);
			BakeTarget.Filter.CaptureAnimSpaceRoot = CaptureTarget;
		}

		MixerTrack->MixedAnimationTarget =
			TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make<FMovieSceneMixerBakeTarget>(MoveTemp(BakeTarget));

		// Force full entity re-import so existing entities pick up the bake target
		InvalidateEntities();

		// Activate the bake target system
		BakeSystem = Linker->FindSystem<UMovieSceneMixerBakeTargetSystem>();
		if (!BakeSystem)
		{
			BakeSystem = Linker->LinkSystem<UMovieSceneMixerBakeTargetSystem>();
		}
		check(BakeSystem);
		BakeSystem->bBakeActive = true;
		BakeSystem->BakedResults.Reset();
	}

	~FBakeEvaluationScope()
	{
		// Deactivate bake
		BakeSystem->bBakeActive = false;

		// Restore original target
		MixerTrack->MixedAnimationTarget = OriginalTarget;

		// Force full entity re-import so entities pick up the restored original target
		InvalidateEntities();

		// Re-evaluate at the original time to restore current-frame state.
		// This uses the full flush (including Finalization) since we need the
		// original target systems to re-apply to the mesh.
		Runner->QueueUpdate(OriginalContext, InstanceHandle);
		Runner->Flush();
	}

	// Invalidate the sequence instance's entity cache so the next Import
	// phase forces a full re-import of all entities with current component values.
	void InvalidateEntities()
	{
		FInstanceRegistry* Registry = Linker->GetInstanceRegistry();
		if (Registry->IsHandleValid(InstanceHandle))
		{
			FSequenceInstance& Instance = Registry->MutateInstance(InstanceHandle);
			Instance.InvalidateCachedData();
		}
	}

	// Evaluate at a single time, capturing the bake result.
	// Sets bSilent + bHasJumped to suppress event triggers and side effects.
	FBakeResult FlushAtTime(FFrameTime Time)
	{
		FFrameRate FrameRate = OriginalContext.GetFrameRate();
		FMovieSceneEvaluationRange Range(Time, FrameRate);
		FMovieSceneContext BakeContext(Range, EMovieScenePlayerStatus::Jumping);
		BakeContext.SetHasJumped(true);
		BakeContext.SetIsSilent(true);

		int32 ResultIndexBefore = BakeSystem->BakedResults.Num();

		Runner->QueueUpdate(BakeContext, InstanceHandle);
		Runner->Flush();

		FBakeResult Result;
		Result.Time = Time;
		if (BakeSystem->BakedResults.Num() > ResultIndexBefore)
		{
			UMovieSceneMixerBakeTargetSystem::FBakedResult& Baked = BakeSystem->BakedResults.Last();
			Result.RootMotionTransform = Baked.RootMotionTransform;
			Result.Pose = MoveTemp(Baked.Pose);
			Result.Curves = MoveTemp(Baked.Curves);
			Result.Attributes = MoveTemp(Baked.Attributes);
		}

		// When capture is active, the conversion task wrote the pre-conversion
		// value. Otherwise (bSkipRootMotionConversion), the result is already
		// in animation space.
		if (CaptureTarget)
		{
			Result.AnimationSpaceRootMotion = *CaptureTarget;
		}
		else
		{
			Result.AnimationSpaceRootMotion = Result.RootMotionTransform;
		}

		return Result;
	}
};

FBakeResult EvaluateAtTime(
	UMovieSceneEntitySystemLinker* Linker,
	FInstanceHandle InstanceHandle,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	FFrameTime Time,
	const FBakeFilter& Filter)
{
	if (!Linker || !MixerTrack)
	{
		return FBakeResult{ Time };
	}

	FBakeEvaluationScope Scope(Linker, InstanceHandle, MixerTrack, Filter);
	return Scope.FlushAtTime(Time);
}

TArray<FBakeResult> EvaluateRange(
	UMovieSceneEntitySystemLinker* Linker,
	FInstanceHandle InstanceHandle,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	FFrameTime StartTime,
	FFrameTime FrameStep,
	int32 NumSamples,
	const FBakeFilter& Filter)
{
	TArray<FBakeResult> Results;
	if (!Linker || !MixerTrack || NumSamples <= 0)
	{
		return Results;
	}

	Results.Reserve(NumSamples);

	FBakeEvaluationScope Scope(Linker, InstanceHandle, MixerTrack, Filter);

	FFrameTime CurrentTime = StartTime;
	for (int32 i = 0; i < NumSamples; ++i)
	{
		Results.Add(Scope.FlushAtTime(CurrentTime));
		CurrentTime += FrameStep;
	}

	return Results;
}

} // namespace UE::MovieScene::AnimMixerBakeEvaluation
