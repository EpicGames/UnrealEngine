// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerBoneMatching.h"

#include "AnimMixerBakeEvaluation.h"
#include "BoneIndices.h"
#include "EvaluationVM/KeyframeState.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationRotationUtils.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneSection.h"

namespace UE::MovieScene::AnimMixerBoneMatching
{

// Compute the match offset to apply to the section's root motion so that the
// matched bone aligns with the underlying pose.
//
// Both rotation and translation matching use the matched bone's component-space
// transform. The rotation checkboxes control which Euler axes of the bone's
// CS rotation are matched; the location checkboxes control which position axes
// are matched. The resulting delta is applied to the root, which cascades
// through the skeleton to move the bone into alignment.
static FTransform ComputeMatchOffset(
	const FTransform& UnderlyingBoneCS,
	const FTransform& SectionBoneCS,
	const FTransform& SectionRootCS,
	const FTransform& SectionBoneRelToRoot,
	const FMovieSceneBoneMatchData& Settings)
{
	// Selective rotation offset from the matched bone's CS rotation.
	const bool bAnyRotation = Settings.bMatchRotationX || Settings.bMatchRotationY || Settings.bMatchRotationZ;
	FQuat DeltaRot = FQuat::Identity;

	if (bAnyRotation)
	{
		ERotationOrder RotationOrder = FindBestRotationOrder(
			Settings.bMatchRotationX, Settings.bMatchRotationY, Settings.bMatchRotationZ);

		FQuat UnderlyingBoneQuat = UnderlyingBoneCS.GetRotation();
		FQuat SecBoneQuat = SectionBoneCS.GetRotation();
		UnderlyingBoneQuat.EnforceShortestArcWith(SecBoneQuat);

		// Decompose both rotations to Euler, use SetClosestToMe to avoid
		// winding ambiguity, then selectively copy unmatched axes.
		FRotator UnderlyingRotation(FRotator::MakeFromEuler(EulerFromQuat(UnderlyingBoneQuat, RotationOrder)));
		FRotator SecBoneRotation(FRotator::MakeFromEuler(EulerFromQuat(SecBoneQuat, RotationOrder)));
		SecBoneRotation.SetClosestToMe(UnderlyingRotation);

		if (!Settings.bMatchRotationZ)
		{
			UnderlyingRotation.Yaw = SecBoneRotation.Yaw;
		}
		if (!Settings.bMatchRotationY)
		{
			UnderlyingRotation.Pitch = SecBoneRotation.Pitch;
		}
		if (!Settings.bMatchRotationX)
		{
			UnderlyingRotation.Roll = SecBoneRotation.Roll;
		}

		FQuat AdjustedUnderlyingQuat = QuatFromEuler(UnderlyingRotation.Euler(), RotationOrder);

		// Root-local delta so the bone's component-space rotation (root rotation R times
		// bone-relative rotation B) lands on AdjustedUnderlyingQuat once post-multiplied
		// onto the root: DeltaRot = R^-1 * AdjustedUnderlyingQuat * B^-1.
		DeltaRot = SectionRootCS.GetRotation().Inverse() * AdjustedUnderlyingQuat * SectionBoneRelToRoot.GetRotation().Inverse();
		DeltaRot.Normalize();
	}

	// Predict where the bone would be after applying just the rotation offset.
	// This accounts for how rotating the root moves attached bones.
	FQuat NewRootRot = SectionRootCS.GetRotation() * DeltaRot;
	FTransform RotatedRoot(NewRootRot, SectionRootCS.GetTranslation());
	RotatedRoot.SetScale3D(FVector::OneVector);
	FTransform PredictedBone = SectionBoneRelToRoot * RotatedRoot;

	// Translation delta from bone positions: how much to shift the root
	// so the bone reaches the underlying pose's position after the rotation.
	FVector TransDelta = UnderlyingBoneCS.GetTranslation() - PredictedBone.GetTranslation();

	if (!Settings.bMatchLocationX)
	{
		TransDelta.X = 0.0;
	}
	if (!Settings.bMatchLocationY)
	{
		TransDelta.Y = 0.0;
	}
	if (!Settings.bMatchLocationZ)
	{
		TransDelta.Z = 0.0;
	}

	// Build the desired new root and compute the offset from the section root.
	FTransform NewRoot(NewRootRot, SectionRootCS.GetTranslation() + TransDelta);
	NewRoot.SetScale3D(FVector::OneVector);

	return SectionRootCS.GetRelativeTransformReverse(NewRoot);
}

// Resolve the match time based on the MatchTimeMode and section boundaries.
static FFrameTime ResolveMatchTime(
	UMovieSceneSection* TargetSection,
	const FMovieSceneBoneMatchData& Settings,
	const FBoneMatchingContext& Context)
{
	switch (Settings.MatchTimeMode)
	{
	case EBoneMatchTimeMode::AtStartOfSelectedSection:
		if (TargetSection->HasStartFrame())
		{
			return FFrameTime(TargetSection->GetInclusiveStartFrame());
		}
		break;

	case EBoneMatchTimeMode::AtEndOfSelectedSection:
		if (TargetSection->HasEndFrame())
		{
			return FFrameTime(TargetSection->GetExclusiveEndFrame() - 1);
		}
		break;

	case EBoneMatchTimeMode::AtStartOfReferenceSection:
		if (UMovieSceneSection* Ref = Settings.ReferenceSection.Get())
		{
			if (Ref->HasStartFrame())
			{
				return FFrameTime(Ref->GetInclusiveStartFrame());
			}
		}
		break;

	case EBoneMatchTimeMode::AtEndOfReferenceSection:
		if (UMovieSceneSection* Ref = Settings.ReferenceSection.Get())
		{
			if (Ref->HasEndFrame())
			{
				return FFrameTime(Ref->GetExclusiveEndFrame() - 1);
			}
		}
		break;

	case EBoneMatchTimeMode::InBetween:
		if (UMovieSceneSection* Ref = Settings.ReferenceSection.Get())
		{
			// Use the midpoint between the target's start and the reference's end.
			FFrameNumber TargetStart = TargetSection->HasStartFrame() ? TargetSection->GetInclusiveStartFrame() : FFrameNumber(0);
			FFrameNumber RefEnd = Ref->HasEndFrame() ? Ref->GetExclusiveEndFrame() : TargetStart;

			return FFrameTime((TargetStart.Value + RefEnd.Value) / 2);
		}
		break;

	case EBoneMatchTimeMode::AtCurrentTime:
		return Context.CurrentTime;

	default:
		break;
	}

	// Fallback: use start of target section
	if (TargetSection->HasStartFrame())
	{
		return FFrameTime(TargetSection->GetInclusiveStartFrame());
	}
	return FFrameTime(0);
}

// Extract a bone's component-space transform from a baked LOD pose.
// Walks up the parent chain from the target bone to root, accumulating
// local transforms into component space. Returns identity if the bone
// is not found or the pose is invalid.
static FTransform ExtractBoneComponentSpaceTransform(
	const UE::UAF::FLODPose& Pose,
	FName BoneName)
{
	if (!Pose.IsValid() || BoneName.IsNone())
	{
		return FTransform::Identity;
	}

	FBoneIndexType BoneIndex = Pose.FindLODBoneIndexFromBoneName(BoneName);
	if (BoneIndex == INVALID_BONE_INDEX)
	{
		return FTransform::Identity;
	}

	FTransform ComponentSpace = (FTransform)Pose.LocalTransformsView[BoneIndex];

	FBoneIndexType ParentIndex = Pose.GetLODBoneParentIndex(BoneIndex);
	while (ParentIndex != INVALID_BONE_INDEX)
	{
		FTransform ParentLocal = (FTransform)Pose.LocalTransformsView[ParentIndex];
		ComponentSpace = ComponentSpace * ParentLocal;
		ParentIndex = Pose.GetLODBoneParentIndex(ParentIndex);
	}

	return ComponentSpace;
}

FMovieSceneBoneMatchData ComputeBoneMatch(
	UMovieSceneSection* TargetSection,
	const FMovieSceneBoneMatchData& Settings,
	const FBoneMatchingContext& Context)
{
	FMovieSceneBoneMatchData Result = Settings;
	Result.bIsValid = false;
	Result.bIsDirty = false;
	Result.MatchTransform = FTransform::Identity;

	if (!TargetSection || !Context.Linker || !Context.MixerTrack || Settings.BoneName.IsNone())
	{
		return Result;
	}

	FFrameTime MatchTime = ResolveMatchTime(TargetSection, Settings, Context);

	// Underlying pose: evaluate everything below the target with full conversion so each
	// contributing section's anim-space offsets are applied. AnimationSpaceRootMotion
	// captures the pre-world-conversion result.
	AnimMixerBakeEvaluation::FBakeFilter UnderlyingFilter;
	UnderlyingFilter.ExcludeSections.Add(FObjectKey(TargetSection));
	UnderlyingFilter.MaxPriority = TargetSection->GetRowIndex();
	UnderlyingFilter.bSkipRootMotionConversion = false;

	AnimMixerBakeEvaluation::FBakeResult UnderlyingResult =
		AnimMixerBakeEvaluation::EvaluateAtTime(
			Context.Linker, Context.InstanceHandle, Context.MixerTrack,
			MatchTime, UnderlyingFilter);

	// Root motion extraction zeroes the root bone, so compose the bone's bone-relative
	// transform with the anim-space root to get full component space.
	FTransform UnderlyingBoneRelToRoot = ExtractBoneComponentSpaceTransform(UnderlyingResult.Pose, Settings.BoneName);
	UnderlyingBoneRelToRoot.NormalizeRotation();
	FTransform UnderlyingRootCS = UnderlyingResult.AnimationSpaceRootMotion;
	UnderlyingRootCS.NormalizeRotation();

	FTransform UnderlyingBoneCS = UnderlyingBoneRelToRoot * UnderlyingRootCS;

	// Target section pose in isolation, no conversion: the match transform we produce
	// is applied before the target's own offsets at runtime, so we want the raw root.
	AnimMixerBakeEvaluation::FBakeFilter SectionFilter;
	SectionFilter.IncludeOnlySections.Add(FObjectKey(TargetSection));
	SectionFilter.bSkipRootMotionConversion = true;

	AnimMixerBakeEvaluation::FBakeResult SectionResult =
		AnimMixerBakeEvaluation::EvaluateAtTime(
			Context.Linker, Context.InstanceHandle, Context.MixerTrack,
			MatchTime, SectionFilter);

	FTransform SectionBoneRelToRoot = ExtractBoneComponentSpaceTransform(SectionResult.Pose, Settings.BoneName);
	SectionBoneRelToRoot.NormalizeRotation();
	FTransform SectionRootCS = SectionResult.RootMotionTransform;
	SectionRootCS.NormalizeRotation();
	FTransform SectionBoneCS = SectionBoneRelToRoot * SectionRootCS;

	if (!UnderlyingResult.IsValid() || !SectionResult.IsValid())
	{
		// The underlying or section pose evaluated to nothing - e.g. the mixer
		// track is muted/deactivated or its bound object (a muted/destroyed
		// spawnable) is gone, so the rematch was triggered against an empty
		// evaluation. Don't clobber the previously-computed match with identity;
		// preserve the last-good transform and mark it dirty so it recomputes
		// once evaluation is live again (unmute / reactivate / respawn).
		FMovieSceneBoneMatchData Preserved = Settings;
		Preserved.bIsDirty = true;
		return Preserved;
	}

	Result.MatchTransform = ComputeMatchOffset(UnderlyingBoneCS, SectionBoneCS, SectionRootCS, SectionBoneRelToRoot, Settings);
	Result.bIsValid = true;
	Result.MatchTime = MatchTime.FloorToFrame();

	return Result;
}

bool DoesSectionAffectBoneMatch(
	UMovieSceneSection* UpdatedSection,
	UMovieSceneSection* Other,
	FFrameNumber MatchKeyTime,
	UMovieSceneSection* OtherReferenceSection)
{
	if (!UpdatedSection || !Other)
	{
		return false;
	}

	if (OtherReferenceSection == UpdatedSection)
	{
		return true;
	}

	// Above Other's row: outside ComputeBoneMatch's underlying-pose filter.
	if (UpdatedSection->GetRowIndex() > Other->GetRowIndex())
	{
		return false;
	}

	// Chain entries from UpdatedSection anchor at or after its start frame,
	// so a later start can't reach back to the match key.
	if (UpdatedSection->HasStartFrame() && UpdatedSection->GetInclusiveStartFrame() > MatchKeyTime)
	{
		return false;
	}

	return true;
}

void PropagateRematch(
	UMovieSceneSection* UpdatedSection,
	const FBoneMatchingContext& Context,
	TSet<UMovieSceneSection*>& Visited)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Context.MixerTrack;
	if (!MixerTrack)
	{
		return;
	}

	for (UMovieSceneSection* Other : MixerTrack->GetAllSections())
	{
		if (!Other || Other == UpdatedSection || Visited.Contains(Other))
		{
			continue;
		}
		if (Other->IsA<UMovieSceneAnimTransitionSectionBase>())
		{
			continue;
		}

		UMovieSceneRootMotionSettingsDecoration* Decoration = Other->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!Decoration || !Decoration->HasBoneMatch())
		{
			continue;
		}

		TMovieSceneChannelData<FMovieSceneBoneMatchData> Data = Decoration->BoneMatchChannel.GetData();
		TArrayView<FMovieSceneBoneMatchData> Values = Data.GetValues();
		TArrayView<const FFrameNumber> Times = Data.GetTimes();
		if (Values.Num() == 0 || Times.Num() == 0)
		{
			continue;
		}

		FMovieSceneBoneMatchData& KeyValue = Values[0];
		const FFrameNumber KeyTime = Times[0];
		UMovieSceneSection* RefSection = KeyValue.ReferenceSection.Get();

		if (!DoesSectionAffectBoneMatch(UpdatedSection, Other, KeyTime, RefSection))
		{
			continue;
		}

		Visited.Add(Other);

		// Key must remain inside the (target, reference) overlap to be valid.
		if (RefSection)
		{
			TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(Other->GetRange(), RefSection->GetRange());
			if (Overlap.IsEmpty() || !Overlap.Contains(KeyTime))
			{
				Decoration->Modify();
				KeyValue.bIsValid = false;
				KeyValue.bIsDirty = true;
				continue;
			}
		}

		FBoneMatchingContext ChainContext = Context;
		ChainContext.CurrentTime = FFrameTime(KeyTime);

		FMovieSceneBoneMatchData Result = ComputeBoneMatch(Other, KeyValue, ChainContext);

		// Modify before mutating so the propagated change is undoable and signature consumers see it.
		Decoration->Modify();
		KeyValue = Result;

		if (Result.bIsValid)
		{
			PropagateRematch(Other, Context, Visited);
		}
	}
}

} // namespace UE::MovieScene::AnimMixerBoneMatching
