// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerBusUtils.h"

#include "MovieSceneAnimBusSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"

// ---------------------------------------------------------------------------
// FBusDependencyGraph
// ---------------------------------------------------------------------------

TArray<int32> FBusDependencyGraph::TopologicalSort()
{
	TMap<FName, int32> WriterByBus;
	for (int32 i = 0; i < Nodes.Num(); ++i)
	{
		if (!Nodes[i].WritesToBus.IsNone())
		{
			WriterByBus.Add(Nodes[i].WritesToBus, i);
		}
		Nodes[i].InDegree = 0;
		Nodes[i].Dependents.Reset();
	}

	for (int32 ReaderIdx = 0; ReaderIdx < Nodes.Num(); ++ReaderIdx)
	{
		for (const FName& ReadBus : Nodes[ReaderIdx].ReadsFromBuses)
		{
			if (ReadBus == Nodes[ReaderIdx].WritesToBus)
			{
				continue;
			}
			if (int32* WriterIdx = WriterByBus.Find(ReadBus))
			{
				if (*WriterIdx != ReaderIdx)
				{
					Nodes[*WriterIdx].Dependents.AddUnique(ReaderIdx);
					Nodes[ReaderIdx].InDegree++;
				}
			}
		}
	}

	TArray<int32> Queue;
	for (int32 i = 0; i < Nodes.Num(); ++i)
	{
		if (Nodes[i].InDegree == 0)
		{
			Queue.Add(i);
		}
	}

	TArray<int32> Result;
	Result.Reserve(Nodes.Num());
	int32 QueueHead = 0;
	while (QueueHead < Queue.Num())
	{
		int32 Current = Queue[QueueHead++];
		Result.Add(Current);
		for (int32 DepIdx : Nodes[Current].Dependents)
		{
			if (--Nodes[DepIdx].InDegree == 0)
			{
				Queue.Add(DepIdx);
			}
		}
	}

	return Result;
}

bool FBusDependencyGraph::IsReachable(FName StartBus, FName TargetBus) const
{
	TMap<FName, TArrayView<const FName>> AdjMap;
	for (const FNode& Node : Nodes)
	{
		if (!Node.WritesToBus.IsNone() && !Node.ReadsFromBuses.IsEmpty())
		{
			AdjMap.Add(Node.WritesToBus, Node.ReadsFromBuses);
		}
	}

	TSet<FName> Visited;
	TArray<FName> Stack;
	Stack.Add(StartBus);
	while (!Stack.IsEmpty())
	{
		FName Current = Stack.Pop();
		if (Current == TargetBus)
		{
			return true;
		}
		if (Visited.Contains(Current))
		{
			continue;
		}
		Visited.Add(Current);

		if (const TArrayView<const FName>* Reads = AdjMap.Find(Current))
		{
			for (const FName& Name : *Reads)
			{
				Stack.Add(Name);
			}
		}
	}
	return false;
}

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace
{

TArray<FName> GatherBusReads(UMovieSceneAnimationMixerTrack* Track)
{
	TArray<FName> Reads;
	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Section))
		{
			if (!BusSection->BusName.IsNone())
			{
				Reads.AddUnique(BusSection->BusName);
			}
		}
	}
	return Reads;
}

FBusDependencyGraph BuildGraphFromTracks(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	FBusDependencyGraph Graph;
	for (UMovieSceneAnimationMixerTrack* Track : MixerTracks)
	{
		const FMovieSceneAnimBusTarget* BusTarget = Track->MixedAnimationTarget.GetPtr<FMovieSceneAnimBusTarget>();
		if (!BusTarget || BusTarget->BusName.IsNone())
		{
			continue;
		}

		FBusDependencyGraph::FNode& Node = Graph.Nodes.AddDefaulted_GetRef();
		Node.WritesToBus = BusTarget->BusName;
		Node.ReadsFromBuses = GatherBusReads(Track);
	}
	return Graph;
}

void GatherMixerTracksRecursive(UMovieSceneSequence* Sequence, TArray<UMovieSceneAnimationMixerTrack*>& OutTracks, TSet<UMovieSceneSequence*>& Visited)
{
	if (!Sequence || Visited.Contains(Sequence))
	{
		return;
	}
	Visited.Add(Sequence);

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const UMovieScene* ConstMovieScene = MovieScene;
	for (const FMovieSceneBinding& Binding : ConstMovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
			{
				OutTracks.Add(MixerTrack);
			}
		}
	}

	for (UMovieSceneTrack* MasterTrack : MovieScene->GetTracks())
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(MasterTrack))
		{
			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					if (UMovieSceneSequence* SubSequence = SubSection->GetSequence())
					{
						GatherMixerTracksRecursive(SubSequence, OutTracks, Visited);
					}
				}
			}
		}
	}
}

} // anonymous namespace

// ---------------------------------------------------------------------------
// FAnimMixerBusUtils
// ---------------------------------------------------------------------------

TArray<FName> FAnimMixerBusUtils::GatherBusNamesFromSequence(UMovieSceneSequence* RootSequence)
{
	TSet<FName> BusNames;

	TArray<UMovieSceneAnimationMixerTrack*> Tracks;
	TSet<UMovieSceneSequence*> Visited;
	GatherMixerTracksRecursive(RootSequence, Tracks, Visited);

	for (UMovieSceneAnimationMixerTrack* Track : Tracks)
	{
		if (const FMovieSceneAnimBusTarget* BusTarget = Track->MixedAnimationTarget.GetPtr<FMovieSceneAnimBusTarget>())
		{
			if (!BusTarget->BusName.IsNone())
			{
				BusNames.Add(BusTarget->BusName);
			}
		}

		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Section))
			{
				if (!BusSection->BusName.IsNone())
				{
					BusNames.Add(BusSection->BusName);
				}
			}
		}
	}

	TArray<FName> Result = BusNames.Array();
	Result.Sort(FNameLexicalLess());
	return Result;
}

TArray<FName> FAnimMixerBusUtils::ComputeBusEvaluationOrder(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	FBusDependencyGraph Graph = BuildGraphFromTracks(MixerTracks);
	if (Graph.Nodes.IsEmpty())
	{
		return {};
	}

	// Mark self-referencing nodes (TopologicalSort skips self-edges so they
	// wouldn't be caught by the cycle check below)
	TSet<FName> SelfReferencingBuses;
	for (const FBusDependencyGraph::FNode& Node : Graph.Nodes)
	{
		if (Node.ReadsFromBuses.Contains(Node.WritesToBus))
		{
			SelfReferencingBuses.Add(Node.WritesToBus);
			ensureMsgf(false, TEXT("Bus '%s' reads from itself."), *Node.WritesToBus.ToString());
		}
	}

	TArray<int32> Sorted = Graph.TopologicalSort();

	if (Sorted.Num() < Graph.Nodes.Num())
	{
		for (const FBusDependencyGraph::FNode& Node : Graph.Nodes)
		{
			if (Node.InDegree > 0)
			{
				ensureMsgf(false, TEXT("Bus dependency cycle detected involving bus '%s'."), *Node.WritesToBus.ToString());
			}
		}
	}

	TArray<FName> Result;
	Result.Reserve(Sorted.Num());
	for (int32 Idx : Sorted)
	{
		FName BusName = Graph.Nodes[Idx].WritesToBus;
		if (!SelfReferencingBuses.Contains(BusName))
		{
			Result.Add(BusName);
		}
	}
	return Result;
}

FAnimMixerBusValidationResult FAnimMixerBusUtils::ValidateBusTopology(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	FAnimMixerBusValidationResult Result;

	TArray<FName> Writers;
	TArray<FName> Readers;

	for (UMovieSceneAnimationMixerTrack* Track : MixerTracks)
	{
		if (const FMovieSceneAnimBusTarget* BusTarget = Track->MixedAnimationTarget.GetPtr<FMovieSceneAnimBusTarget>())
		{
			if (!BusTarget->BusName.IsNone())
			{
				Writers.Add(BusTarget->BusName);
			}
		}

		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Section))
			{
				if (!BusSection->BusName.IsNone())
				{
					Readers.Add(BusSection->BusName);
				}
			}
		}
	}

	// Duplicate writer detection
	TMap<FName, int32> WriterCounts;
	for (const FName& Name : Writers)
	{
		WriterCounts.FindOrAdd(Name)++;
	}
	for (const auto& Pair : WriterCounts)
	{
		if (Pair.Value > 1)
		{
			Result.Warnings.Add(FString::Printf(
				TEXT("Multiple mixers write to bus '%s'. Only one will take effect per frame."),
				*Pair.Key.ToString()));
		}
	}

	// Orphan bus section detection
	TSet<FName> WrittenBuses(Writers);
	for (const FName& Name : Readers)
	{
		if (!WrittenBuses.Contains(Name))
		{
			Result.Warnings.Add(FString::Printf(
				TEXT("Bus section references bus '%s' but no mixer targets that bus."),
				*Name.ToString()));
		}
	}

	// Orphan target detection
	TSet<FName> ReadBuses(Readers);
	for (const FName& Name : Writers)
	{
		if (!ReadBuses.Contains(Name))
		{
			Result.Warnings.Add(FString::Printf(
				TEXT("Mixer targets bus '%s' but no bus section reads from it."),
				*Name.ToString()));
		}
	}

	// Self-reference + cycle detection
	FBusDependencyGraph Graph = BuildGraphFromTracks(MixerTracks);

	for (const FBusDependencyGraph::FNode& Node : Graph.Nodes)
	{
		if (Node.ReadsFromBuses.Contains(Node.WritesToBus))
		{
			Result.Errors.Add(FString::Printf(
				TEXT("Mixer reads from bus '%s' which it also writes to."),
				*Node.WritesToBus.ToString()));
		}
	}

	TArray<int32> Sorted = Graph.TopologicalSort();
	if (Sorted.Num() < Graph.Nodes.Num())
	{
		TArray<FString> CycleBuses;
		for (int32 i = 0; i < Graph.Nodes.Num(); ++i)
		{
			if (Graph.Nodes[i].InDegree > 0)
			{
				CycleBuses.Add(Graph.Nodes[i].WritesToBus.ToString());
			}
		}
		Result.Errors.Add(FString::Printf(
			TEXT("Bus dependency cycle detected involving buses: %s."),
			*FString::Join(CycleBuses, TEXT(", "))));
	}

	return Result;
}

bool FAnimMixerBusUtils::WouldBusSectionCreateCycle(FName BusName, UMovieSceneAnimationMixerTrack* Track,
	TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	if (!Track || BusName.IsNone())
	{
		return false;
	}

	const FMovieSceneAnimBusTarget* BusTarget = Track->MixedAnimationTarget.GetPtr<FMovieSceneAnimBusTarget>();
	if (!BusTarget || BusTarget->BusName.IsNone())
	{
		return false;
	}

	if (BusTarget->BusName == BusName)
	{
		return true;
	}

	FBusDependencyGraph Graph = BuildGraphFromTracks(MixerTracks);

	// Add the hypothetical read
	for (FBusDependencyGraph::FNode& Node : Graph.Nodes)
	{
		if (Node.WritesToBus == BusTarget->BusName)
		{
			Node.ReadsFromBuses.AddUnique(BusName);
			break;
		}
	}

	return Graph.IsReachable(BusName, BusTarget->BusName);
}

bool FAnimMixerBusUtils::WouldBusTargetCreateCycle(FName BusName, UMovieSceneAnimationMixerTrack* Track,
	TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	if (!Track || BusName.IsNone())
	{
		return false;
	}

	TArray<FName> TrackReads = GatherBusReads(Track);
	if (TrackReads.IsEmpty())
	{
		return false;
	}

	if (TrackReads.Contains(BusName))
	{
		return true;
	}

	FBusDependencyGraph Graph = BuildGraphFromTracks(MixerTracks);

	// Add the hypothetical node
	FBusDependencyGraph::FNode& HypNode = Graph.Nodes.AddDefaulted_GetRef();
	HypNode.WritesToBus = BusName;
	HypNode.ReadsFromBuses = TrackReads;

	for (const FName& ReadBus : TrackReads)
	{
		if (Graph.IsReachable(ReadBus, BusName))
		{
			return true;
		}
	}

	return false;
}
