// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"

namespace UE::MovieScene
{

	const UE::Anim::FAttributeId FAnimMixerComponentTypes::RootTransformAttributeId(TEXT("RootTransform"), FCompactPoseBoneIndex(0));
	const UE::Anim::FAttributeId FAnimMixerComponentTypes::RootTransformWeightAttributeId(TEXT("RootTransformWeight"), FCompactPoseBoneIndex(0));
	const UE::Anim::FAttributeId FAnimMixerComponentTypes::RootTransformIsAuthoritativeAttributeId(TEXT("RootTransformIsAuthoritative"), FCompactPoseBoneIndex(0));

	static TUniquePtr<FAnimMixerComponentTypes> GAnimMixerComponentTypes;
	static bool GAnimMixerComponentTypesDestroyed = false;

	FAnimMixerComponentTypes* FAnimMixerComponentTypes::Get()
	{
		if (!GAnimMixerComponentTypes.IsValid())
		{
			check(!GAnimMixerComponentTypesDestroyed);
			GAnimMixerComponentTypes.Reset(new FAnimMixerComponentTypes);
		}
		return GAnimMixerComponentTypes.Get();
	}

	void FAnimMixerComponentTypes::Destroy()
	{
		GAnimMixerComponentTypes.Reset();
		GAnimMixerComponentTypesDestroyed = true;
	}

	FAnimMixerComponentTypes::FAnimMixerComponentTypes()
	{
		FBuiltInComponentTypes* BuiltInTypes = FBuiltInComponentTypes::Get();
		FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

		ComponentRegistry->NewComponentType(&Priority, TEXT("Mixed Animation Priority"));
		ComponentRegistry->NewComponentType(&Target, TEXT("Mixed Animation Target"));
		ComponentRegistry->NewComponentType(&Task, TEXT("Mixed Animation Task"));
		ComponentRegistry->NewComponentType(&MixerTask, TEXT("Mixed Animation Mixer Task"));
		ComponentRegistry->NewComponentType(&BlendProviderData, TEXT("Mixed Animation Blend Provider Data"));
		ComponentRegistry->NewComponentType(&RootMotionSettings, TEXT("Root Motion Settings"));
		ComponentRegistry->NewComponentType(&MeshComponent, TEXT("Mixed Animation Mesh Component"));
		ComponentRegistry->NewComponentType(&MixerRootMotion, TEXT("Root Motion"));
		ComponentRegistry->NewComponentType(&MixerEntry, TEXT("MixerEntry"));
		ComponentRegistry->NewComponentType(&RootDestination, TEXT("Root Destination"));
		ComponentRegistry->NewComponentType(&MixerLayer, TEXT("Mixer Layer"));
		ComponentRegistry->NewComponentType(&MirrorTable, TEXT("Mirror Table"));
		ComponentRegistry->NewComponentType(&BoneMatchTransform, TEXT("Bone Match Transform"));
		ComponentRegistry->NewComponentType(&GapBehavior, TEXT("Gap Behavior"));
		ComponentRegistry->NewComponentType(&EntityOwner, TEXT("Entity Owner"));
		ComponentRegistry->NewComponentType(&LayerBlendTask, TEXT("Layer Blend Task"));
		ComponentRegistry->NewComponentType(&BusName, TEXT("Bus Name"));

		Tags.Transition = ComponentRegistry->NewTag(TEXT("Transition"));
		Tags.SkipPoseWeight = ComponentRegistry->NewTag(TEXT("Skip Pose Weight"));
		Tags.LayerWeight = ComponentRegistry->NewTag(TEXT("Layer Weight"));
		Tags.PreBakeRootMotion = ComponentRegistry->NewTag(TEXT("Pre Bake Root Motion"));

		ComponentRegistry->Factories.DuplicateChildComponent(Priority);
		ComponentRegistry->Factories.DuplicateChildComponent(Target);
		ComponentRegistry->Factories.DuplicateChildComponent(Task);
		ComponentRegistry->Factories.DuplicateChildComponent(MixerTask);
		ComponentRegistry->Factories.DuplicateChildComponent(BlendProviderData);
		ComponentRegistry->Factories.DuplicateChildComponent(RootMotionSettings);
		ComponentRegistry->Factories.DuplicateChildComponent(MeshComponent);
		ComponentRegistry->Factories.DuplicateChildComponent(MixerRootMotion);
		ComponentRegistry->Factories.DuplicateChildComponent(RootDestination);
		ComponentRegistry->Factories.DuplicateChildComponent(MixerLayer);
		ComponentRegistry->Factories.DuplicateChildComponent(GapBehavior);
		ComponentRegistry->Factories.DuplicateChildComponent(EntityOwner);
		ComponentRegistry->Factories.DuplicateChildComponent(BoneMatchTransform);

		ComponentRegistry->Factories.DuplicateChildComponent(MirrorTable);
		ComponentRegistry->Factories.DuplicateChildComponent(BusName);
		
		ComponentRegistry->Factories.DefineChildComponent(RootDestination, MixerRootMotion);
		ComponentRegistry->Factories.DefineChildComponent(Task, MixerEntry);

		ComponentRegistry->Factories.DefineChildComponent(Tags.Transition, Tags.Transition);
		ComponentRegistry->Factories.DefineChildComponent(Tags.LayerWeight, Tags.LayerWeight);
	}

}