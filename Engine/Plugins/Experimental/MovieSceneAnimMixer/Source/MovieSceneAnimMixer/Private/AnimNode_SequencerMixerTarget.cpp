// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_SequencerMixerTarget.h"

#include "AnimSubsystem_SequencerMixer.h"
#include "DataRegistryTypes.h"
#include "GenerationTools.h"
#include "Animation/AnimInstanceProxy.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Graph/AnimNext_LODPose.h"
#include "Components/SkeletalMeshComponent.h"


const FName FAnimNode_SequencerMixerTarget::DefaultTargetName = FName(TEXT("DefaultTarget"));

FAnimNode_SequencerMixerTarget::FAnimNode_SequencerMixerTarget()
	: TargetName(DefaultTargetName)
{
}

void FAnimNode_SequencerMixerTarget::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_Base::Initialize_AnyThread(Context);
	SourcePose.Initialize(Context);
}

void FAnimNode_SequencerMixerTarget::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	FAnimNode_Base::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);
}

void FAnimNode_SequencerMixerTarget::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	FAnimNode_Base::Update_AnyThread(Context);
	SourcePose.Update(Context);

	// PreUpdateAnimation has already cleared NotifyQueue and PostUpdateAnimation has not
	// yet dispatched: the only window where adds will survive to dispatch this frame.
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
	{
		FAnimSubsystem_SequencerMixer& MixerSubsystem = AnimInstance->GetSubsystem<FAnimSubsystem_SequencerMixer>();
		for (FSequencerMixerPendingNotifyBatch& Batch : MixerSubsystem.ConsumePendingNotifyBatches(TargetName))
		{
			Context.AnimInstanceProxy->AddAnimNotifies(Batch.Notifies, Batch.Weight);
		}
	}
}

void FAnimNode_SequencerMixerTarget::Evaluate_AnyThread(FPoseContext& Output)
{
	SourcePose.Evaluate(Output);

	ApplySequencerMixedPose(Output, TargetName);
}

void FAnimNode_SequencerMixerTarget::ApplySequencerMixedPose(FPoseContext& Output, FName InTargetName)
{
	UAnimInstance* AnimInstance = Cast<UAnimInstance>(Output.GetAnimInstanceObject());
	
	const FAnimSubsystem_SequencerMixer& MixerSubsystem = AnimInstance->GetSubsystem<FAnimSubsystem_SequencerMixer>();
	
	const TSharedPtr<FAnimNextEvaluationTask>* MixerTask = MixerSubsystem.GetEvalTask(InTargetName);
	if (!MixerTask || !MixerTask->IsValid())
	{
		return;
	}

	USkeletalMeshComponent* SkeletalMeshComponent = AnimInstance->GetSkelMeshComponent();
	
	if (!SkeletalMeshComponent)
	{
		return;
	}
	
	int32 LODLevel = Output.AnimInstanceProxy->GetLODLevel();

	using namespace UE::UAF;
	
	FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(SkeletalMeshComponent);
	FAnimNextGraphReferencePose GraphReferencePose(RefPoseHandle);

	const UE::UAF::FReferencePose& RefPose = RefPoseHandle.GetRef<UE::UAF::FReferencePose>();
	FAnimNextGraphLODPose ResultPose;
	ResultPose.LODPose = FLODPoseHeap(RefPose, LODLevel, true, Output.ExpectsAdditivePose());

	{
		bool bHasValidOutput = false;
		FEvaluationVM EvaluationVM(EEvaluationFlags::All, RefPose, LODLevel);

		// Use the output pose in the mixer as the 'base pose'- push this pose first.
		FKeyframeState Keyframe = EvaluationVM.MakeUninitializedKeyframe(false);
		FGenerationTools::RemapPose(Output, Keyframe.Pose);
		Keyframe.Curves.CopyFrom(Output.Curve);
		// TODO: There is not a remap attributes the other way- do we need one?
		Keyframe.Attributes.CopyFrom(Output.CustomAttributes);

		EvaluationVM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
		
		MixerTask->Get()->Execute(EvaluationVM);

		TUniquePtr<FKeyframeState> EvaluatedKeyframe;
		if (EvaluationVM.PopValue(KEYFRAME_STACK_NAME, EvaluatedKeyframe))
		{
			ResultPose.LODPose.CopyFrom(EvaluatedKeyframe->Pose);
			ResultPose.Curves.CopyFrom(EvaluatedKeyframe->Curves);
			ResultPose.Attributes.CopyFrom(EvaluatedKeyframe->Attributes);
			bHasValidOutput = true;
		}

		if (!bHasValidOutput)
		{
			// We need to output a valid pose, generate one
			FKeyframeState ReferenceKeyframe = EvaluationVM.MakeReferenceKeyframe(Output.ExpectsAdditivePose());
			ResultPose.LODPose.CopyFrom(ReferenceKeyframe.Pose);
			ResultPose.Curves.CopyFrom(ReferenceKeyframe.Curves);
			ResultPose.Attributes.CopyFrom(ReferenceKeyframe.Attributes);
		}
	}
	
	FGenerationTools::RemapPose(ResultPose.LODPose, Output);
	Output.Curve.CopyFrom(ResultPose.Curves);
	FGenerationTools::RemapAttributes(ResultPose.LODPose, ResultPose.Attributes, Output);
}
