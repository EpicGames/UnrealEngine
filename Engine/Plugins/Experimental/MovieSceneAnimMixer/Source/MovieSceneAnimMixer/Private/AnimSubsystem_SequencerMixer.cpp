// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSubsystem_SequencerMixer.h"

void FAnimSubsystem_SequencerMixer::Initialize_WorkerThread()
{
	TargetToTaskMap.Empty();
	TargetToPendingNotifiesMap.Empty();
}

void FAnimSubsystem_SequencerMixer::RegisterEvalTask(const FName TargetName,
                                                     TSharedPtr<FAnimNextEvaluationTask> EvalTask)
{
	TargetToTaskMap.FindOrAdd(TargetName) = EvalTask;
}

const TSharedPtr<FAnimNextEvaluationTask>* FAnimSubsystem_SequencerMixer::GetEvalTask(const FName TargetName) const
{
	return TargetToTaskMap.Find(TargetName);
}

void FAnimSubsystem_SequencerMixer::ResetEvalTasks()
{
	TargetToTaskMap.Reset();
	TargetToPendingNotifiesMap.Reset();
}

void FAnimSubsystem_SequencerMixer::StagePendingNotifyBatches(const FName TargetName, TArray<FSequencerMixerPendingNotifyBatch>&& InBatches)
{
	TargetToPendingNotifiesMap.FindOrAdd(TargetName) = MoveTemp(InBatches);
}

TArray<FSequencerMixerPendingNotifyBatch> FAnimSubsystem_SequencerMixer::ConsumePendingNotifyBatches(const FName TargetName)
{
	TArray<FSequencerMixerPendingNotifyBatch> Result;
	if (TArray<FSequencerMixerPendingNotifyBatch>* Found = TargetToPendingNotifiesMap.Find(TargetName))
	{
		Result = MoveTemp(*Found);
		TargetToPendingNotifiesMap.Remove(TargetName);
	}
	return Result;
}
