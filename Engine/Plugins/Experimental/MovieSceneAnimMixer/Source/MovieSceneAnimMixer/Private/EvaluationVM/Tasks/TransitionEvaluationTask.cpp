// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationVM/Tasks/TransitionEvaluationTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransitionEvaluationTask)

void FAnimNextTransitionEvaluationTask::Update(
	const TSharedPtr<FAnimNextEvaluationTask>& InFromTask,
	const TSharedPtr<FAnimNextEvaluationTask>& InToTask,
	double InBlendWeight,
	float InDeltaTime)
{
	FromTask = InFromTask;
	ToTask = InToTask;
	BlendWeight = InBlendWeight;
	DeltaTime = InDeltaTime;
}
