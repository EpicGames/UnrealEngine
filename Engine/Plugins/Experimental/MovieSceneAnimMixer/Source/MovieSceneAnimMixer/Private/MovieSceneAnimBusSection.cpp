// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimBusSection.h"

#include "AnimMixerComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimBusSection)

#define LOCTEXT_NAMESPACE "MovieSceneAnimBusSection"

UMovieSceneAnimBusSection::UMovieSceneAnimBusSection()
{
	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());
	BlendType = EMovieSceneBlendType::Absolute;
	SetColorTint(BusSectionTintColor);

	// Default weight of 1.0 (full contribution)
	Weight.SetDefault(1.0f);
}

EMovieSceneChannelProxyType UMovieSceneAnimBusSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR
	static FMovieSceneChannelMetaData MetaData("Weight", LOCTEXT("WeightChannelName", "Weight"));
	MetaData.bCanCollapseToTrack = false;

	Channels.Add(Weight, MetaData, TMovieSceneExternalValue<float>());
#else
	Channels.Add(Weight);
#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneAnimBusSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FAnimMixerComponentTypes* AnimMixerComponentTypes = FAnimMixerComponentTypes::Get();
	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

	BuildDefaultComponents(EntityLinker, Params, OutImportedEntity);

	if (BusName.IsNone())
	{
		return;
	}

	// Find the owning mixer track and layer for this section
	UMovieSceneAnimationMixerTrack* MixerTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>();
	UMovieSceneAnimationMixerLayer* Layer = nullptr;
	if (MixerTrack)
	{
		Layer = MixerTrack->GetLayerForSection(this);
	}

	// Task and MixerEntry start as null; UMovieSceneAnimBusSystem populates the
	// Task with a FMovieSceneAnimBusReadTask each frame. The mixer system then
	// wraps it into an entry like any other section.
	TSharedPtr<FAnimNextEvaluationTask> NullTask;
	TSharedPtr<FMovieSceneAnimMixerEntry> NullEntry;

	// Resolve target from the owning mixer track
	TInstancedStruct<FMovieSceneMixedAnimationTarget> ResolvedTarget;
	if (MixerTrack && MixerTrack->MixedAnimationTarget.IsValid())
	{
		ResolvedTarget = MixerTrack->MixedAnimationTarget;
	}

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
		.Add(BuiltInComponentTypes->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
		.Add(AnimMixerComponentTypes->Priority, GetRowIndex())
		.Add(AnimMixerComponentTypes->Task, NullTask)
		.Add(AnimMixerComponentTypes->MixerEntry, NullEntry)
		.Add(AnimMixerComponentTypes->EntityOwner, FObjectKey(this))
		.Add(AnimMixerComponentTypes->BusName, BusName)
		.AddConditional(BuiltInComponentTypes->WeightChannel, &Weight, Weight.HasAnyData())
		.AddConditional(AnimMixerComponentTypes->MixerLayer, Layer, Layer != nullptr)
		.AddConditional(AnimMixerComponentTypes->Target, ResolvedTarget, ResolvedTarget.IsValid())
		.AddMutualComponents());
}

#undef LOCTEXT_NAMESPACE
