// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimCrossfadeTransitionSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/Tasks/ExecuteProgram.h"
#include "EvaluationVM/Tasks/TransitionEvaluationTask.h"
#include "Systems/MovieSceneAnimMixerSystem.h"

/**
 * FAnimNextCrossfadeTransitionTask
 *
 * A simple crossfade transition task that blends between two poses
 * based on a blend weight.
 */
struct FAnimNextCrossfadeTransitionTask : public FAnimNextTransitionEvaluationTask
{
	DECLARE_ANIM_EVALUATION_TASK(FAnimNextCrossfadeTransitionTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override
	{
		// Handle edge cases where one or both tasks are missing
		if (!FromTask && !ToTask)
		{
			return;
		}
		if (!FromTask)
		{
			ToTask->Execute(VM);
			return;
		}
		if (!ToTask)
		{
			FromTask->Execute(VM);
			return;
		}

		// Execute from task first (pushes pose onto stack)
		FromTask->Execute(VM);

		// Execute to task (pushes second pose onto stack)
		ToTask->Execute(VM);

		// Blend the two poses using the blend weight
		// FAnimNextBlendTwoKeyframesPreserveRootMotionTask blends top two stack items
		FAnimNextBlendTwoKeyframesPreserveRootMotionTask BlendTask =
			FAnimNextBlendTwoKeyframesPreserveRootMotionTask::Make(static_cast<float>(BlendWeight));
		BlendTask.Execute(VM);
	}
};

UMovieSceneAnimCrossfadeTransitionSection::UMovieSceneAnimCrossfadeTransitionSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	// Initialize with a default linear blend curve
	InitializeDefaultCurve();
}

void UMovieSceneAnimCrossfadeTransitionSection::InitializeDefaultCurve()
{
	BlendCurve.Reset();

	TRange<FFrameNumber> Range = GetRange();

	if (Range.IsEmpty() || !Range.HasLowerBound() || !Range.HasUpperBound())
	{
		BlendCurve.SetDefault(0.0f);
		return;
	}

	// Add keyframes with cubic interpolation for smooth ease-in/ease-out (like easing curves)
	FFrameNumber StartFrame = Range.GetLowerBoundValue();
	FFrameNumber EndFrame = Range.GetUpperBoundValue();
	BlendCurve.AddCubicKey(StartFrame, 0.0f, RCTM_Auto);
	BlendCurve.AddCubicKey(EndFrame, 1.0f, RCTM_Auto);
}

TSharedPtr<FAnimNextTransitionEvaluationTask> UMovieSceneAnimCrossfadeTransitionSection::CreateTransitionTask() const
{
	return MakeShared<FAnimNextCrossfadeTransitionTask>();
}

FName UMovieSceneAnimCrossfadeTransitionSection::GetTransitionIconStyleName() const
{
	// TODO: Register a custom icon for crossfade transitions
	// For now, return NAME_None to skip icon drawing
	return NAME_None;
}

FText UMovieSceneAnimCrossfadeTransitionSection::GetTransitionDisplayName() const
{
	return NSLOCTEXT("MovieSceneAnimCrossfadeTransitionSection", "CrossfadeDisplayName", "Crossfade");
}

void UMovieSceneAnimCrossfadeTransitionSection::RebuildChannelProxy(FMovieSceneChannelProxyData& Channels)
{
#if WITH_EDITOR
	FMovieSceneChannelMetaData MetaData(TEXT("BlendCurve"), NSLOCTEXT("MovieSceneAnimCrossfadeTransitionSection", "BlendCurve", "Blend"));
	// Don't collapse to track - show as expandable row so users can access the blend curve
	MetaData.bCanCollapseToTrack = false;
	Channels.Add(
		BlendCurve,
		MetaData,
		TMovieSceneExternalValue<float>()
	);
#else
	Channels.Add(BlendCurve);
#endif
}
