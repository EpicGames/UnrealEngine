// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimInertialDeadBlendTransitionSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/Tasks/TransitionEvaluationTask.h"
#include "EvaluationVM/Tasks/DeadBlending.h"
#include "EvaluationVM/Tasks/StoreKeyframe.h"
#include "Traits/Inertialization.h"
#include "TransformArray.h"
#include "TransformArrayOperations.h"
#include "Curves/CurveFloat.h"
#include "MovieScene.h"

/**
 * FAnimNextInertialDeadBlendTransitionTask
 *
 * A stateful transition task that performs inertial dead blending.
 * Captures poses from the "from" section to compute velocity, then
 * extrapolates that motion while blending toward the "to" pose.
 * Uses the pre-existing UAF Evaluation Tasks internally for the actual dead blend calculations.
 *
 * Execution flow:
 * - Frame 1: Capture from-pose, output from-pose (need velocity data, avoid pop)
 * - Frame 2: Capture second from-pose, compute velocity, initialize dead blend, apply
 * - Subsequent frames: Apply dead blending to extrapolate and blend
 */
struct FAnimNextInertialDeadBlendTransitionTask : public FAnimNextTransitionEvaluationTask
{
	DECLARE_ANIM_EVALUATION_TASK(FAnimNextInertialDeadBlendTransitionTask)

	// Parameters (set at creation from section properties)
	float BlendDuration = 0.3f;
	EAlphaBlendOption BlendModeParam = EAlphaBlendOption::HermiteCubic;
	TWeakObjectPtr<UCurveFloat> CustomBlendCurveParam = nullptr;
	UE::UAF::FDeadBlendTransitionTaskParameters ExtrapolationParams;

	// Persistent state
	mutable UE::UAF::FTransformArraySoAHeap CurrFromPose;
	mutable UE::UAF::FTransformArraySoAHeap PrevFromPose;
	mutable UE::UAF::FDeadBlendingState DeadBlendState;
	mutable float PoseDeltaTime = 0.0f;
	mutable float TimeSinceTransitionStart = 0.0f;
	mutable bool bHasCurrPose = false;
	mutable bool bHasPrevPose = false;
	mutable bool bTransitionInitialized = false;

	virtual void Update(
		const TSharedPtr<FAnimNextEvaluationTask>& InFromTask,
		const TSharedPtr<FAnimNextEvaluationTask>& InToTask,
		double InBlendWeight,
		float InDeltaTime) override
	{
		// Call base to update from/to tasks and blend weight
		FAnimNextTransitionEvaluationTask::Update(InFromTask, InToTask, InBlendWeight, InDeltaTime);

		// Accumulate time since transition started
		if (bTransitionInitialized)
		{
			TimeSinceTransitionStart += InDeltaTime;
		}

		// Accumulate delta time for pose velocity computation
		PoseDeltaTime += InDeltaTime;
	}

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override
	{
		using namespace UE::UAF;

		// Handle edge cases
		if (!FromTask && !ToTask)
		{
			return;
		}
		if (!FromTask)
		{
			ToTask->Execute(VM);
			return;
		}
		if (!ToTask)
		{
			FromTask->Execute(VM);
			return;
		}

		// Phase 1: output the from-pose and, on the first evaluation, snapshot it as
		// the velocity baseline for Phase 2. Phase 2 only fires once PoseDeltaTime
		// exceeds UE_SMALL_NUMBER, so Phase 1 re-outputs the from-pose every eval
		// until then to keep the stack contract satisfied.
		if (!bHasPrevPose && (!bHasCurrPose || PoseDeltaTime <= UE_SMALL_NUMBER))
		{
			FromTask->Execute(VM);

			if (!bHasCurrPose)
			{
				if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
				{
					if (*Keyframe)
					{
						CurrFromPose.SetNumUninitialized((*Keyframe)->Pose.LocalTransforms.Num());
						CopyTransforms(CurrFromPose.GetView(), (*Keyframe)->Pose.LocalTransforms.GetConstView());
						bHasCurrPose = true;
					}
				}

				PoseDeltaTime = 0.0f;
			}

			return;
		}

		// Phase 2: Second frame - we have CurrPose, capture another for velocity
		if (!bHasPrevPose && PoseDeltaTime > UE_SMALL_NUMBER)
		{
			// Swap poses - move current to previous
			PrevFromPose = MoveTemp(CurrFromPose);

			// Execute from-task to capture new current pose
			FromTask->Execute(VM);

			const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
			if (!Keyframe || !(*Keyframe))
			{
				// Failed to capture pose - pop any stack entry and restore previous state
				TUniquePtr<FKeyframeState> DiscardedKeyframe;
				(void)VM.PopValue(KEYFRAME_STACK_NAME, DiscardedKeyframe);
				CurrFromPose = MoveTemp(PrevFromPose);
				return;
			}

			CurrFromPose.SetNumUninitialized((*Keyframe)->Pose.LocalTransforms.Num());
			CopyTransforms(CurrFromPose.GetView(), (*Keyframe)->Pose.LocalTransforms.GetConstView());

			// Pop the from-pose (discard it)
			TUniquePtr<FKeyframeState> PoppedKeyframe;
			(void)VM.PopValue(KEYFRAME_STACK_NAME, PoppedKeyframe);

			bHasPrevPose = true;

			// Now execute to-task to get destination pose
			ToTask->Execute(VM);

			// Initialize dead blending state with velocity
			if (const TUniquePtr<FKeyframeState>* ToKeyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
			{
				// Use the existing transition task to initialize our state
				FAnimNextDeadBlendingTransitionTask TransitionTask = FAnimNextDeadBlendingTransitionTask::Make(
					&DeadBlendState,
					&CurrFromPose,
					&PrevFromPose,
					PoseDeltaTime,
					ExtrapolationParams);

				// Execute transition task - it will read destination from stack and fill DeadBlendState
				TransitionTask.Execute(VM);
			}

			bTransitionInitialized = true;
			TimeSinceTransitionStart = 0.0f;
			PoseDeltaTime = 0.0f;

			// Apply dead blending to the to-pose on the stack
			{
				FAnimNextDeadBlendingApplyTask ApplyTask = FAnimNextDeadBlendingApplyTask::Make(
					&DeadBlendState,
					BlendDuration,
					TimeSinceTransitionStart,
					BlendModeParam,
					CustomBlendCurveParam.Get());

				ApplyTask.Execute(VM);
			}

			return;
		}

		// Phase 3: Subsequent frames - apply dead blending
		if (bTransitionInitialized)
		{
			// Execute to-task to get destination pose
			ToTask->Execute(VM);

			// Apply dead blending
			FAnimNextDeadBlendingApplyTask ApplyTask = FAnimNextDeadBlendingApplyTask::Make(
				&DeadBlendState,
				BlendDuration,
				TimeSinceTransitionStart,
				BlendModeParam,
				CustomBlendCurveParam.Get());

			ApplyTask.Execute(VM);
		}
	}
};

UMovieSceneAnimInertialDeadBlendTransitionSection::UMovieSceneAnimInertialDeadBlendTransitionSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

TSharedPtr<FAnimNextTransitionEvaluationTask> UMovieSceneAnimInertialDeadBlendTransitionSection::CreateTransitionTask() const
{
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return nullptr;
	}
	TSharedPtr<FAnimNextInertialDeadBlendTransitionTask> Task = MakeShared<FAnimNextInertialDeadBlendTransitionTask>();

	// Calculate blend duration from section range
	TRange<FFrameNumber> Range = GetRange();
	if (Range.HasLowerBound() && Range.HasUpperBound())
	{
		FFrameNumber Duration = Range.GetUpperBoundValue() - Range.GetLowerBoundValue();
		// Get frame rate from the owning movie scene
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		Task->BlendDuration = static_cast<float>(Duration.Value) / static_cast<float>(TickResolution.AsDecimal());
	}

	// Configure task with section parameters
	Task->BlendModeParam = BlendMode;
	Task->CustomBlendCurveParam = CustomBlendCurve;

	Task->ExtrapolationParams.ExtrapolationHalfLife = ExtrapolationHalfLife;
	Task->ExtrapolationParams.ExtrapolationHalfLifeMin = ExtrapolationHalfLifeMin;
	Task->ExtrapolationParams.ExtrapolationHalfLifeMax = ExtrapolationHalfLifeMax;
	Task->ExtrapolationParams.MaximumTranslationVelocity = MaximumTranslationVelocity;
	Task->ExtrapolationParams.MaximumRotationVelocity = FMath::DegreesToRadians(MaximumRotationVelocity);
	Task->ExtrapolationParams.MaximumScaleVelocity = MaximumScaleVelocity;

	return Task;
}

FName UMovieSceneAnimInertialDeadBlendTransitionSection::GetTransitionIconStyleName() const
{
	// TODO: Make an icon
	return NAME_None;
}

FText UMovieSceneAnimInertialDeadBlendTransitionSection::GetTransitionDisplayName() const
{
	return NSLOCTEXT("MovieSceneAnimInertialDeadBlendTransitionSection", "DisplayName", "Inertial Dead Blend");
}

void UMovieSceneAnimInertialDeadBlendTransitionSection::RebuildChannelProxy(FMovieSceneChannelProxyData& Channels)
{
	// No channels needed - dead blending doesn't use a blend curve
}
