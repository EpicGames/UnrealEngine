// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerLayerWeightSection.h"

#include "AnimMixerComponentTypes.h"
#include "MovieSceneLayerWeightDecoration.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerLayerWeightSection)

UMovieSceneAnimMixerLayerWeightSection::UMovieSceneAnimMixerLayerWeightSection()
{
	BlendType = EMovieSceneBlendType::Absolute;
	SetColorTint(LayerWeightTintColor);

	SectionRange.Value = TRange<FFrameNumber>::All();
}

EMovieSceneChannelProxyType UMovieSceneAnimMixerLayerWeightSection::CacheChannelProxy()
{
	UMovieSceneLayerWeightDecoration* Decoration = GetTypedOuter<UMovieSceneLayerWeightDecoration>();
	if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		FMovieSceneChannelProxyData ChannelProxyData;
		EMovieSceneChannelProxyType ProxyType = ChannelDecoration->PopulateChannelProxy(ChannelProxyData);
		ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(ChannelProxyData));
		return ProxyType;
	}
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>();
	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneAnimMixerLayerWeightSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params,
	FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FAnimMixerComponentTypes* AnimMixerComponentTypes = FAnimMixerComponentTypes::Get();
	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

	BuildDefaultComponents(EntityLinker, Params, OutImportedEntity);

	if (!OwningLayer)
	{
		return;
	}

	UMovieSceneLayerWeightDecoration* Decoration = GetTypedOuter<UMovieSceneLayerWeightDecoration>();
	if (!Decoration)
	{
		return;
	}

	const FMovieSceneFloatChannel& WeightChannel = Decoration->GetWeightChannel();

	OutImportedEntity->AddBuilder(
	FEntityBuilder()
	.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
	.Add(BuiltInComponentTypes->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
	.Add(AnimMixerComponentTypes->MixerLayer, OwningLayer)
	.AddConditional(BuiltInComponentTypes->WeightChannel, &WeightChannel, WeightChannel.HasAnyData())
	.AddTag(AnimMixerComponentTypes->Tags.LayerWeight)
	.AddMutualComponents());
}
