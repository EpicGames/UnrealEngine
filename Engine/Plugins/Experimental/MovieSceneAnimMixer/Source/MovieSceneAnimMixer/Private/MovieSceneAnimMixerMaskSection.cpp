// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerMaskSection.h"

#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimationMaskDecoration.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "MaskProfile/HierarchyTableTypeMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerMaskSection)

#define LOCTEXT_NAMESPACE "MovieSceneAnimationMixerMaskSection"

UMovieSceneAnimMixerMaskSection::UMovieSceneAnimMixerMaskSection()
{
	BlendType = EMovieSceneBlendType::Absolute;
	// This class implements, IMovieSceneAnimationMixerItemInterface, but since it's owned by a decoration, it is not added to the mixer track, which
	// is where other mixer items have their tint color set. So the tint color is set here instead.
	SetColorTint(MaskTintColor);
}

void UMovieSceneAnimMixerMaskSection::SetBlendMask(UObject* InMask)
{
	if (UUAFBlendMask* InBlendMask = Cast<UUAFBlendMask>(InMask))
	{
		BlendMask = InBlendMask;
	}
}

void UMovieSceneAnimMixerMaskSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params,
	FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FAnimMixerComponentTypes* AnimMixerComponentTypes = FAnimMixerComponentTypes::Get(); 
	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();


	BuildDefaultComponents(EntityLinker, Params, OutImportedEntity);
	
	if (!OwningLayer)
	{
		return;
	}

	FMovieSceneMaskBlendProviderData BlendData;

	BlendData.BlendMask = BlendMask;


	OutImportedEntity->AddBuilder(
	FEntityBuilder()
	.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
	.Add(BuiltInComponentTypes->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
	.Add(AnimMixerComponentTypes->MixerLayer, OwningLayer)
	.Add(AnimMixerComponentTypes->BlendProviderData, 	TInstancedStruct<FMovieSceneMaskBlendProviderData>::Make(MoveTemp(BlendData)))
	.AddMutualComponents());
}

#undef LOCTEXT_NAMESPACE
