// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerModule.h"

#include "AnimNode_SequencerMixerTarget.h"
#include "IMovieSceneModule.h"
#include "MovieSceneAnimMixerSettings.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneRootMotionTargetDecoration.h"
#include "MovieSceneAnimationMaskDecoration.h"
#include "MovieSceneLayerWeightDecoration.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneAnimBusSection.h"
#include "MovieSceneMirroringDecoration.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"

#if WITH_UNREAL_DEVELOPER_TOOLS
#include "ISettingsModule.h"
#endif

DEFINE_LOG_CATEGORY(LogMovieSceneAnimMixer);

#define LOCTEXT_NAMESPACE "FAnimMixerModule"

namespace UE::MovieScene
{
	FName GSequencerDefaultAnimNextInjectionSite = NAME_None;

	FAutoConsoleVariableRef CVarDefaultAnimNextInjectionSite(
	TEXT("Sequencer.AnimNext.DefaultInjectionSite"),
	GSequencerDefaultAnimNextInjectionSite,
	TEXT("(Default: None) Specifies the default injection site name for Sequencer Anim Next Targets that is used when none is specified on the target itself."),
	ECVF_Default
	);

void FMovieSceneAnimMixerModule::StartupModule()
{
#if WITH_UNREAL_DEVELOPER_TOOLS
	ISettingsModule& SettingsModule = FModuleManager::Get().LoadModuleChecked<ISettingsModule>("Settings");
	
	SettingsModule.RegisterSettings("Plugins", "Animation", "AnimMixer",
		LOCTEXT("RuntimeSettingsName", "Anim Mixer"),
		LOCTEXT("RuntimeSettingsDescription", "Configure project settings relating to the Anim Mixer Plugin"),
		GetMutableDefault<UMovieSceneAnimMixerSettings>()
	);
#endif

	CVarDefaultAnimNextInjectionSite->Set(*GetMutableDefault<UMovieSceneAnimMixerSettings>()->DefaultInjectionSite.ToString(), ECVF_SetByProjectSetting);

	// Anim Mixer requires the non-legacy Control Rig template path; .ini, command line, and console still take precedence.
	if (IConsoleVariable* CVarUseLegacyControlRigTemplate = IConsoleManager::Get().FindConsoleVariable(TEXT("ControlRig.UseLegacySequencerTemplate")))
	{
		CVarUseLegacyControlRigTemplate->Set(false, ECVF_SetByGameSetting);
	}

	// Register Root Motion Target decoration as compatible with the Animation Mixer Track
	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneAnimationMixerTrack::StaticClass(), UMovieSceneRootMotionTargetDecoration::StaticClass());

	// Register Root Motion Settings decoration as compatible with Skeletal Animation Sections and Bus Sections
	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneSkeletalAnimationSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());
	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneAnimBusSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());

	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneAnimationMixerLayer::StaticClass(), UMovieSceneAnimationMaskDecoration::StaticClass());
	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneAnimationMixerLayer::StaticClass(), UMovieSceneLayerWeightDecoration::StaticClass());

	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneSkeletalAnimationSection::StaticClass(), UMovieSceneMirroringDecoration::StaticClass());

	IModularFeatures::Get().RegisterModularFeature(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName(), this);
}

void FMovieSceneAnimMixerModule::ShutdownModule()
{
	if (IMovieSceneModule::IsAvailable() && UObjectInitialized())
	{
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneAnimationMixerTrack::StaticClass(), UMovieSceneRootMotionTargetDecoration::StaticClass());
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneSkeletalAnimationSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneAnimBusSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneAnimationMixerLayer::StaticClass(), UMovieSceneAnimationMaskDecoration::StaticClass());
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneAnimationMixerLayer::StaticClass(), UMovieSceneLayerWeightDecoration::StaticClass());
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneSkeletalAnimationSection::StaticClass(), UMovieSceneMirroringDecoration::StaticClass());
	}

	IModularFeatures::Get().UnregisterModularFeature(IAnimGraphRuntime_SequencerMixerTargetConnector::GetModularFeatureName(), this);
}

void FMovieSceneAnimMixerModule::ApplySequencerMixedPose(FPoseContext& Output, FName InTargetName) const
{
	FAnimNode_SequencerMixerTarget::ApplySequencerMixedPose(Output, InTargetName);
}

}

IMPLEMENT_MODULE(UE::MovieScene::FMovieSceneAnimMixerModule, MovieSceneAnimMixer)

#undef LOCTEXT_NAMESPACE