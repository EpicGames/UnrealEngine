// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerSettings.h"

#if WITH_EDITOR
#include "MovieSceneAnimMixerModule.h"
#endif

FName UMovieSceneAnimMixerSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}



#if WITH_EDITOR
void UMovieSceneAnimMixerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UE::MovieScene::CVarDefaultAnimNextInjectionSite->Set(*DefaultInjectionSite.ToString(), ECVF_SetByProjectSetting);
}

FText UMovieSceneAnimMixerSettings::GetSectionText() const
{
	return Super::GetSectionText();
}
#endif