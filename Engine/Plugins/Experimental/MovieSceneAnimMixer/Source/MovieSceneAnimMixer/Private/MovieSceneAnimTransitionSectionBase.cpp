// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimTransitionSectionBase.h"

#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/Tasks/TransitionEvaluationTask.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"

UMovieSceneAnimTransitionSectionBase::UMovieSceneAnimTransitionSectionBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

bool UMovieSceneAnimTransitionSectionBase::IsValid() const
{
	if (!FromSection || !ToSection)
	{
		return false;
	}

	// All three sections must be on the same row
	const int32 TransitionRow = GetRowIndex();
	if (FromSection->GetRowIndex() != TransitionRow || ToSection->GetRowIndex() != TransitionRow)
	{
		return false;
	}

	TRange<FFrameNumber> FromRange = FromSection->GetRange();
	TRange<FFrameNumber> ToRange = ToSection->GetRange();

	// Both sections must have finite bounds
	if (!FromRange.HasLowerBound() || !FromRange.HasUpperBound() ||
		!ToRange.HasLowerBound() || !ToRange.HasUpperBound())
	{
		return false;
	}

	// Check if one section is fully contained within the other - this is not a valid transition
	// A transition requires both sections to extend beyond the overlap region
	// If one section fully contains the other, there's no valid transition point
	if (FromRange.Contains(ToRange) || ToRange.Contains(FromRange))
	{
		return false;
	}

	// There must be an overlap between the sections that spans at least one display frame.
	// Sub-frame overlaps can occur when section endpoints don't align to display frame
	// boundaries and should not be treated as valid transitions.
	TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(FromRange, ToRange);
	return UMovieSceneAnimationMixerTrack::IsOverlapLargeEnoughForTransition(Overlap, this);
}

TRange<FFrameNumber> UMovieSceneAnimTransitionSectionBase::ComputeOverlapRange() const
{
	if (!IsValid())
	{
		return TRange<FFrameNumber>::Empty();
	}

	return TRange<FFrameNumber>::Intersection(
		FromSection->GetRange(),
		ToSection->GetRange()
	);
}

bool UMovieSceneAnimTransitionSectionBase::UpdateBoundsFromSourceSections()
{
	// IsValid() checks: sections exist, same row, bounded, no containment, has overlap
	if (!IsValid())
	{
		return false;
	}

	TRange<FFrameNumber> FromRange = FromSection->GetRange();
	TRange<FFrameNumber> ToRange = ToSection->GetRange();

	// Check if the order has reversed (the "to" section now starts before "from")
	// In a proper transition, FromSection should start before ToSection
	if (ToRange.GetLowerBoundValue() < FromRange.GetLowerBoundValue())
	{
		// Swap from/to to maintain correct ordering
		Modify();
		Swap(FromSection, ToSection);

		// Update local variables after swap
		FromRange = FromSection->GetRange();
		ToRange = ToSection->GetRange();
	}

	TRange<FFrameNumber> NewOverlap = TRange<FFrameNumber>::Intersection(FromRange, ToRange);

	// Check if more than 2 sections overlap this range - transitions only work between exactly 2 sections
	if (UMovieSceneAnimationMixerTrack* MixerTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>())
	{
		if (MixerTrack->CountSectionsOverlappingRange(NewOverlap, GetRowIndex()) > 2)
		{
			return false;
		}
	}

	// Get the current range before updating
	TRange<FFrameNumber> OldRange = GetRange();

	// Update transition bounds to match the full overlap
	SetRange(NewOverlap);

	// Scale existing keys proportionally to the new range
	ScaleKeysToRange(OldRange, NewOverlap);

	return true;
}

void UMovieSceneAnimTransitionSectionBase::ScaleKeysToRange(const TRange<FFrameNumber>& OldRange, const TRange<FFrameNumber>& NewRange)
{
	const FMovieSceneFloatChannel* ConstChannel = GetBlendWeightChannel();
	if (!ConstChannel)
	{
		return;
	}

	if (!OldRange.HasLowerBound() || !OldRange.HasUpperBound() ||
		!NewRange.HasLowerBound() || !NewRange.HasUpperBound())
	{
		return;
	}

	const FFrameNumber OldStart = OldRange.GetLowerBoundValue();
	const FFrameNumber OldEnd = OldRange.GetUpperBoundValue();
	const FFrameNumber NewStart = NewRange.GetLowerBoundValue();
	const FFrameNumber NewEnd = NewRange.GetUpperBoundValue();

	const double OldDuration = (OldEnd - OldStart).Value;
	const double NewDuration = (NewEnd - NewStart).Value;

	if (OldDuration <= 0 || NewDuration <= 0)
	{
		return;
	}

	// Need non-const access to modify keys
	FMovieSceneFloatChannel* Channel = const_cast<FMovieSceneFloatChannel*>(ConstChannel);

	// Collect existing key data
	TArrayView<const FFrameNumber> OldTimes = Channel->GetData().GetTimes();
	TArrayView<const FMovieSceneFloatValue> OldValues = Channel->GetValues();

	TArray<FFrameNumber> NewTimes;
	TArray<FMovieSceneFloatValue> NewValues;
	NewTimes.Reserve(OldTimes.Num());
	NewValues.Reserve(OldValues.Num());

	for (int32 i = 0; i < OldTimes.Num(); ++i)
	{
		// Calculate normalized position in old range [0, 1]
		double NormalizedPos = (OldTimes[i] - OldStart).Value / OldDuration;
		// Map to new range
		FFrameNumber ScaledTime = NewStart + FFrameNumber(static_cast<int32>(NormalizedPos * NewDuration));
		NewTimes.Add(ScaledTime);
		NewValues.Add(OldValues[i]);
	}

	// Clear and re-add keys at new positions
	Channel->Reset();
	Channel->AddKeys(NewTimes, NewValues);
}

float UMovieSceneAnimTransitionSectionBase::GetTransitionProgress(FFrameTime InTime) const
{
	TRange<FFrameNumber> Range = GetRange();
	if (Range.IsEmpty() || !Range.HasLowerBound() || !Range.HasUpperBound())
	{
		return 0.0f;
	}

	FFrameNumber Start = Range.GetLowerBoundValue();
	FFrameNumber End = Range.GetUpperBoundValue();
	FFrameNumber Duration = End - Start;

	if (Duration.Value <= 0)
	{
		return 0.0f;
	}

	double Progress = (InTime.AsDecimal() - Start.Value) / Duration.Value;
	return FMath::Clamp(static_cast<float>(Progress), 0.0f, 1.0f);
}

EMovieSceneChannelProxyType UMovieSceneAnimTransitionSectionBase::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	// Let subclass add its specific channels
	RebuildChannelProxy(Channels);

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Static;
}

void UMovieSceneAnimTransitionSectionBase::ImportEntityImpl(
	UMovieSceneEntitySystemLinker* EntityLinker,
	const FEntityImportParams& Params,
	FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	if (Params.GetObjectBindingID().IsValid())
	{
		// Create the transition task - it will persist as an entity component and have
		// its Update() method called each frame by the mixer system to set from/to tasks.
		// The Transition tag marks this entity for special handling by the mixer system.
		TSharedPtr<FAnimNextTransitionEvaluationTask> TransitionTask = CreateTransitionTask();
		const FMovieSceneFloatChannel* BlendChannel = GetBlendWeightChannel();

		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(BuiltInComponents->GenericObjectBinding, Params.GetObjectBindingID())
			.Add(BuiltInComponents->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
			.Add(AnimMixerComponents->Task, TransitionTask)
			.Add(AnimMixerComponents->EntityOwner, FObjectKey(this))
			.AddTag(AnimMixerComponents->Tags.Transition)
			.AddConditional(BuiltInComponents->WeightChannel, BlendChannel, BlendChannel != nullptr)
		);
	}
}

static bool IsSectionEffectivelyDisabled(const UMovieSceneSection* Section)
{
	if (!Section || !Section->IsActive())
	{
		return true;
	}

#if WITH_EDITOR
	if (Section->IsLocalEvalDisabled())
	{
		return true;
	}
#endif

	if (const UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
	{
		if (Track->IsRowEvalDisabled(Section->GetRowIndex()))
		{
			return true;
		}
	}

	return false;
}

bool UMovieSceneAnimTransitionSectionBase::PopulateEvaluationFieldImpl(
	const TRange<FFrameNumber>& EffectiveRange,
	const FMovieSceneEvaluationFieldEntityMetaData& InMetaData,
	FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	// Skip evaluation if either the from or to section is disabled (muted, deactivated, or inactive)
	if (IsSectionEffectivelyDisabled(FromSection) || IsSectionEffectivelyDisabled(ToSection))
	{
		return true;
	}

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
	OutFieldBuilder->AddPersistentEntity(EffectiveRange, this, 0, MetaDataIndex);
	return true;
}

#if WITH_EDITOR
bool UMovieSceneAnimTransitionSectionBase::IsLocalEvalDisabled() const
{
	// A transition is disabled if it was directly muted or if either of its source sections
	// is effectively disabled (muted, deactivated, row-disabled, or inactive).
	return Super::IsLocalEvalDisabled()
		|| IsSectionEffectivelyDisabled(FromSection)
		|| IsSectionEffectivelyDisabled(ToSection);
}
#endif
