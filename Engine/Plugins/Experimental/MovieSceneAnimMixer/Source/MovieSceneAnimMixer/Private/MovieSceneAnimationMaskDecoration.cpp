// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationMaskDecoration.h"

#include "MovieScene.h"
#include "MovieSceneAnimMixerMaskSection.h"
#include "MovieSceneAnimationMixerLayer.h"

UMovieSceneAnimationMaskDecoration::UMovieSceneAnimationMaskDecoration(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{

}

UMovieSceneSection* UMovieSceneAnimationMaskDecoration::CreateNewSection()
{
	return NewObject<UMovieSceneAnimMixerMaskSection>(this, UMovieSceneAnimMixerMaskSection::StaticClass(), NAME_None, RF_Transactional);
}

TSubclassOf<UMovieSceneSection> UMovieSceneAnimationMaskDecoration::GetHostedSectionClass() const
{
	return UMovieSceneAnimMixerMaskSection::StaticClass();
}

void UMovieSceneAnimationMaskDecoration::AddSection(UMovieSceneSection* InSection)
{
	UMovieSceneAnimMixerMaskSection* NewMaskSection = Cast<UMovieSceneAnimMixerMaskSection>(InSection);
	if (!NewMaskSection)
	{
		return;
	}

	Modify();

	const TRange<FFrameNumber> NewSectionRange = NewMaskSection->GetRange();

	if (NewSectionRange.GetLowerBound().IsClosed() && NewSectionRange.GetUpperBound().IsClosed())
	{
		for (UMovieSceneSection* Section : MaskSections)
		{
			const TRange<FFrameNumber> CurrentSectionRange = Section->GetRange();
			if (CurrentSectionRange.GetUpperBound().IsOpen() || CurrentSectionRange.GetLowerBound().IsOpen())
			{
				continue;
			}
			if (CurrentSectionRange.Overlaps(NewSectionRange))
			{
				if (Section->GetInclusiveStartFrame() < NewMaskSection->GetInclusiveStartFrame())
				{
					Section->SetEndFrame(NewMaskSection->GetInclusiveStartFrame());
				}
				else
				{
					NewMaskSection->SetEndFrame(Section->GetInclusiveStartFrame());
				}
			}
		}
	}

	MaskSections.Add(NewMaskSection);

	MarkAsChanged();
#if WITH_EDITOR
	MarkStructureChanged();
#endif

	UMovieSceneAnimationMixerLayer* Layer = GetTypedOuter<UMovieSceneAnimationMixerLayer>();
	NewMaskSection->SetAnimMixerLayer(Layer);

	// Match the layer's row index so per-row mute/disable applies correctly.
	if (Layer)
	{
		NewMaskSection->SetRowIndex(Layer->GetLayerIndex());
	}
}

void UMovieSceneAnimationMaskDecoration::RemoveSection(UMovieSceneSection& SectionToRemove)
{
	Modify();
	MaskSections.RemoveAll([&SectionToRemove](const UMovieSceneSection* InSection){ return InSection == &SectionToRemove; });
	MarkAsChanged();
#if WITH_EDITOR
	MarkStructureChanged();
#endif
}

#if WITH_EDITOR
void UMovieSceneAnimationMaskDecoration::PostEditUndo()
{
	Super::PostEditUndo();

	// Re-establish owning layer references after undo restores the section array.
	UMovieSceneAnimationMixerLayer* Layer = GetTypedOuter<UMovieSceneAnimationMixerLayer>();
	for (TObjectPtr<UMovieSceneSection> Section : MaskSections)
	{
		if (UMovieSceneAnimMixerMaskSection* MaskSection = Cast<UMovieSceneAnimMixerMaskSection>(Section))
		{
			MaskSection->SetAnimMixerLayer(Layer);
		}
	}

	MarkStructureChanged();
}
#endif
