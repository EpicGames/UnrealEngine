// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "MovieSceneAnimationMixerTrack.h"

UMovieSceneAnimationMixerLayer::UMovieSceneAnimationMixerLayer(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UMovieSceneAnimationMixerLayer::SetChildTrack(UMovieSceneTrack* InTrack)
{
	if (ChildTrack != InTrack)
	{
		Modify();

		// Clear sections if we're setting a child track
		if (InTrack && Sections.Num() > 0)
		{
			Sections.Empty();
		}

		ChildTrack = InTrack;
		MarkAsChanged();
	}
}

void UMovieSceneAnimationMixerLayer::AddSection(UMovieSceneSection* Section)
{
	if (Section && !Sections.Contains(Section))
	{
		Modify();

		// Clear child track if we're adding sections
		if (ChildTrack)
		{
			ChildTrack = nullptr;
		}

		Sections.Add(Section);
		MarkAsChanged();
	}
}

void UMovieSceneAnimationMixerLayer::RemoveSection(UMovieSceneSection* Section)
{
	if (Sections.Contains(Section))
	{
		Modify();
		Sections.Remove(Section);
		MarkAsChanged();
	}
}

bool UMovieSceneAnimationMixerLayer::ContainsSection(UMovieSceneSection* Section) const
{
	return Sections.Contains(Section);
}

int32 UMovieSceneAnimationMixerLayer::GetLayerIndex() const
{
	if (const UMovieSceneAnimationMixerTrack* MixerTrack = GetMixerTrack())
	{
		return MixerTrack->GetLayers().IndexOfByKey(this);
	}

	return INDEX_NONE;
}

const UMovieSceneAnimationMixerTrack* UMovieSceneAnimationMixerLayer::GetMixerTrack() const
{
	return GetTypedOuter<UMovieSceneAnimationMixerTrack>();
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneAnimationMixerLayer::GetDisplayName() const
{
	if (!DisplayName.IsEmpty())
	{
		return DisplayName;
	}

	FText LayerNumberText = NSLOCTEXT("MovieSceneAnimationMixerLayer", "DefaultLayerName", "Mix Layer");

	UMovieSceneAnimationMixerTrack* ParentTrack = Cast<UMovieSceneAnimationMixerTrack>(GetOuter());
	if (ParentTrack)
	{
		int32 LayerIndex = ParentTrack->GetLayers().IndexOfByKey(this);
		if (LayerIndex != INDEX_NONE)
		{
			LayerNumberText = FText::Format(NSLOCTEXT("MovieSceneAnimationMixerLayer", "LayerNameFormat", "Mix Layer {0}"), FText::AsNumber(LayerIndex + 1));
		}
	}

	// Generate default name based on content
	if (ChildTrack)
	{
		return FText::Format(NSLOCTEXT("MovieSceneAnimationMixerLayer", "ChildTrackLayerNameFormat", "{0}: {1}"), LayerNumberText, ChildTrack->GetDisplayName());
	}
	else
	{
		return LayerNumberText;
	}
}

void UMovieSceneAnimationMixerLayer::SetDisplayName(const FText& InDisplayName)
{
	if (!GetDisplayName().EqualTo(InDisplayName))
	{
		DisplayName = InDisplayName;
		MarkAsChanged();
	}
}

#endif
