// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneAnimCrossfadeTransitionSection.h"
#include "MovieSceneRootMotionTargetDecoration.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "Misc/AxisDisplayInfo.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "MovieSceneAnimationTrackDecoration.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieScene.h"
#include "MovieSceneAnimMixerMaskSection.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "AnimMixerBakeEvaluation.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "MovieSceneRootMotionSection.h"
#include "Compilation/MovieSceneEvaluationTreePopulationRules.h"

#define LOCTEXT_NAMESPACE "MovieSceneAnimationMixerTrack"

bool FMovieSceneByteChannelDefaultOnly::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName MovieSceneByteChannel("MovieSceneByteChannel");
	if (Tag.GetType().IsStruct(MovieSceneByteChannel))
	{
		StaticStruct()->SerializeItem(Slot, this, nullptr);
		return true;
	}

	return false;
}

void UMovieSceneRootMotionHostSection::SetOwnerDecoration(UObject* InOwnerDecoration)
{
	OwnerDecoration = InOwnerDecoration;
}

EMovieSceneChannelProxyType UMovieSceneRootMotionHostSection::CacheChannelProxy()
{
	if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(OwnerDecoration))
	{
		FMovieSceneChannelProxyData ChannelProxyData;
		EMovieSceneChannelProxyType ProxyType = ChannelDecoration->PopulateChannelProxy(ChannelProxyData);
		ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(ChannelProxyData));
		return ProxyType;
	}
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>();
	return EMovieSceneChannelProxyType::Dynamic;
}

UMovieSceneRootMotionSettingsDecoration::UMovieSceneRootMotionSettingsDecoration(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Location[0].SetDefault(0.0);
	Location[1].SetDefault(0.0);
	Location[2].SetDefault(0.0);
	Rotation[0].SetDefault(0.0);
	Rotation[1].SetDefault(0.0);
	Rotation[2].SetDefault(0.0);

	RootMotionSpace.SetEnum(StaticEnum<EMovieSceneRootMotionSpace>());
	RootMotionSpace.SetDefault((uint8)EMovieSceneRootMotionSpace::AnimationSpace);

	RootMotionSourceChannel.SetEnum(StaticEnum<EMovieSceneRootMotionSource>());
	RootMotionSourceChannel.SetDefault((uint8)EMovieSceneRootMotionSource::RootBone);

	TransformMode.SetEnum(StaticEnum<EMovieSceneRootMotionTransformMode>());
	TransformMode.SetDefault((uint8)EMovieSceneRootMotionTransformMode::Offset);

	KeepStateChannel.SetEnum(StaticEnum<EMovieSceneRootMotionKeepState>());
	KeepStateChannel.SetDefault((uint8)EMovieSceneRootMotionKeepState::DontKeep);

	LoopModeChannel.SetEnum(StaticEnum<EMovieSceneAnimLoopMode>());
	LoopModeChannel.SetDefault((uint8)EMovieSceneAnimLoopMode::Accumulated);

	// Invalidate root motion caches whenever this decoration is modified.
	// This covers channel default edits (which only call Modify(), not
	// PostEditChangeProperty) as well as property edits and undo.
	auto InvalidateCacheLambda = [this]()
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>())
		{
			MixerTrack->InvalidateAccumulatedOffsetCache();
		}
	};

	OnSignatureChanged().AddWeakLambda(this, InvalidateCacheLambda);

	// Also invalidate when the owning section's properties change.
	if (UMovieSceneSection* OwnerSection = GetTypedOuter<UMovieSceneSection>())
	{
		OwnerSection->OnSignatureChanged().AddWeakLambda(this, InvalidateCacheLambda);
	}
}

void UMovieSceneRootMotionSettingsDecoration::OnDecorationAdded(UMovieSceneSection* Section)
{
	// When this Root Motion Settings decoration is added to a section that belongs
	// to a mixer track, ensure the mixer track also has a Root Motion Target decoration.
	if (UMovieSceneAnimationMixerTrack* MixerTrack = Section ? Section->GetTypedOuter<UMovieSceneAnimationMixerTrack>() : nullptr)
	{
		if (!MixerTrack->FindDecoration<UMovieSceneRootMotionTargetDecoration>())
		{
			MixerTrack->GetOrCreateDecoration<UMovieSceneRootMotionTargetDecoration>();
		}
	}
}

void UMovieSceneRootMotionSettingsDecoration::OnDecorationAdded(UMovieSceneTrack* Track)
{
	// When this Root Motion Settings decoration is added to a child track (e.g. Control Rig)
	// within a mixer, ensure the mixer track also has a Root Motion Target decoration.
	if (UMovieSceneAnimationTrackDecoration* TrackDecoration = Track ? Track->FindDecoration<UMovieSceneAnimationTrackDecoration>() : nullptr)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = TrackDecoration->MixerTrack)
		{
			if (!MixerTrack->FindDecoration<UMovieSceneRootMotionTargetDecoration>())
			{
				MixerTrack->GetOrCreateDecoration<UMovieSceneRootMotionTargetDecoration>();
			}
		}
	}
}

TOptional<FFrameTime> UMovieSceneRootMotionSettingsDecoration::GetFirstLoopBoundary(const FFrameRate& TickResolution) const
{
	UMovieSceneSection* OwnerSection = GetTypedOuter<UMovieSceneSection>();
	auto* SkelAnimSection = OwnerSection ? Cast<UMovieSceneSkeletalAnimationSection>(OwnerSection) : nullptr;
	if (!SkelAnimSection || !OwnerSection->HasStartFrame() || !OwnerSection->HasEndFrame())
	{
		return {};
	}

	const float SequenceLength = SkelAnimSection->Params.GetSequenceLength();
	if (SequenceLength <= KINDA_SMALL_NUMBER)
	{
		return {};
	}

	// Use the section's timing transform to find the actual loop boundary.
	// Manual frame arithmetic with FloorToFrame() loses sub-frames, which causes
	// the boundary to land before the timing system's actual loop wrap point.
	FMovieSceneSequenceTransform Transform = SkelAnimSection->Params.MakeTransform(
		TickResolution, OwnerSection->GetRange());

	TOptional<FFrameTime> FirstBoundary;
	Transform.ExtractBoundariesWithinRange(
		FFrameTime(OwnerSection->GetInclusiveStartFrame()),
		FFrameTime(OwnerSection->GetExclusiveEndFrame()),
		[&FirstBoundary, &Transform, SequenceLength](FFrameTime Boundary) -> bool
		{
			FirstBoundary = FFrameTime(UMovieSceneRootMotionSection::ResolveLoopTransitionTick(Transform, Boundary, SequenceLength));
			return false; // stop after first
		});

	return FirstBoundary;
}

FTransform UMovieSceneRootMotionSettingsDecoration::GetInitialRootTransformAtTime(FFrameTime Time, const FFrameRate& TickResolution) const
{
	if (!bInitialRootTransformValid)
	{
		return FTransform::Identity;
	}

	TOptional<FFrameTime> FirstLoop = GetFirstLoopBoundary(TickResolution);
	if (FirstLoop.IsSet() && Time >= FirstLoop.GetValue())
	{
		return LoopStartRootTransform;
	}
	return InitialRootTransform;
}

EMovieSceneRootMotionTransformMode UMovieSceneRootMotionSettingsDecoration::GetRootTransformMode() const
{
	return (EMovieSceneRootMotionTransformMode)TransformMode.GetDefault().Get((uint8)EMovieSceneRootMotionTransformMode::Offset);
}

EMovieSceneRootMotionSource UMovieSceneRootMotionSettingsDecoration::GetRootMotionSource() const
{
	return (EMovieSceneRootMotionSource)RootMotionSourceChannel.GetDefault().Get((uint8)EMovieSceneRootMotionSource::RootBone);
}

EMovieSceneChannelProxyType UMovieSceneRootMotionSettingsDecoration::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData)
{
	EMovieSceneRootMotionTransformMode CurrentTransformMode = GetRootTransformMode();

#if WITH_EDITOR
	int32 SortOrder = 0;
	{
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("Space", LOCTEXT("Space", "Space"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(RootMotionSpace, ChannelMetaData, TMovieSceneExternalValue<uint8>());
	}
	{
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("RootMotionSource", LOCTEXT("Source", "Source"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(RootMotionSourceChannel, ChannelMetaData, TMovieSceneExternalValue<uint8>());
	}
	{
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("Root Mode", LOCTEXT("Mode", "Mode"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(TransformMode, ChannelMetaData, TMovieSceneExternalValue<uint8>());
	}
	if (CurrentTransformMode != EMovieSceneRootMotionTransformMode::Override)
	{
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("LoopMode", LOCTEXT("LoopMode", "Loop Mode"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(LoopModeChannel, ChannelMetaData, TMovieSceneExternalValue<uint8>());
	}
	{
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("KeepState", LOCTEXT("KeepState", "Keep Root Motion"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(KeepStateChannel, ChannelMetaData, TMovieSceneExternalValue<uint8>());
	}

	if (CurrentTransformMode != EMovieSceneRootMotionTransformMode::Asset)
	{
		FName SliderExponent("SliderExponent");
		FText OffsetGroup = LOCTEXT("RootMotionOffset", "Offset");

		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseLocation.X", AxisDisplayInfo::GetAxisDisplayName(EAxisList::X), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::X);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Location[0], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseLocation.Y", AxisDisplayInfo::GetAxisDisplayName(EAxisList::Y), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Location[1], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseLocation.Z", AxisDisplayInfo::GetAxisDisplayName(EAxisList::Z), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::Z);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Location[2], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseRotation.X", LOCTEXT("RotationX", "Roll"), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::X);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Rotation[0], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseRotation.Y", LOCTEXT("RotationY", "Pitch"), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Rotation[1], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
		{
			FMovieSceneChannelMetaData ChannelMetaData;
			ChannelMetaData.SetIdentifiers("RootBaseRotation.Z", LOCTEXT("RotationZ", "Yaw"), OffsetGroup);
			ChannelMetaData.Color = AxisDisplayInfo::GetAxisColor(EAxisList::Z);
			ChannelMetaData.WeakOwningObject = this;
			ChannelMetaData.SortOrder = SortOrder++;
			ChannelMetaData.PropertyMetaData.Add(SliderExponent, TEXT("0.2"));
			OutProxyData.Add(Rotation[2], ChannelMetaData, TMovieSceneExternalValue<double>());
		}
	}

#if WITH_EDITOR
	if (CurrentTransformMode == EMovieSceneRootMotionTransformMode::BoneMatch && BoneMatchChannel.GetNumKeys() > 0)
	{
		const bool bMatchValid = GetBoneMatchData().bIsValid;
		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.SetIdentifiers("BoneMatch",
			bMatchValid
				? LOCTEXT("BoneMatch", "Bone Match")
				: LOCTEXT("BoneMatchInvalid", "Bone Match (invalid)"));
		ChannelMetaData.WeakOwningObject = this;
		ChannelMetaData.SortOrder = SortOrder++;
		OutProxyData.Add(BoneMatchChannel, ChannelMetaData);
	}
#endif

#else

	if (CurrentTransformMode != EMovieSceneRootMotionTransformMode::Asset)
	{
		OutProxyData.Add(RootMotionSpace);
		OutProxyData.Add(RootMotionSourceChannel);
		OutProxyData.Add(TransformMode);
	}

	if (CurrentTransformMode != EMovieSceneRootMotionTransformMode::Override)
	{
		OutProxyData.Add(LoopModeChannel);
	}
	OutProxyData.Add(KeepStateChannel);

	OutProxyData.Add(Location[0]);
	OutProxyData.Add(Location[1]);
	OutProxyData.Add(Location[2]);

	OutProxyData.Add(Rotation[0]);
	OutProxyData.Add(Rotation[1]);
	OutProxyData.Add(Rotation[2]);

	OutProxyData.Add(BoneMatchChannel);

#endif

	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneRootMotionSettingsDecoration::ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FMovieSceneRootMotionSettings RootMotionSettings;
	{
		RootMotionSettings.RootMotionSpace  = (EMovieSceneRootMotionSpace)RootMotionSpace.GetDefault().Get((uint8)EMovieSceneRootMotionSpace::AnimationSpace);
		RootMotionSettings.RootMotionSource = GetRootMotionSource();
		RootMotionSettings.TransformMode    = GetRootTransformMode();
		RootMotionSettings.LoopMode         = GetLoopMode();

		UMovieSceneSkeletalAnimationSection* AnimSection = GetTypedOuter<UMovieSceneSkeletalAnimationSection>();
		if (AnimSection)
		{
			RootMotionSettings.LegacySwapRootBone = AnimSection->Params.SwapRootBone;
		}
	}

	RootMotionSettings.RootOriginLocation = RootOriginLocation;

	if (RootMotionSettings.TransformMode != EMovieSceneRootMotionTransformMode::Asset)
	{
		RootMotionSettings.OffsetChannels[0] = &Location[0];
		RootMotionSettings.OffsetChannels[1] = &Location[1];
		RootMotionSettings.OffsetChannels[2] = &Location[2];
		RootMotionSettings.OffsetChannels[3] = &Rotation[0];
		RootMotionSettings.OffsetChannels[4] = &Rotation[1];
		RootMotionSettings.OffsetChannels[5] = &Rotation[2];
	}

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(AnimMixerComponents->RootMotionSettings, RootMotionSettings)
		.AddConditional(AnimMixerComponents->BoneMatchTransform, GetBoneMatchData().MatchTransform, HasBoneMatch() && GetBoneMatchData().bIsValid)
	);
}

TArrayView<TObjectPtr<UMovieSceneSection>> UMovieSceneRootMotionSettingsDecoration::GetSections()
{
	// Only provide a host section when the decoration is on a track (not a section).
	// When on a section, return empty so the channel decoration path handles it instead.
	if (!GetTypedOuter<UMovieSceneSection>())
	{
		EnsureHostSection();
		return TArrayView<TObjectPtr<UMovieSceneSection>>(&HostSection, 1);
	}
	return TArrayView<TObjectPtr<UMovieSceneSection>>();
}

void UMovieSceneRootMotionSettingsDecoration::EnsureHostSection()
{
	if (!HostSection)
	{
		UMovieSceneRootMotionHostSection* NewSection = NewObject<UMovieSceneRootMotionHostSection>(this, NAME_None, RF_Transactional);
		NewSection->SetOwnerDecoration(this);
		NewSection->SetRange(TRange<FFrameNumber>::All());
		NewSection->SetColorTint(FColor(20, 70, 70, 200));
		HostSection = NewSection;
	}
}

void UMovieSceneAnimationSectionDecoration::ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	UMovieSceneSection* Section = GetTypedOuter<UMovieSceneSection>();
	UMovieSceneAnimationMixerLayer* MixerLayer = MixerTrack ? MixerTrack->GetLayerForSection(Section) : nullptr;

	TInstancedStruct<FMovieSceneMixedAnimationTarget> Target;
	if (MixerTrack != nullptr && MixerTrack->MixedAnimationTarget.IsValid())
	{
		Target = MixerTrack->MixedAnimationTarget;
	}
	else
	{
		Target = TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make();
	}

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(FMovieSceneTracksComponentTypes::Get()->Tags.AnimMixerPoseProducer)
		// Mixer pose producers are routed/blended by the mixer (by target), not by hierarchical bias.
		.AddTag(FBuiltInComponentTypes::Get()->Tags.IgnoreHierarchicalBias)
		.Add(AnimMixerComponents->Target, Target)
		.Add(AnimMixerComponents->Priority, MixerLayer ? MixerLayer->GetLayerIndex() : 0)
		.Add(AnimMixerComponents->MixerLayer, MixerLayer)
		.Add(AnimMixerComponents->EntityOwner, FObjectKey(Section))
		.AddMutualComponents()
	);
}

bool WouldCreateBoneMatchCycle(UMovieSceneSection* CandidateSection, UMovieSceneSection* TargetSection)
{
	TSet<UMovieSceneSection*> Visited;
	UMovieSceneSection* Current = CandidateSection;
	while (Current)
	{
		if (Current == TargetSection)
		{
			return true;
		}
		if (Visited.Contains(Current))
		{
			break;
		}
		Visited.Add(Current);

		UMovieSceneRootMotionSettingsDecoration* Decoration = Current->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!Decoration || !Decoration->HasBoneMatch())
		{
			break;
		}
		Current = Decoration->GetBoneMatchData().ReferenceSection.Get();
	}
	return false;
}

UMovieSceneAnimationMixerTrack::UMovieSceneAnimationMixerTrack(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(66, 56, 88, 255);
	bSupportsDefaultSections = false;
#endif

	MixedAnimationTarget = TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make();
}


bool UMovieSceneAnimationMixerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	if (Super::SupportsType(SectionClass))
	{
		return true;
	}

	UClass* Class = SectionClass.Get();
	if (!Class || !Class->ImplementsInterface(UMovieSceneAnimationMixerItemInterface::StaticClass()))
	{
		return false;
	}

	// Use the same allowlist as the "+ Add Section" menu so drag/drop, paste,
	// and the add menu stay in sync. Transition and decoration host sections
	// override IsVisibleInAddSectionMenu() to false.
	const IMovieSceneAnimationMixerItemInterface* MixerItem = Cast<IMovieSceneAnimationMixerItemInterface>(Class->GetDefaultObject());
	return MixerItem && MixerItem->IsVisibleInAddSectionMenu();
}

UMovieSceneDecorationContainerObject* UMovieSceneAnimationMixerTrack::GetDecorationContainerForRow(int32 RowIndex) const
{
	return GetLayer(RowIndex);
}

bool UMovieSceneAnimationMixerTrack::PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
{
	UE::MovieScene::FEvaluationTreePopulationRules::HighPassPerRow(AnimationSections, OutData);

	for (const UMovieSceneAnimationMixerLayer* Layer : GetLayers())
	{
		for (UObject* Decoration : Layer->GetActiveDecorations())
		{
			if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
			{
				UE::MovieScene::FEvaluationTreePopulationRules::Blended(SectionProvider->GetSections(), OutData);
			}
		}
	}
	return true;
}

bool UMovieSceneAnimationMixerTrack::FixRowIndices()
{
	// Do nothing with row indices here- our rows are layers and we can have empty ones.
	return false;
}

void UMovieSceneAnimationMixerTrack::RemoveSection(UMovieSceneSection& Section)
{
	Modify();

	// Call OnSectionRemovedImpl FIRST so we can find and cascade-delete related transitions
	// before the section is removed from our internal arrays
	OnSectionRemovedImpl(&Section);

	// Now call parent to actually remove the section
	Super::RemoveSection(Section);
}

void UMovieSceneAnimationMixerTrack::OnSectionAddedImpl(UMovieSceneSection* Section)
{
	if (Section)
	{
		UMovieSceneAnimationSectionDecoration* ExistingDecoration = Section->FindDecoration<UMovieSceneAnimationSectionDecoration>();
		if (!ExistingDecoration)
		{
			UMovieSceneAnimationSectionDecoration* NewDecoration = NewObject<UMovieSceneAnimationSectionDecoration>(Section, UMovieSceneAnimationSectionDecoration::StaticClass(), NAME_None, RF_Transactional);
			NewDecoration->MixerTrack = this;
			Section->AddDecoration(NewDecoration);
		}
		else
		{
			ExistingDecoration->MixerTrack = this;
		}

		if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
		{
			if (!AnimSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
			{
				if (UAnimSequenceBase* Animation = AnimSection->Params.Animation)
				{
					if (Animation->HasRootMotion())
					{
						UMovieSceneRootMotionSettingsDecoration* RootMotionSettings = NewObject<UMovieSceneRootMotionSettingsDecoration>(AnimSection, UMovieSceneRootMotionSettingsDecoration::StaticClass(), NAME_None, RF_Transactional);
						AnimSection->AddDecoration(RootMotionSettings);
					}
				}
			}
		}

#if WITH_EDITOR
		if (IMovieSceneAnimationMixerItemInterface* AnimationItemInterface = Cast<IMovieSceneAnimationMixerItemInterface>(Section))
		{
			Section->SetColorTint(AnimationItemInterface->GetMixerItemTint());
		}
#endif

		// Create/update layer for this section
		int32 RowIndex = Section->GetRowIndex();
		UMovieSceneAnimationMixerLayer* Layer = GetOrCreateLayer(RowIndex);
		if (Layer)
		{
			Layer->AddSection(Section);
		}

		if (!Section->IsA<UMovieSceneAnimTransitionSectionBase>())
		{
			UpdateTransitionsForRow(RowIndex);
		}

		InvalidateAccumulatedOffsetCache();
	}
}

void UMovieSceneAnimationMixerTrack::OnSectionRemovedImpl(UMovieSceneSection* Section)
{
	// Remove section from its layer
	if (Section)
	{
		UMovieSceneAnimationMixerLayer* Layer = GetLayerForSection(Section);
		if (Layer)
		{
			Layer->RemoveSection(Section);
		}

		// Cascade delete: remove any transition sections that reference this section
		TArray<UMovieSceneAnimTransitionSectionBase*> RelatedTransitions = FindTransitionsForSection(Section);
		for (UMovieSceneAnimTransitionSectionBase* Transition : RelatedTransitions)
		{
			RemoveSection(*Transition);
		}

		// Mark bone matches that referenced the removed section as invalid
		for (UMovieSceneSection* OtherSection : GetAllSections())
		{
			if (!OtherSection || OtherSection == Section || OtherSection->IsA<UMovieSceneAnimTransitionSectionBase>())
			{
				continue;
			}

			UMovieSceneRootMotionSettingsDecoration* OtherDecoration = OtherSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
			if (!OtherDecoration || !OtherDecoration->HasBoneMatch())
			{
				continue;
			}

			TMovieSceneChannelData<FMovieSceneBoneMatchData> Data = OtherDecoration->BoneMatchChannel.GetData();
			TArrayView<FMovieSceneBoneMatchData> Values = Data.GetValues();
			for (FMovieSceneBoneMatchData& Value : Values)
			{
				if (Value.ReferenceSection.Get() == Section)
				{
					Value.bIsValid = false;
					Value.bIsDirty = true;
				}
			}
		}

		InvalidateAccumulatedOffsetCache();
	}
}

TArray<UMovieSceneAnimTransitionSectionBase*> UMovieSceneAnimationMixerTrack::FindTransitionsForSection(UMovieSceneSection* Section) const
{
	TArray<UMovieSceneAnimTransitionSectionBase*> Result;

	if (!Section)
	{
		return Result;
	}

	for (UMovieSceneSection* Sec : GetAllSections())
	{
		if (UMovieSceneAnimTransitionSectionBase* Transition = Cast<UMovieSceneAnimTransitionSectionBase>(Sec))
		{
			if (Transition->FromSection == Section || Transition->ToSection == Section)
			{
				Result.Add(Transition);
			}
		}
	}

	return Result;
}

bool UMovieSceneAnimationMixerTrack::IsOverlapLargeEnoughForTransition(const TRange<FFrameNumber>& Overlap, const UObject* ContextObject)
{
	if (Overlap.IsEmpty() || !Overlap.HasLowerBound() || !Overlap.HasUpperBound())
	{
		return false;
	}

	int32 OverlapTicks = Overlap.GetUpperBoundValue().Value - Overlap.GetLowerBoundValue().Value;
	if (OverlapTicks <= 0)
	{
		return false;
	}

	// Section endpoints may not align to display frame boundaries (e.g. skeletal animation
	// sections whose tick length doesn't evenly divide the tick resolution). When such
	// sections are snapped to adjacent frame boundaries they can end up with a sub-frame
	// overlap that should not produce a transition. Require the overlap to span at least
	// one full display frame.
	if (const UMovieScene* MovieScene = ContextObject ? ContextObject->GetTypedOuter<UMovieScene>() : nullptr)
	{
		const double DisplayRateFPS = MovieScene->GetDisplayRate().AsDecimal();
		if (DisplayRateFPS > 0.0)
		{
			int32 MinOverlapTicks = FMath::FloorToInt32(MovieScene->GetTickResolution().AsDecimal() / DisplayRateFPS);
			if (OverlapTicks < MinOverlapTicks)
			{
				return false;
			}
		}
	}

	return true;
}

TArray<UMovieSceneSection*> UMovieSceneAnimationMixerTrack::FindOverlappingSections(UMovieSceneSection* Section) const
{
	TArray<UMovieSceneSection*> Result;

	if (!Section)
	{
		return Result;
	}

	const int32 RowIndex = Section->GetRowIndex();
	const TRange<FFrameNumber> SectionRange = Section->GetRange();

	if (UMovieSceneAnimationMixerLayer* Layer = GetLayer(RowIndex))
	{
		for (UMovieSceneSection* OtherSection : Layer->GetSections())
		{
			// Skip self and transitions
			if (OtherSection == Section ||
				OtherSection->IsA<UMovieSceneAnimTransitionSectionBase>())
			{
				continue;
			}

			// Check for overlap (must be at least one display frame to avoid
			// zero-width transitions from sub-frame endpoint misalignment)
			TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(SectionRange, OtherSection->GetRange());
			if (IsOverlapLargeEnoughForTransition(Overlap, this))
			{
				Result.Add(OtherSection);
			}
		}
	}

	return Result;
}

int32 UMovieSceneAnimationMixerTrack::CountSectionsOverlappingRange(const TRange<FFrameNumber>& Range, int32 RowIndex) const
{
	int32 Count = 0;

	if (UMovieSceneAnimationMixerLayer* Layer = GetLayer(RowIndex))
	{
		for (UMovieSceneSection* Section : Layer->GetSections())
		{
			// Skip transitions
			if (Section->IsA<UMovieSceneAnimTransitionSectionBase>())
			{
				continue;
			}

			// Check for overlap
			TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(Range, Section->GetRange());
			if (!Overlap.IsEmpty())
			{
				Count++;
			}
		}
	}

	return Count;
}

UMovieSceneAnimTransitionSectionBase* UMovieSceneAnimationMixerTrack::FindTransitionForPair(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection) const
{
	if (!FromSection || !ToSection)
	{
		return nullptr;
	}

	for (UMovieSceneSection* Sec : GetAllSections())
	{
		if (UMovieSceneAnimTransitionSectionBase* Transition = Cast<UMovieSceneAnimTransitionSectionBase>(Sec))
		{
			if (Transition->FromSection == FromSection && Transition->ToSection == ToSection)
			{
				return Transition;
			}
		}
	}

	return nullptr;
}

UMovieSceneAnimTransitionSectionBase* UMovieSceneAnimationMixerTrack::CreateTransitionSectionOfType(
	UMovieSceneSection* FromSection,
	UMovieSceneSection* ToSection,
	TSubclassOf<UMovieSceneAnimTransitionSectionBase> TransitionClass)
{
	if (!FromSection || !ToSection || !TransitionClass)
	{
		return nullptr;
	}

	// Verify the class is valid
	if (TransitionClass->HasAnyClassFlags(CLASS_Abstract))
	{
		return nullptr;
	}

	// Sections must be on the same row (layer)
	if (FromSection->GetRowIndex() != ToSection->GetRowIndex())
	{
		return nullptr;
	}

	// Compute the overlap range
	TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(
		FromSection->GetRange(),
		ToSection->GetRange()
	);

	if (!IsOverlapLargeEnoughForTransition(Overlap, this))
	{
		return nullptr;
	}

	Modify();

	// Create the transition section
	UMovieSceneAnimTransitionSectionBase* TransitionSection = NewObject<UMovieSceneAnimTransitionSectionBase>(
		this, TransitionClass, NAME_None, RF_Transactional);
	TransitionSection->SetRange(Overlap);
	TransitionSection->SetRowIndex(FromSection->GetRowIndex());
	TransitionSection->FromSection = FromSection;
	TransitionSection->ToSection = ToSection;

	// Ensure transition sections always draw on top of regular sections
	TransitionSection->SetOverlapPriority(INT32_MAX);

	// Initialize any default curves now that the range is set
	TransitionSection->InitializeDefaultCurve();

	// Add via FSectionParameter to ensure proper lifecycle hooks are called
	UMovieSceneTrack::AddSection(FSectionParameter{*TransitionSection});

	return TransitionSection;
}

void UMovieSceneAnimationMixerTrack::AutoCreateTransitionsForSection(UMovieSceneSection* Section)
{
	if (!Section || Section->IsA<UMovieSceneAnimTransitionSectionBase>())
	{
		return;
	}

	TArray<UMovieSceneSection*> OverlappingSections = FindOverlappingSections(Section);

	for (UMovieSceneSection* OtherSection : OverlappingSections)
	{
		TRange<FFrameNumber> SectionRange = Section->GetRange();
		TRange<FFrameNumber> OtherRange = OtherSection->GetRange();

		// Skip if either section doesn't have proper bounds
		if (!SectionRange.HasLowerBound() || !SectionRange.HasUpperBound() ||
			!OtherRange.HasLowerBound() || !OtherRange.HasUpperBound())
		{
			continue;
		}

		FFrameNumber SectionStart = SectionRange.GetLowerBoundValue();
		FFrameNumber SectionEnd = SectionRange.GetUpperBoundValue();
		FFrameNumber OtherStart = OtherRange.GetLowerBoundValue();
		FFrameNumber OtherEnd = OtherRange.GetUpperBoundValue();

		// Check if one section is fully contained within the other - this is not a valid transition
		// A transition requires both sections to extend beyond the overlap region
		bool bSectionInsideOther = (OtherStart <= SectionStart && SectionEnd <= OtherEnd);
		bool bOtherInsideSection = (SectionStart <= OtherStart && OtherEnd <= SectionEnd);

		if (bSectionInsideOther || bOtherInsideSection)
		{
			// One section is fully encompassed by the other - don't create a transition
			continue;
		}

		// Compute the overlap range between these two sections
		TRange<FFrameNumber> OverlapRange = TRange<FFrameNumber>::Intersection(SectionRange, OtherRange);

		// Check if more than 2 sections overlap this range - transitions only work between exactly 2 sections
		if (CountSectionsOverlappingRange(OverlapRange, Section->GetRowIndex()) > 2)
		{
			continue;
		}

		// Determine which is "from" and which is "to" based on start time
		UMovieSceneSection* FromSection = nullptr;
		UMovieSceneSection* ToSection = nullptr;

		if (SectionStart <= OtherStart)
		{
			FromSection = Section;
			ToSection = OtherSection;
		}
		else
		{
			FromSection = OtherSection;
			ToSection = Section;
		}

		// Check if a transition already exists for this pair
		if (!FindTransitionForPair(FromSection, ToSection))
		{
			// Create a crossfade transition (default type)
			CreateTransitionSection<UMovieSceneAnimCrossfadeTransitionSection>(FromSection, ToSection);
		}
	}
}

bool UMovieSceneAnimationMixerTrack::UpdateTransitionsForRow(int32 RowIndex)
{
	const int32 InitialSectionCount = GetAllSections().Num();

	UMovieSceneAnimationMixerLayer* Layer = GetLayer(RowIndex);
	if (!Layer)
	{
		return false;
	}

	// Step 1: Collect all non-transition sections on this row
	// We collect first because subsequent operations may modify the layer's sections
	TArray<UMovieSceneSection*> SectionsOnRow;
	TArray<UMovieSceneAnimTransitionSectionBase*> TransitionsOnRow;
	for (UMovieSceneSection* Sec : Layer->GetSections())
	{
		if (Sec)
		{
			if (UMovieSceneAnimTransitionSectionBase* Transition = Cast<UMovieSceneAnimTransitionSectionBase>(Sec))
			{
				TransitionsOnRow.Add(Transition);
			}
			else
			{
				SectionsOnRow.Add(Sec);
			}
		}
	}

	// Step 2: Validate existing transitions on this row - remove invalid ones
	// Invalid cases handled by UpdateBoundsFromSourceSections:
	// - From/To sections on different rows
	// - No overlap between From/To
	// - 3+ sections overlapping the transition range
	// - One section fully contained within the other
	for (UMovieSceneAnimTransitionSectionBase* Transition : TransitionsOnRow)
	{
		if (!Transition->UpdateBoundsFromSourceSections())
		{
			RemoveSection(*Transition);
		}
	}

	// Step 3: Auto-create transitions for any valid 2-section overlaps
	for (UMovieSceneSection* Sec : SectionsOnRow)
	{
		AutoCreateTransitionsForSection(Sec);
	}

	return GetAllSections().Num() != InitialSectionCount;
}

bool UMovieSceneAnimationMixerTrack::UpdateTransitionsForRows(const TSet<int32>& RowIndices)
{
	bool bAnyChanges = false;
	for (int32 RowIndex : RowIndices)
	{
		bAnyChanges |= UpdateTransitionsForRow(RowIndex);
	}
	return bAnyChanges;
}

#if WITH_EDITOR
EMovieSceneSectionMovedResult UMovieSceneAnimationMixerTrack::HandleTransitionSectionMoved(UMovieSceneAnimTransitionSectionBase& TransitionSection)
{
	if (!TransitionSection.FromSection || !TransitionSection.ToSection)
	{
		return EMovieSceneSectionMovedResult::None;
	}

	// If either source section is locked, block the move and restore bounds
	if (TransitionSection.FromSection->IsLocked() || TransitionSection.ToSection->IsLocked())
	{
		TransitionSection.UpdateBoundsFromSourceSections();
		return EMovieSceneSectionMovedResult::None;
	}

	// 1. Handle row changes - transition must stay on same row as from/to sections
	// This prevents vertical dragging of transitions
	int32 CorrectRowIndex = TransitionSection.FromSection->GetRowIndex();
	if (TransitionSection.GetRowIndex() != CorrectRowIndex)
	{
		TransitionSection.SetRowIndex(CorrectRowIndex);
	}

	// 2. Check if transition already matches the from/to overlap
	// This happens when all 3 sections are moved together - no adjustment needed
	TRange<FFrameNumber> CurrentOverlap = TRange<FFrameNumber>::Intersection(
		TransitionSection.FromSection->GetRange(),
		TransitionSection.ToSection->GetRange()
	);
	TRange<FFrameNumber> NewTransitionRange = TransitionSection.GetRange();

	if (CurrentOverlap == NewTransitionRange && !CurrentOverlap.IsEmpty())
	{
		// Transition already matches overlap - no from/to adjustment needed
		return EMovieSceneSectionMovedResult::None;
	}

	// 3. Handle horizontal movement - adjust from/to sections to match new transition bounds

	if (NewTransitionRange.IsEmpty() || !NewTransitionRange.HasLowerBound() || !NewTransitionRange.HasUpperBound())
	{
		return EMovieSceneSectionMovedResult::None;
	}

	FFrameNumber TransitionStart = NewTransitionRange.GetLowerBoundValue();
	FFrameNumber TransitionEnd = NewTransitionRange.GetUpperBoundValue();
	FFrameNumber TransitionSize = TransitionEnd - TransitionStart;

	// FROM's start is the hard limit on how far left the transition can go
	FFrameNumber FromStart = TransitionSection.FromSection->HasStartFrame() ? TransitionSection.FromSection->GetInclusiveStartFrame() : TNumericLimits<int32>::Min();

	// TO's end is the hard limit on how far right the transition can go
	FFrameNumber ToEnd = TransitionSection.ToSection->HasEndFrame() ? TransitionSection.ToSection->GetExclusiveEndFrame() : TNumericLimits<int32>::Max();

	// Check if the transition has hit its movement bounds
	// When moving, we must preserve the transition size - don't allow shrinking via movement
	// Users should use edge resize for that
	FFrameNumber FinalStart = TransitionStart;
	FFrameNumber FinalEnd = TransitionEnd;

	if (TransitionStart < FromStart)
	{
		// Hit left bound - clamp to FROM's start, preserving size
		FinalStart = FromStart;
		FinalEnd = FromStart + TransitionSize;
	}

	if (FinalEnd > ToEnd)
	{
		// Hit right bound - clamp to TO's end, preserving size
		FinalEnd = ToEnd;
		FinalStart = ToEnd - TransitionSize;
	}

	// Final validation - if we still can't fit, restore original bounds
	if (FinalStart < FromStart || FinalEnd > ToEnd || FinalStart >= FinalEnd)
	{
		TransitionSection.UpdateBoundsFromSourceSections();
		return EMovieSceneSectionMovedResult::None;
	}

	// Update FROM section's end to match transition's end (creates the overlap)
	TransitionSection.FromSection->Modify();
	TransitionSection.FromSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(FinalEnd));

	// Update TO section's start to match transition's start (creates the overlap)
	TransitionSection.ToSection->Modify();
	TransitionSection.ToSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(FinalStart));

	// Update transition to the final range
	TransitionSection.Modify();
	TransitionSection.SetRange(TRange<FFrameNumber>(
		TRangeBound<FFrameNumber>::Inclusive(FinalStart),
		TRangeBound<FFrameNumber>::Exclusive(FinalEnd)
	));

	// If we clamped the position, the keys were already moved by MoveSection but we moved the bounds
	// to a different position. We need to offset the keys to match the actual clamped position.
	// Keys were moved by the original delta, but should have been moved by (FinalStart - OriginalStart).
	// Since TransitionStart = OriginalStart + OriginalDelta, the correction is (FinalStart - TransitionStart).
	FFrameNumber KeyCorrection = FinalStart - TransitionStart;
	if (KeyCorrection != 0)
	{
		for (const FMovieSceneChannelEntry& Entry : TransitionSection.GetChannelProxy().GetAllEntries())
		{
			for (FMovieSceneChannel* Channel : Entry.GetChannels())
			{
				if (Channel)
				{
					Channel->Offset(KeyCorrection);
				}
			}
		}
	}

	return EMovieSceneSectionMovedResult::None;
}

EMovieSceneSectionMovedResult UMovieSceneAnimationMixerTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	// Handle transition section moves specially
	if (UMovieSceneAnimTransitionSectionBase* TransitionSection = Cast<UMovieSceneAnimTransitionSectionBase>(&Section))
	{
		return HandleTransitionSectionMoved(*TransitionSection);
	}

	const bool bSectionsChanged = UpdateTransitionsForRow(Section.GetRowIndex());

	// Reference section tracking: when a section moves, find bone matches
	// on other sections that reference it. Update their key times so the
	// bone match key follows the reference section's anchor point.
	// This runs on every move tick (including interactive drag) so the
	// key stays in sync during the drag.
	for (UMovieSceneSection* OtherSection : GetAllSections())
	{
		if (!OtherSection || OtherSection == &Section || OtherSection->IsA<UMovieSceneAnimTransitionSectionBase>())
		{
			continue;
		}

		UMovieSceneRootMotionSettingsDecoration* OtherDecoration = OtherSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!OtherDecoration || !OtherDecoration->HasBoneMatch())
		{
			continue;
		}

		FMovieSceneBoneMatchData MatchData = OtherDecoration->GetBoneMatchData();
		if (MatchData.ReferenceSection.Get() != &Section)
		{
			continue;
		}

		// Resolve the new key time based on the reference section's new position
		FFrameNumber NewTime;
		switch (MatchData.MatchTimeMode)
		{
		case EBoneMatchTimeMode::AtStartOfReferenceSection:
			NewTime = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : FFrameNumber(0);
			break;

		case EBoneMatchTimeMode::AtEndOfReferenceSection:
			NewTime = Section.HasEndFrame() ? (Section.GetExclusiveEndFrame() - 1) : FFrameNumber(0);
			break;

		case EBoneMatchTimeMode::InBetween:
		{
			FFrameNumber TargetStart = OtherSection->HasStartFrame()
				? OtherSection->GetInclusiveStartFrame() : FFrameNumber(0);
			FFrameNumber RefEnd = Section.HasEndFrame()
				? Section.GetExclusiveEndFrame() : TargetStart;
			NewTime = FFrameNumber((TargetStart.Value + RefEnd.Value) / 2);
			break;
		}

		default:
			continue;
		}

		const bool bWasValid = MatchData.bIsValid;

		OtherDecoration->Modify();
		OtherDecoration->BoneMatchChannel.NotifyReferenceChanged(NewTime);

		// Dirty the decoration proxy when validity flips so the channel
		// display name updates.
		FMovieSceneBoneMatchData UpdatedData = OtherDecoration->GetBoneMatchData();
		if (UpdatedData.bIsValid != bWasValid)
		{
			OtherDecoration->InvalidateDecorationChannelProxy();
		}
	}

	InvalidateAccumulatedOffsetCache();

	if (UMovieSceneRootMotionSettingsDecoration* Decoration = Section.FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
	{
		if (Decoration->HasBoneMatch())
		{
			TOptional<FFrameNumber> NewStart;
			TOptional<FFrameNumber> NewEnd;
			if (Section.HasStartFrame())
			{
				NewStart = Section.GetInclusiveStartFrame();
			}
			if (Section.HasEndFrame())
			{
				NewEnd = Section.GetExclusiveEndFrame() - 1;
			}

			Decoration->Modify();
			Decoration->BoneMatchChannel.ApplyOwningSectionRange(NewStart, NewEnd);

			if (Params.MoveType == EPropertyChangeType::ValueSet)
			{
				Decoration->InvalidateDecorationChannelProxy();
			}
		}
	}

	return bSectionsChanged ? EMovieSceneSectionMovedResult::SectionsChanged : EMovieSceneSectionMovedResult::None;
}
#endif // WITH_EDITOR

EMovieSceneTrackEasingSupportFlags UMovieSceneAnimationMixerTrack::SupportsEasing(FMovieSceneSupportsEasingParams& Params) const
{
	if (Params.ForSection && Params.ForSection->IsA(UMovieSceneAnimMixerMaskSection::StaticClass()))
	{
		return EMovieSceneTrackEasingSupportFlags::All;
	}
	// Disable manual easing since the mixer track uses transition sections for blending
	// instead of per-section easing. This prevents users from accidentally setting up
	// easing that would conflict with transition blends.
	return EMovieSceneTrackEasingSupportFlags::None;
}

void UMovieSceneAnimationMixerTrack::UpdateEasing()
{
	// Update easing on a per-section basis. Skeletal animations are handled by transitions within a layer, easing is only applied to sections within
	// section provider decorations.

	for (TObjectPtr<UMovieSceneAnimationMixerLayer> Layer : Layers)
	{
		if (!Layer)
		{
			continue;
		}
		for (UObject* Decoration : Layer.Get()->GetActiveDecorations())
		{
			if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
			{
				TArrayView<TObjectPtr<UMovieSceneSection>> Sections = SectionProvider->GetSections();
				int32 MaxRowIndex = 0;
				TArray<UMovieSceneSection*> RowSections;
				for (TObjectPtr<UMovieSceneSection> Section : Sections)
				{
					MaxRowIndex = FMath::Max(MaxRowIndex, Section->GetRowIndex());
				}
				for (int32 RowIndex = 0; RowIndex <= MaxRowIndex; ++RowIndex)
				{
					RowSections.Reset();
					for (TObjectPtr<UMovieSceneSection> Section : Sections)
					{
						if (Section && Section->GetRowIndex() == RowIndex)
						{
							RowSections.Add(Section);
						}
					}
					for (int32 SectionIndex = 0; SectionIndex < RowSections.Num(); ++SectionIndex)
					{
						UMovieSceneSection* CurrentSection = RowSections[SectionIndex];

						FMovieSceneSupportsEasingParams SupportsEasingParams(CurrentSection);
						EMovieSceneTrackEasingSupportFlags EasingFlags = SupportsEasing(SupportsEasingParams);

						// Auto-deactivate manual easing if we lost the ability to use it.
						if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseIn))
						{
							CurrentSection->Easing.bManualEaseIn = false;
						}
						if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseOut))
						{
							CurrentSection->Easing.bManualEaseOut = false;
						}

						if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::AutomaticEasing))
						{
							if (CurrentSection->Easing.AutoEaseInDuration != 0 || CurrentSection->Easing.AutoEaseOutDuration != 0)
							{
								CurrentSection->Modify();

								CurrentSection->Easing.AutoEaseInDuration = 0;
								CurrentSection->Easing.AutoEaseOutDuration = 0;
							}
							continue;
						}

						// Check overlaps with exclusive ranges so that sections can butt up against each other
						UMovieSceneTrack* OuterTrack = CurrentSection->GetTypedOuter<UMovieSceneTrack>();
						int32 MaxEaseIn = 0;
						int32 MaxEaseOut = 0;
						bool bIsEntirelyUnderlapped = false;

						TRange<FFrameNumber> CurrentSectionRange = CurrentSection->GetRange();
						for (int32 OtherIndex = 0; OtherIndex < RowSections.Num(); ++OtherIndex)
						{
							if (OtherIndex == SectionIndex)
							{
								continue;
							}

							UMovieSceneSection* Other = RowSections[OtherIndex];
							TRange<FFrameNumber> OtherSectionRange = Other->GetRange();

							if (!OtherSectionRange.HasLowerBound() && !OtherSectionRange.HasUpperBound())
							{
								// If we're testing against an infinite range we want to use the PlayRange of the sequence
								// instead so that blends stop at the end of a clip instead of a quarter of the length.
								UMovieScene* OuterScene = OuterTrack->GetTypedOuter<UMovieScene>();
								OtherSectionRange = OuterScene->GetPlaybackRange();
							}

							bIsEntirelyUnderlapped = bIsEntirelyUnderlapped || OtherSectionRange.Contains(CurrentSectionRange);

							// Check the lower bound of the current section against the other section's upper bound
							const bool bSectionRangeContainsOtherUpperBound = !OtherSectionRange.GetUpperBound().IsOpen() && !CurrentSectionRange.GetLowerBound().IsOpen() && CurrentSectionRange.Contains(OtherSectionRange.GetUpperBoundValue());
							const bool bSectionRangeContainsOtherLowerBound = !OtherSectionRange.GetLowerBound().IsOpen() && !CurrentSectionRange.GetUpperBound().IsOpen() && CurrentSectionRange.Contains(OtherSectionRange.GetLowerBoundValue());
							if (bSectionRangeContainsOtherUpperBound && !bSectionRangeContainsOtherLowerBound)
							{
								const int32 Difference = UE::MovieScene::DiscreteSize(TRange<FFrameNumber>(CurrentSectionRange.GetLowerBound(), OtherSectionRange.GetUpperBound()));
								MaxEaseIn = FMath::Max(MaxEaseIn, Difference);
							}

							if (bSectionRangeContainsOtherLowerBound &&!bSectionRangeContainsOtherUpperBound)
							{
								const int32 Difference = UE::MovieScene::DiscreteSize(TRange<FFrameNumber>(OtherSectionRange.GetLowerBound(), CurrentSectionRange.GetUpperBound()));
								MaxEaseOut = FMath::Max(MaxEaseOut, Difference);
							}
						}

						const bool  bIsFinite = CurrentSectionRange.HasLowerBound() && CurrentSectionRange.HasUpperBound();
						const int32 MaxSize   = bIsFinite ? UE::MovieScene::DiscreteSize(CurrentSectionRange) : TNumericLimits<int32>::Max();

						if (MaxEaseOut == 0 && MaxEaseIn == 0 && bIsEntirelyUnderlapped)
						{
							MaxEaseOut = MaxEaseIn = MaxSize / 4;
						}

						// Only modify the section if the ease in or out times have actually changed
						MaxEaseIn  = FMath::Clamp(MaxEaseIn, 0, MaxSize);
						MaxEaseOut = FMath::Clamp(MaxEaseOut, 0, MaxSize);

						if (CurrentSection->Easing.AutoEaseInDuration != MaxEaseIn || CurrentSection->Easing.AutoEaseOutDuration != MaxEaseOut)
						{
							CurrentSection->Modify();

							CurrentSection->Easing.AutoEaseInDuration  = MaxEaseIn;
							CurrentSection->Easing.AutoEaseOutDuration = MaxEaseOut;
						}
					}
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneAnimationMixerTrack::GetTrackRowDisplayName(int32 RowIndex) const
{
	return FText::Format(LOCTEXT("AnimMixer", "Mix Layer {0}"), RowIndex);
}

FText UMovieSceneAnimationMixerTrack::GetDefaultDisplayName() const
{
	return NSLOCTEXT("AnimMixer", "DefaultTrackName", "Animation Mixer");
}

bool UMovieSceneAnimationMixerTrack::CanRename() const
{
	return true;
}

#endif // WITH_EDITORONLY_DATA


UMovieSceneTrack* UMovieSceneAnimationMixerTrack::AddChildTrack(FGuid ObjectBinding, TSubclassOf<UMovieSceneTrack> TrackClass, int32 RowIndex)
{
	if (UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>())
	{
		if (UMovieScene* MovieScene = OuterSequence->GetMovieScene())
		{
			if (ObjectBinding.IsValid())
			{
				Modify();
				UMovieSceneTrack* NewTrack = NewObject<UMovieSceneTrack>(this, TrackClass, NAME_None, RF_Transactional);
				// Set that this is a child track. This ensures it will be not be listed with the main tracks of the sequence
				NewTrack->ParentTrack = this;
				if (MovieScene->AddGivenTrack(NewTrack, ObjectBinding))
				{
					ChildTracks.Add(NewTrack, RowIndex);

					// Add track decoration to the child track- this handles adding required components to entities the track manages
					UMovieSceneAnimationTrackDecoration* Decoration = NewObject<UMovieSceneAnimationTrackDecoration>(NewTrack, NAME_None, RF_Transactional);
					Decoration->MixerTrack = this;
					NewTrack->AddDecoration(Decoration);

					// Create/update layer for this child track
					UMovieSceneAnimationMixerLayer* Layer = GetOrCreateLayer(RowIndex);
					if (Layer)
					{
						Layer->SetChildTrack(NewTrack);
					}

					return NewTrack;
				}
			}
		}
	}
	return nullptr;
}

int32 UMovieSceneAnimationMixerTrack::GetNextChildTrackRowIndex() const
{
	int32 MaxRowIndex = -1;
	for (const auto& Pair : ChildTracks)
	{
		MaxRowIndex = FMath::Max(MaxRowIndex, Pair.Value);
	}
	return MaxRowIndex + 1;
}

UMovieSceneSection* UMovieSceneAnimationMixerTrack::AddNewAnimationOnRow(FFrameNumber KeyTime, UAnimSequenceBase* AnimSequence, int32 RowIndex)
{
	// If RowIndex is INDEX_NONE, use base class behavior which will find the next available row
	if (RowIndex == INDEX_NONE)
	{
		return Super::AddNewAnimationOnRow(KeyTime, AnimSequence, RowIndex);
	}

	// For mixer tracks with explicit row index, we explicitly WANT overlaps,
	// so we bypass InitialPlacementOnRow which would bump other sections to avoid overlap.
	UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(CreateNewSection());
	if (!NewSection)
	{
		return nullptr;
	}

	// Calculate animation length
	FFrameTime AnimationLength = AnimSequence->GetPlayLength() * GetTypedOuter<UMovieScene>()->GetTickResolution();
	int32 IFrameNumber = AnimationLength.FrameNumber.Value + (int32)(AnimationLength.GetSubFrame() + 0.5f) + 1;

	// Set range and row directly without bumping other sections
	NewSection->SetRange(TRange<FFrameNumber>(KeyTime, KeyTime + IFrameNumber));
	NewSection->SetRowIndex(RowIndex);
	NewSection->Params.Animation = AnimSequence;

	// Add via FSectionParameter to ensure proper lifecycle hooks are called
	UMovieSceneTrack::AddSection(FSectionParameter{ *NewSection });

	return NewSection;
}

void UMovieSceneAnimationMixerTrack::AddSectionWithAutoRow(UMovieSceneSection* Section)
{
	if (!Section)
	{
		return;
	}

	// Rename the section to have this track as the outer
	Section->Rename(nullptr, this);

	// Compute next available row considering both sections and child tracks
	int32 NextRowIndex = ComputeNextAvailableRowIndex();

	// Assign next available row
	Section->SetRowIndex(NextRowIndex);

	// Add the section via FSectionParameter to ensure OnSectionAddedImpl is called,
	// which assigns the section to a mixer layer so it appears in the UI.
	UMovieSceneTrack::AddSection(FSectionParameter{*Section});
}

int32 UMovieSceneAnimationMixerTrack::ComputeNextAvailableRowIndex() const
{
	return Layers.Num();
}

void UMovieSceneAnimationMixerTrack::GetAllChildTracks(TArray<TObjectPtr<UMovieSceneTrack>>& OutChildTracks) const
{
	ChildTracks.GenerateKeyArray(OutChildTracks);
}

void UMovieSceneAnimationMixerTrack::RemoveChildTrack(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		return;
	}

	Modify();


	// Remove child track reference from its layer
	UMovieSceneAnimationMixerLayer* Layer = GetLayerForChildTrack(Track);
	if (Layer)
	{
		Layer->SetChildTrack(nullptr);
	}

	ChildTracks.Remove(Track);

	// Remove track decoration (if it exists)
	Track->RemoveDecoration(UMovieSceneAnimationTrackDecoration::StaticClass());

	if (UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>())
	{
		if (UMovieScene* MovieScene = OuterSequence->GetMovieScene())
		{
			MovieScene->RemoveTrack(*Track);
		}
	}
}

void UMovieSceneAnimationMixerTrack::OnRemovedFromMovieSceneImpl()
{
	// Child tracks (e.g. Control Rig tracks on mixer layers) are registered on
	// the same object binding as the mixer. Deleting the mixer alone would leave
	// them on the binding and they'd keep driving the character. Cascade through
	// RemoveChildTrack so layer/map/decoration/movie-scene cleanup stays shared
	// with single-layer deletion. Copy keys first since RemoveChildTrack mutates
	// ChildTracks.
	TArray<TObjectPtr<UMovieSceneTrack>> TracksToRemove;
	ChildTracks.GenerateKeyArray(TracksToRemove);
	for (const TObjectPtr<UMovieSceneTrack>& ChildTrack : TracksToRemove)
	{
		if (ChildTrack)
		{
			RemoveChildTrack(ChildTrack);
		}
	}
}

UMovieSceneTrack* UMovieSceneAnimationMixerTrack::GetChildTrackAtRow(int32 RowIndex) const
{
	for (const auto& Pair : ChildTracks)
	{
		if (Pair.Value == RowIndex)
		{
			return Pair.Key;
		}
	}
	return nullptr;
}

bool UMovieSceneAnimationMixerTrack::IsChildTrack(UMovieSceneTrack* Track) const
{
	return ChildTracks.Contains(Track);
}

int32 UMovieSceneAnimationMixerTrack::GetChildTrackRow(UMovieSceneTrack* Track) const
{
	const int32* RowPtr = ChildTracks.Find(Track);

	return RowPtr ? *RowPtr : INDEX_NONE;
}

void UMovieSceneAnimationMixerTrack::SetChildTrackRow(UMovieSceneTrack* Track, int32 NewRowIndex)
{
	if (!Track || !IsChildTrack(Track))
	{
		return;
	}

	int32 OldRowIndex = GetChildTrackRow(Track);
	if (OldRowIndex == NewRowIndex)
	{
		return;
	}

	Modify();

	ChildTracks[Track] = NewRowIndex;
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimationMixerTrack::GetOrCreateLayer(int32 RowIndex)
{
	// Find existing layer at this row index
	for (UMovieSceneAnimationMixerLayer* Layer : Layers)
	{
		if (Layer)
		{
			int32 LayerIndex = Layers.IndexOfByKey(Layer);
			if (LayerIndex == RowIndex)
			{
				return Layer;
			}
		}
	}

	// Create new layer
	Modify();
	UMovieSceneAnimationMixerLayer* NewLayer = NewObject<UMovieSceneAnimationMixerLayer>(this, NAME_None, RF_Transactional);

	// Insert at correct position to keep array sorted by row index
	if (RowIndex >= Layers.Num())
	{
		// Expand array if needed
		Layers.SetNum(RowIndex + 1);
	}

	Layers[RowIndex] = NewLayer;
	return NewLayer;
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimationMixerTrack::GetLayer(int32 RowIndex) const
{
	if (Layers.IsValidIndex(RowIndex))
	{
		return Layers[RowIndex];
	}
	return nullptr;
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimationMixerTrack::GetLayerForSection(UMovieSceneSection* Section) const
{
	if (!Section)
	{
		return nullptr;
	}

	int32 RowIndex = Section->GetRowIndex();
	return GetLayer(RowIndex);
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimationMixerTrack::GetLayerForChildTrack(UMovieSceneTrack* Track) const
{
	if (!Track)
	{
		return nullptr;
	}

	const int32* RowIndex = ChildTracks.Find(Track);
	if (RowIndex)
	{
		return GetLayer(*RowIndex);
	}
	return nullptr;
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimationMixerTrack::InsertLayer(int32 InsertIndex)
{
	// Clamp insert index to valid range
	InsertIndex = FMath::Clamp(InsertIndex, 0, Layers.Num());

	// Create the new layer
	UMovieSceneAnimationMixerLayer* NewLayer = NewObject<UMovieSceneAnimationMixerLayer>(this, NAME_None, RF_Transactional);

	// Insert at the specified position
	Layers.Insert(NewLayer, InsertIndex);

	// Update row indices for all layers from InsertIndex onwards
	for (int32 LayerIndex = InsertIndex; LayerIndex < Layers.Num(); ++LayerIndex)
	{
		UMovieSceneAnimationMixerLayer* Layer = Layers[LayerIndex];
		if (!Layer)
		{
			continue;
		}

		// Update sections in this layer
		for (UMovieSceneSection* Section : Layer->GetSections())
		{
			if (Section)
			{
				Section->Modify();
				Section->SetRowIndex(LayerIndex);
			}
		}

		// Update decoration section row indices to match the layer so
		// per-row mute/disable applies correctly after reordering.
		for (UObject* Decoration : Layer->GetDecorations())
		{
			if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
			{
				for (UMovieSceneSection* DecSection : SectionProvider->GetSections())
				{
					if (DecSection)
					{
						DecSection->Modify();
						DecSection->SetRowIndex(LayerIndex);
					}
				}
			}
		}

		// Update child track row index if this layer has one
		if (UMovieSceneTrack* ChildTrack = Layer->GetChildTrack())
		{
			if (int32* TrackRowIndex = ChildTracks.Find(ChildTrack))
			{
				*TrackRowIndex = LayerIndex;
			}
		}
	}

	return NewLayer;
}

void UMovieSceneAnimationMixerTrack::InvalidateAccumulatedOffsetCache()
{
	SetRootMotionsDirty();

	if (UMovieSceneRootMotionTargetDecoration* Decoration = FindDecoration<UMovieSceneRootMotionTargetDecoration>())
	{
		if (UMovieSceneRootMotionSection* RootMotionSection = Decoration->GetRootMotionSection())
		{
			RootMotionSection->InvalidateAccumulatedOffsetCache();
		}
	}
}

bool UMovieSceneAnimationMixerTrack::HasDirtyAccumulatedOffsetCache() const
{
	if (UMovieSceneRootMotionTargetDecoration* Decoration = FindDecoration<UMovieSceneRootMotionTargetDecoration>())
	{
		if (const UMovieSceneRootMotionSection* RootMotionSection = Decoration->GetRootMotionSection())
		{
			return RootMotionSection->bAccumulatedOffsetCacheDirty;
		}
	}
	return false;
}

void UMovieSceneAnimationMixerTrack::RebuildDirtyAccumulatedOffsetCache(
	UMovieSceneEntitySystemLinker* Linker,
	UE::MovieScene::FInstanceHandle InstanceHandle)
{
	if (UMovieSceneRootMotionTargetDecoration* Decoration = FindDecoration<UMovieSceneRootMotionTargetDecoration>())
	{
		if (UMovieSceneRootMotionSection* RootMotionSection = Decoration->GetRootMotionSection())
		{
			if (!RootMotionSection->bAccumulatedOffsetCacheDirty)
			{
				return;
			}

			RootMotionSection->bAccumulatedOffsetCacheDirty = false;

			// Bake each qualifying section's raw anim-space root at its start;
			// the runtime conversion rebases per-frame motion relative to it.
			for (UMovieSceneSection* Section : GetAllSections())
			{
				if (!Section)
				{
					continue;
				}

				auto* Settings = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
				if (!Settings || !Section->HasStartFrame())
				{
					continue;
				}

				// Clear before the qualifying check so a mode change can't leave
				// the previous run's values readable.
				Settings->InitialRootTransform = FTransform::Identity;
				Settings->LoopStartRootTransform = FTransform::Identity;
				Settings->bInitialRootTransformValid = false;

				const EMovieSceneRootMotionTransformMode Mode = Settings->GetRootTransformMode();
				if (Mode == EMovieSceneRootMotionTransformMode::Override)
				{
					continue;
				}
				const bool bIsAccumulatedOffset = Mode == EMovieSceneRootMotionTransformMode::AccumulatedOffset;
				const bool bWantsLoopAccumulation =
					Settings->GetLoopMode() == EMovieSceneAnimLoopMode::Accumulated;
				if (!bIsAccumulatedOffset && !bWantsLoopAccumulation)
				{
					continue;
				}

				FFrameTime SectionStart(Section->GetInclusiveStartFrame());

				using namespace UE::MovieScene::AnimMixerBakeEvaluation;
				FBakeFilter Filter;
				Filter.IncludeOnlySections.Add(FObjectKey(Section));
				Filter.bSkipRootMotionConversion = true;

				FBakeResult Result = EvaluateAtTime(
					Linker, InstanceHandle, this, SectionStart, Filter);

				Settings->InitialRootTransform = Result.RootMotionTransform;

				// Compute the root transform at the animation's natural start for
				// post-first-loop iterations. When the section starts mid-animation
				// (extended left), the first loop wraps back to anim frame 0 which
				// has a different root than the section start.
				{
					const FFrameRate TickResolution = GetTypedOuter<UMovieScene>()->GetTickResolution();
					TOptional<FFrameTime> FirstLoopBoundary = Settings->GetFirstLoopBoundary(TickResolution);
					if (FirstLoopBoundary.IsSet())
					{
						FBakeResult LoopResult = EvaluateAtTime(
							Linker, InstanceHandle, this, FirstLoopBoundary.GetValue(), Filter);
						Settings->LoopStartRootTransform = LoopResult.RootMotionTransform;
					}
					else
					{
						Settings->LoopStartRootTransform = Result.RootMotionTransform;
					}
				}

				Settings->bInitialRootTransformValid = true;
			}

			RootMotionSection->RebuildAccumulatedOffsetCache(Linker, InstanceHandle, this);
		}
	}
}

#undef LOCTEXT_NAMESPACE