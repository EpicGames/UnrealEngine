// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationTrackDecoration.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneAnimationMixerLayer.h"

void UMovieSceneAnimationTrackDecoration::ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	UMovieSceneTrack* ChildTrack = GetTypedOuter<UMovieSceneTrack>();

	UMovieSceneAnimationMixerLayer* MixerLayer = MixerTrack ? MixerTrack->GetLayerForChildTrack(ChildTrack) : nullptr;

	TInstancedStruct<FMovieSceneMixedAnimationTarget> Target;
	if (MixerTrack != nullptr && MixerTrack->MixedAnimationTarget.IsValid())
	{
		Target = MixerTrack->MixedAnimationTarget;
	}
	else
	{
		Target = TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make();
	}

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(FMovieSceneTracksComponentTypes::Get()->Tags.AnimMixerPoseProducer)
		// Exempt mixer pose producers from hierarchical-bias override - they are routed/blended by the
		// mixer (by target), not overridden by sequence depth.
		.AddTag(FBuiltInComponentTypes::Get()->Tags.IgnoreHierarchicalBias)
		.Add(AnimMixerComponents->Target, Target)
		.Add(AnimMixerComponents->Priority, MixerLayer ? MixerLayer->GetLayerIndex() : 0)
		.Add(AnimMixerComponents->MixerLayer, MixerLayer)
		.AddMutualComponents()
	);
}
