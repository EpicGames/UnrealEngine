// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBoneMatchChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoneMatchChannel)

void FMovieSceneBoneMatchChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneBoneMatchChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneBoneMatchChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	// Manual key drag disconnects from anchors: switch to AtCurrentTime
	// and clear the reference section.
	for (int32 i = 0; i < InHandles.Num(); ++i)
	{
		const int32 KeyIndex = GetData().GetIndex(InHandles[i]);
		if (KeyIndex != INDEX_NONE)
		{
			Values[KeyIndex].MatchTimeMode = EBoneMatchTimeMode::AtCurrentTime;
			Values[KeyIndex].ReferenceSection = nullptr;
		}
	}

	// MoveKeyInternal fires OnKeyMovedEvent automatically
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneBoneMatchChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneBoneMatchChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneBoneMatchChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneBoneMatchChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneBoneMatchChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneBoneMatchChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneBoneMatchChannel::Reset()
{
	GetData().Reset();
}

void FMovieSceneBoneMatchChannel::ApplyOwningSectionRange(TOptional<FFrameNumber> NewStart, TOptional<FFrameNumber> NewEnd)
{
	TArray<FKeyMoveEventItem> Items;
	for (int32 i = 0; i < Times.Num(); ++i)
	{
		const EBoneMatchTimeMode Mode = Values[i].MatchTimeMode;

		TOptional<FFrameNumber> Target;
		if (Mode == EBoneMatchTimeMode::AtStartOfSelectedSection && NewStart.IsSet())
		{
			Target = NewStart.GetValue();
		}
		else if (Mode == EBoneMatchTimeMode::AtEndOfSelectedSection && NewEnd.IsSet())
		{
			Target = NewEnd.GetValue();
		}

		if (!Target.IsSet() || Times[i] == Target.GetValue())
		{
			continue;
		}

		const FFrameNumber OldTime = Times[i];
		Times[i] = Target.GetValue();
		Items.Add(FKeyMoveEventItem(i, OldTime, i, Times[i]));
	}

	if (Items.Num() > 0)
	{
		OnKeyMovedEvent().Broadcast(this, Items);
	}
}

void FMovieSceneBoneMatchChannel::NotifyReferenceChanged(FFrameNumber NewResolvedTime)
{
	// Called when a reference section moves. The caller already verified
	// that this channel's bone match references the moved section.
	// Only adjust keys whose mode is anchored to a reference section.
	TArray<FKeyMoveEventItem> Items;
	for (int32 i = 0; i < Times.Num(); ++i)
	{
		const EBoneMatchTimeMode Mode = Values[i].MatchTimeMode;
		if (Mode != EBoneMatchTimeMode::AtStartOfReferenceSection &&
			Mode != EBoneMatchTimeMode::AtEndOfReferenceSection &&
			Mode != EBoneMatchTimeMode::InBetween)
		{
			continue;
		}

		const FFrameNumber OldTime = Times[i];
		Times[i] = NewResolvedTime;
		Items.Add(FKeyMoveEventItem(i, OldTime, i, NewResolvedTime));
	}

	if (Items.Num() > 0)
	{
		OnKeyMovedEvent().Broadcast(this, Items);
	}
}

void FMovieSceneBoneMatchChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
}

void FMovieSceneBoneMatchChannel::ClearDefault()
{
}

FKeyHandle FMovieSceneBoneMatchChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneBoneMatchChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}
