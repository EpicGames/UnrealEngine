// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLayerWeightDecoration.h"

#include "MovieSceneAnimMixerLayerWeightSection.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "Channels/MovieSceneChannelProxy.h"

#define LOCTEXT_NAMESPACE "MovieSceneLayerWeightDecoration"

UMovieSceneLayerWeightDecoration::UMovieSceneLayerWeightDecoration(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	WeightChannel.SetDefault(1.0f);
}

EMovieSceneChannelProxyType UMovieSceneLayerWeightDecoration::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData)
{
#if WITH_EDITOR
	FMovieSceneChannelMetaData MetaData(TEXT("WeightChannel"), LOCTEXT("Weight", "Weight"));
	MetaData.WeakOwningObject = this;
	OutProxyData.Add(WeightChannel, MetaData, TMovieSceneExternalValue<float>());
#else
	OutProxyData.Add(WeightChannel);
#endif
	return EMovieSceneChannelProxyType::Dynamic;
}

UMovieSceneSection* UMovieSceneLayerWeightDecoration::CreateNewSection()
{
	return NewObject<UMovieSceneAnimMixerLayerWeightSection>(this, UMovieSceneAnimMixerLayerWeightSection::StaticClass(), NAME_None, RF_Transactional);
}

TSubclassOf<UMovieSceneSection> UMovieSceneLayerWeightDecoration::GetHostedSectionClass() const
{
	return UMovieSceneAnimMixerLayerWeightSection::StaticClass();
}

void UMovieSceneLayerWeightDecoration::AddSection(UMovieSceneSection* InSection)
{
	UMovieSceneAnimMixerLayerWeightSection* NewWeightSection = Cast<UMovieSceneAnimMixerLayerWeightSection>(InSection);
	if (!NewWeightSection || WeightSections.Num() > 0)
	{
		return;
	}

	Modify();

	WeightSections.Add(NewWeightSection);

	MarkAsChanged();
#if WITH_EDITOR
	MarkStructureChanged();
#endif

	UMovieSceneAnimationMixerLayer* Layer = GetTypedOuter<UMovieSceneAnimationMixerLayer>();
	NewWeightSection->SetAnimMixerLayer(Layer);

	if (Layer)
	{
		NewWeightSection->SetRowIndex(Layer->GetLayerIndex());
	}
}

void UMovieSceneLayerWeightDecoration::RemoveSection(UMovieSceneSection& SectionToRemove)
{
	Modify();
	WeightSections.RemoveAll([&SectionToRemove](const UMovieSceneSection* InSection){ return InSection == &SectionToRemove; });
	MarkAsChanged();
#if WITH_EDITOR
	MarkStructureChanged();
#endif
}

#if WITH_EDITOR
void UMovieSceneLayerWeightDecoration::PostEditUndo()
{
	Super::PostEditUndo();

	UMovieSceneAnimationMixerLayer* Layer = GetTypedOuter<UMovieSceneAnimationMixerLayer>();
	for (TObjectPtr<UMovieSceneSection> Section : WeightSections)
	{
		if (UMovieSceneAnimMixerLayerWeightSection* WeightSection = Cast<UMovieSceneAnimMixerLayerWeightSection>(Section))
		{
			WeightSection->SetAnimMixerLayer(Layer);
		}
	}

	MarkStructureChanged();
}
#endif

#undef LOCTEXT_NAMESPACE
