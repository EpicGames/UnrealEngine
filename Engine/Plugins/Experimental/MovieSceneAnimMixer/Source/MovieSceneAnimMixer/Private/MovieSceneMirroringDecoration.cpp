// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMirroringDecoration.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "AnimMixerComponentTypes.h"

void UMovieSceneMirroringDecoration::ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (MirrorDataTable)
	{
		FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(AnimMixerComponents->MirrorTable, MirrorDataTable)
			.AddMutualComponents()
		);
	}
}
