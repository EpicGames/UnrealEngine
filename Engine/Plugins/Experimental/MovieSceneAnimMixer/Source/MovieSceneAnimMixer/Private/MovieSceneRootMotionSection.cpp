// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneRootMotionSection.h"

#include "AnimMixerBakeEvaluation.h"
#include "AnimMixerComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieScene.h"
#include "MovieSceneRootMotionTargetDecoration.h"
#include "MovieSceneTrack.h"
#include "MovieSceneTransformTypes.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"

namespace
{
	// Loop-Mode opt-in: gates the section's own seed and loop-boundary entries.
	bool SectionUsesAccumulatedLoopMode(const UMovieSceneRootMotionSettingsDecoration* Settings)
	{
		if (!Settings)
		{
			return false;
		}
		if (Settings->GetLoopMode() != EMovieSceneAnimLoopMode::Accumulated)
		{
			return false;
		}
		return Settings->GetRootTransformMode() != EMovieSceneRootMotionTransformMode::Override;
	}

	// Superset that also includes AccumulatedOffset transform mode regardless of
	// Loop Mode, so its end-of-section entry stays in delta form for downstream
	// chaining even when its own loops reset.
	bool SectionUsesAccumulatedOffsetDeltaPath(const UMovieSceneRootMotionSettingsDecoration* Settings)
	{
		if (!Settings)
		{
			return false;
		}
		const EMovieSceneRootMotionTransformMode Mode = Settings->GetRootTransformMode();
		if (Mode == EMovieSceneRootMotionTransformMode::Override)
		{
			return false;
		}
		if (Mode == EMovieSceneRootMotionTransformMode::AccumulatedOffset)
		{
			return true;
		}
		return Settings->GetLoopMode() == EMovieSceneAnimLoopMode::Accumulated;
	}

	// Computes the runtime's loop index at an outer tick by harvesting the
	// breadcrumb the transform stack records at the loop sub-transform; that
	// breadcrumb is exactly what the runtime feeds into LoopTime. Only the
	// differences between consecutive ticks matter, so any constant offset
	// from the loop sub-transform's own Offset cancels out.
	int32 GetRuntimeLoopIndexAt(
		const FMovieSceneSequenceTransform& Transform,
		FFrameNumber Tick,
		float Duration)
	{
		FMovieSceneTransformBreadcrumbs Crumbs;
		UE::MovieScene::FTransformTimeParams Params;
		Params.HarvestBreadcrumbs(Crumbs);
		Transform.TransformTime(FFrameTime(Tick), Params);

		if (Crumbs.Num() == 0)
		{
			return 0;
		}
		const FFrameTime LoopInput = Crumbs[Crumbs.Num() - 1];
		return FMath::FloorToInt(LoopInput.AsDecimal() / (double)Duration);
	}
} // anonymous namespace

FFrameNumber UMovieSceneRootMotionSection::ResolveLoopTransitionTick(
	const FMovieSceneSequenceTransform& Transform,
	FFrameTime ApproximateBoundary,
	float Duration)
{
	const FFrameNumber Floor = ApproximateBoundary.FloorToFrame();

	// Guard the index division below from inf/NaN. Form catches NaN too.
	if (!(Duration > 0.0f))
	{
		return Floor + 1;
	}

	// Window covers float-precision drift in either direction while staying
	// small enough that only one transition can fall inside for a sane loop.
	// Direction-agnostic: forward play increases the index, reverse decreases
	// it; either way the transition is the tick whose index differs from T-1's.
	constexpr int32 SearchBefore = 2;
	constexpr int32 SearchAfter  = 3;

	int32 PrevIndex = GetRuntimeLoopIndexAt(Transform, Floor - (SearchBefore + 1), Duration);
	for (int32 Offset = -SearchBefore; Offset <= SearchAfter; ++Offset)
	{
		const FFrameNumber T = Floor + Offset;
		const int32 Index = GetRuntimeLoopIndexAt(Transform, T, Duration);
		if (Index != PrevIndex)
		{
			return T;
		}
		PrevIndex = Index;
	}
	return Floor + 1;
}

UMovieSceneRootMotionSection::UMovieSceneRootMotionSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	RootDestinationChannel.SetEnum(StaticEnum<EMovieSceneRootMotionDestination>());
	RootDestinationChannel.SetDefault((uint8)EMovieSceneRootMotionDestination::RootBone);

	SectionRange.Value = TRange<FFrameNumber>::All();

	// Invalidate root motion caches whenever this section is modified.
	OnSignatureChanged().AddWeakLambda(this, [this]()
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>())
		{
			MixerTrack->InvalidateAccumulatedOffsetCache();
		}
	});
}

EMovieSceneChannelProxyType UMovieSceneRootMotionSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	FMovieSceneChannelMetaData MetaData("RootDestination", NSLOCTEXT("AnimMixer", "RootDestinationName", "Root Destination"));
	MetaData.bCanCollapseToTrack = false;
	MetaData.WeakOwningObject = this;	// Owning object must be the section for FChannelCurveModel::Modify 

	Channels.Add(RootDestinationChannel, MetaData, TMovieSceneExternalValue<uint8>());

#else

	Channels.Add(RootDestinationChannel);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Static;
}

int32 UMovieSceneRootMotionSection::GetRowSortOrder() const
{
	// Always sort root destination sections to the top
	return MIN_int32;
}

void UMovieSceneRootMotionSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes*   BuiltInComponents   = FBuiltInComponentTypes::Get();
	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	if (Params.GetObjectBindingID().IsValid())
	{
		UMovieSceneAnimationMixerTrack* AnimTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>();

		const bool bPersistGaps = (GapBehavior == EMovieSceneRootMotionGapBehavior::PersistPreviousTransform);

		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(BuiltInComponents->GenericObjectBinding, Params.GetObjectBindingID())
			.Add(BuiltInComponents->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
			.Add(AnimMixerComponents->Target, AnimTrack->MixedAnimationTarget)
			.Add(BuiltInComponents->ByteChannel, FSourceByteChannel{ &RootDestinationChannel })
			.AddDefaulted(AnimMixerComponents->RootDestination)
			.Add(AnimMixerComponents->GapBehavior, GapBehavior)
			.Add(AnimMixerComponents->EntityOwner, FObjectKey(this))
			.Add(AnimMixerComponents->Priority, MIN_int32)
			// Add a null Task so the mixer system creates a gap sentinel entry,
			// keeping the mixer alive during gaps for root motion persistence.
			.AddConditional(AnimMixerComponents->Task, TSharedPtr<FAnimNextEvaluationTask>(), bPersistGaps)
		);
	}
}

bool UMovieSceneRootMotionSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	if (RootDestinationChannel.HasAnyData()
		|| GapBehavior == EMovieSceneRootMotionGapBehavior::PersistPreviousTransform)
	{
		const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, this, 0, MetaDataIndex);
	}

	return true;
}

FTransform UMovieSceneRootMotionSection::GetAccumulatedOffsetAt(FFrameTime Time) const
{
	FTransform Result = FTransform::Identity;

	for (const FRootMotionAccumulatedOffset& Offset : AccumulatedOffsetCache)
	{
		if (Offset.StartFrame <= Time)
		{
			Result = Offset.AccumulatedOffset;
		}
		else
		{
			break;
		}
	}

	return Result;
}

FTransform UMovieSceneRootMotionSection::GetAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const
{
	return FindAccumulatedOffsetForSection(Time, Section, false, bAllowCrossSection);
}

FTransform UMovieSceneRootMotionSection::GetAnimSpaceAccumulatedOffsetAt(FFrameTime Time) const
{
	return FindAccumulatedOffsetForSection(Time, nullptr, true, true);
}

FTransform UMovieSceneRootMotionSection::GetAnimSpaceAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const
{
	return FindAccumulatedOffsetForSection(Time, Section, true, bAllowCrossSection);
}

bool UMovieSceneRootMotionSection::HasApplicableAccumulatedEntry(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const
{
	const FFrameTime SectionStart = Section && Section->HasStartFrame()
		? FFrameTime(Section->GetInclusiveStartFrame()) : FFrameTime();

	for (const FRootMotionAccumulatedOffset& Offset : AccumulatedOffsetCache)
	{
		if (Offset.StartFrame > Time)
		{
			break;
		}
		const bool bIsOwn = !Section || Offset.OwnerSection == Section || !Offset.OwnerSection;
		const bool bIsCrossSection = bAllowCrossSection && Offset.StartFrame <= SectionStart;
		if (bIsOwn || bIsCrossSection)
		{
			return true;
		}
	}
	return false;
}

FTransform UMovieSceneRootMotionSection::FindAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAnimSpace, bool bAllowCrossSection) const
{
	FTransform Result = FTransform::Identity;

	const FFrameTime SectionStart = Section && Section->HasStartFrame()
		? FFrameTime(Section->GetInclusiveStartFrame()) : FFrameTime();

	for (const FRootMotionAccumulatedOffset& Offset : AccumulatedOffsetCache)
	{
		if (Offset.StartFrame <= Time)
		{
			const bool bIsOwn = !Section || Offset.OwnerSection == Section || !Offset.OwnerSection;
			const bool bIsCrossSection = bAllowCrossSection && Offset.StartFrame <= SectionStart;
			if (bIsOwn || bIsCrossSection)
			{
				Result = bAnimSpace ? Offset.AnimSpaceAccumulatedOffset : Offset.AccumulatedOffset;
			}
		}
		else
		{
			break;
		}
	}

	return Result;
}

void UMovieSceneRootMotionSection::RebuildAccumulatedOffsetCache(
	UMovieSceneEntitySystemLinker* Linker,
	UE::MovieScene::FInstanceHandle InstanceHandle,
	UMovieSceneAnimationMixerTrack* MixerTrack)
{
	using namespace UE::MovieScene;

	AccumulatedOffsetCache.Reset();

	UMovieSceneTrack* OwningTrack = GetTypedOuter<UMovieSceneTrack>();
	if (!OwningTrack || !Linker || !MixerTrack)
	{
		return;
	}

	// Lookups iterate AccumulatedOffsetCache and break when StartFrame exceeds
	// the query time, so the cache must stay sorted by StartFrame.
	auto InsertSortedByStartFrame = [this](const FRootMotionAccumulatedOffset& Entry)
	{
		int32 Idx = AccumulatedOffsetCache.Num();
		while (Idx > 0 && AccumulatedOffsetCache[Idx - 1].StartFrame > Entry.StartFrame)
		{
			--Idx;
		}
		AccumulatedOffsetCache.Insert(Entry, Idx);
	};

	// Pending entries are computed in sorted order so each entry's accumulated
	// offset lookup finds correct values from previously computed entries.
	struct FPendingEntry
	{
		FFrameTime EntryStartFrame;
		UMovieSceneSection* Section;
		FFrameTime EvalTime;
		// If set, the cache entry's OwnerSection is this section; the bake
		// still samples Section. Used to anchor a downstream section to an
		// upstream section's accumulated position.
		UMovieSceneSection* OwnerSectionOverride = nullptr;
		// Set for a section's own loop-boundary entries; their loop delta is
		// conjugated by the keyed offset rather than post-multiplied by it.
		bool bIsLoopBoundary = false;
	};

	TArray<FPendingEntry> PendingEntries;
	TArray<UMovieSceneAnimTransitionSectionBase*> Transitions;

	auto AddLoopBoundaryEntries = [](UMovieSceneSection* Section, TArray<FPendingEntry>& OutEntries)
	{
		UMovieSceneSkeletalAnimationSection* SkelAnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
		if (!SkelAnimSection || !Section->HasStartFrame() || !Section->HasEndFrame())
		{
			return;
		}
		const float SequenceLength = SkelAnimSection->Params.GetSequenceLength();
		if (!(SequenceLength > KINDA_SMALL_NUMBER))
		{
			return;
		}
		const UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
		if (!MovieScene)
		{
			return;
		}
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		FMovieSceneSequenceTransform Transform = SkelAnimSection->Params.MakeTransform(
			TickResolution, Section->GetRange());

		Transform.ExtractBoundariesWithinRange(
			FFrameTime(Section->GetInclusiveStartFrame()),
			FFrameTime(Section->GetExclusiveEndFrame()),
			[&OutEntries, Section, &Transform, SequenceLength](FFrameTime Boundary) -> bool
			{
				const FFrameNumber TransitionTick = ResolveLoopTransitionTick(Transform, Boundary, SequenceLength);

				FPendingEntry LoopEntry;
				LoopEntry.EntryStartFrame = FFrameTime(TransitionTick);
				LoopEntry.Section = Section;
				LoopEntry.EvalTime = FFrameTime(TransitionTick) - 1;
				LoopEntry.bIsLoopBoundary = true;
				OutEntries.Add(LoopEntry);
				return true; // continue to get all boundaries
			});
	};

	for (UMovieSceneSection* Section : OwningTrack->GetAllSections())
	{
		if (!Section)
		{
			continue;
		}

		if (auto* TransitionSection = Cast<UMovieSceneAnimTransitionSectionBase>(Section))
		{
			if (TransitionSection->IsValid())
			{
				Transitions.Add(TransitionSection);
			}
			continue;
		}

		if (!Section->HasEndFrame())
		{
			continue;
		}

		auto* Settings = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();

		if (Settings && Settings->GetKeepStateAfterSectionEnds())
		{
			FPendingEntry Entry;
			Entry.EntryStartFrame = Section->GetExclusiveEndFrame();
			Entry.Section = Section;
			Entry.EvalTime = FFrameTime(Section->GetExclusiveEndFrame() - 1);
			PendingEntries.Add(Entry);
		}

		if (SectionUsesAccumulatedLoopMode(Settings))
		{
			AddLoopBoundaryEntries(Section, PendingEntries);
		}
	}

	TSet<TPair<UMovieSceneSection*, UMovieSceneSection*>> ExplicitTransitions;

	for (UMovieSceneAnimTransitionSectionBase* Transition : Transitions)
	{
		TRange<FFrameNumber> TransitionRange = Transition->GetRange();
		if (TransitionRange.IsEmpty() || !TransitionRange.HasLowerBound() || !TransitionRange.HasUpperBound())
		{
			continue;
		}

		UMovieSceneSection* FromSection = Transition->FromSection.Get();
		UMovieSceneSection* ToSection = Transition->ToSection.Get();
		if (!FromSection || !ToSection)
		{
			continue;
		}

		ExplicitTransitions.Add({FromSection, ToSection});

		auto* FromSettings = FromSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!FromSettings || !FromSettings->GetKeepStateAfterSectionEnds())
		{
			continue;
		}

		// Only AccumulatedOffset consumers read this anchor.
		auto* ToSettings = ToSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!ToSettings || ToSettings->GetRootTransformMode() != EMovieSceneRootMotionTransformMode::AccumulatedOffset)
		{
			continue;
		}

		FFrameNumber TransitionStart = TransitionRange.GetLowerBoundValue();

		if (FromSection->HasEndFrame() && TransitionStart >= FromSection->GetExclusiveEndFrame())
		{
			continue;
		}

		FPendingEntry Entry;
		Entry.EntryStartFrame = TransitionStart;
		Entry.Section = FromSection;
		Entry.EvalTime = FFrameTime(TransitionStart);
		Entry.OwnerSectionOverride = ToSection;
		PendingEntries.Add(Entry);
	}

	// Anchor pairs of consecutive sections that overlap on the same row without
	// an explicit transition. From must be AccumulatedOffset or KeepState; To
	// must be AccumulatedOffset.
	{
		TArray<UMovieSceneSection*> SortedSections;
		for (UMovieSceneSection* Section : OwningTrack->GetAllSections())
		{
			if (!Section || Cast<UMovieSceneAnimTransitionSectionBase>(Section)
				|| !Section->HasStartFrame() || !Section->HasEndFrame())
			{
				continue;
			}
			auto* Settings = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
			if (Settings
				&& (Settings->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::AccumulatedOffset
					|| Settings->GetKeepStateAfterSectionEnds()))
			{
				SortedSections.Add(Section);
			}
		}
		SortedSections.Sort([](const UMovieSceneSection& X, const UMovieSceneSection& Y)
		{
			return X.GetInclusiveStartFrame() < Y.GetInclusiveStartFrame();
		});

		for (int32 SectionIndex = 0; SectionIndex + 1 < SortedSections.Num(); ++SectionIndex)
		{
			UMovieSceneSection* From = SortedSections[SectionIndex];
			UMovieSceneSection* To = SortedSections[SectionIndex + 1];
			if (ExplicitTransitions.Contains({From, To}))
			{
				continue;
			}
			if (From->GetRowIndex() != To->GetRowIndex())
			{
				continue;
			}
			auto* ToSettings = To->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
			if (!ToSettings
				|| ToSettings->GetRootTransformMode() != EMovieSceneRootMotionTransformMode::AccumulatedOffset)
			{
				continue;
			}
			const FFrameNumber FromEnd = From->GetExclusiveEndFrame();
			const FFrameNumber ToStart = To->GetInclusiveStartFrame();
			if (ToStart >= FromEnd)
			{
				continue;
			}

			FPendingEntry Entry;
			Entry.EntryStartFrame = ToStart;
			Entry.Section = From;
			Entry.EvalTime = FFrameTime(ToStart);
			Entry.OwnerSectionOverride = To;
			PendingEntries.Add(Entry);
		}
	}

	PendingEntries.StableSort([](const FPendingEntry& A, const FPendingEntry& B)
	{
		return A.EntryStartFrame < B.EntryStartFrame;
	});

	// Seed each section's start so its own loop boundary entries have a base
	// to chain off of, and so the runtime conversion has an anchor on frame 0.
	for (UMovieSceneSection* Section : OwningTrack->GetAllSections())
	{
		auto* SkelAnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
		if (!SkelAnimSection || !Section->HasStartFrame())
		{
			continue;
		}
		auto* Settings = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!Settings || !Settings->bInitialRootTransformValid
			|| !SectionUsesAccumulatedLoopMode(Settings))
		{
			continue;
		}
		const UMovieScene* SeedMovieScene = MixerTrack->GetTypedOuter<UMovieScene>();
		if (!SeedMovieScene)
		{
			continue;
		}
		TOptional<FFrameTime> FirstBoundary = Settings->GetFirstLoopBoundary(
			SeedMovieScene->GetTickResolution());
		if (FirstBoundary.IsSet())
		{
			FRootMotionAccumulatedOffset InitialEntry;
			InitialEntry.StartFrame = FFrameTime(Section->GetInclusiveStartFrame());
			InitialEntry.AccumulatedOffset = Settings->InitialRootTransform;
			InitialEntry.AnimSpaceAccumulatedOffset = Settings->InitialRootTransform;
			InitialEntry.Weight = 1.0f;
			InitialEntry.OwnerSection = Section;
			InsertSortedByStartFrame(InitialEntry);
		}
	}

	for (const FPendingEntry& Pending : PendingEntries)
	{
		auto* Settings = Pending.Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		const bool bUsesAccumulatedOffset = SectionUsesAccumulatedOffsetDeltaPath(Settings);

		if (bUsesAccumulatedOffset && Settings->bInitialRootTransformValid)
		{
			FRootMotionAccumulatedOffset CacheEntry;
			CacheEntry.StartFrame = Pending.EntryStartFrame;
			CacheEntry.Weight = 1.0f;
			CacheEntry.OwnerSection = Pending.OwnerSectionOverride
				? Pending.OwnerSectionOverride : Pending.Section;

			// Bake in animation space (skip conversion), then combine
			// with the accumulated offset from earlier entries.
			AnimMixerBakeEvaluation::FBakeFilter Filter;
			Filter.IncludeOnlySections.Add(FObjectKey(Pending.Section));
			Filter.bSkipRootMotionConversion = true;

			AnimMixerBakeEvaluation::FBakeResult Result = AnimMixerBakeEvaluation::EvaluateAtTime(
				Linker, InstanceHandle, MixerTrack, Pending.EvalTime, Filter);

			const UMovieScene* CacheMovieScene = MixerTrack->GetTypedOuter<UMovieScene>();
			FTransform InitialRoot = CacheMovieScene
				? Settings->GetInitialRootTransformAtTime(Pending.EvalTime, CacheMovieScene->GetTickResolution())
				: Settings->InitialRootTransform;
			FTransform Delta = Result.RootMotionTransform * InitialRoot.Inverse();

			// Cross-section and end-of-section anchors post-multiply the offset for
			// the downstream section. Loop-boundary anchors conjugate the loop delta
			// so the runtime's per-frame offset reorients the whole loop just once.
			const bool bIsEndOfSectionEntry = Pending.Section->HasEndFrame()
				&& Pending.EntryStartFrame.FrameNumber == Pending.Section->GetExclusiveEndFrame();
			const bool bShouldBakeOffset = Pending.OwnerSectionOverride || bIsEndOfSectionEntry;
			if (bShouldBakeOffset || Pending.bIsLoopBoundary)
			{
				FVector OffsetLocation = FVector::ZeroVector;
				FRotator OffsetRotation = FRotator::ZeroRotator;
				double Val = 0.0;
				if (Settings->Location[0].Evaluate(Pending.EvalTime, Val)) { OffsetLocation.X = Val; }
				if (Settings->Location[1].Evaluate(Pending.EvalTime, Val)) { OffsetLocation.Y = Val; }
				if (Settings->Location[2].Evaluate(Pending.EvalTime, Val)) { OffsetLocation.Z = Val; }
				if (Settings->Rotation[0].Evaluate(Pending.EvalTime, Val)) { OffsetRotation.Roll  = Val; }
				if (Settings->Rotation[1].Evaluate(Pending.EvalTime, Val)) { OffsetRotation.Pitch = Val; }
				if (Settings->Rotation[2].Evaluate(Pending.EvalTime, Val)) { OffsetRotation.Yaw   = Val; }

				if (!OffsetLocation.IsZero() || !OffsetRotation.IsZero())
				{
					FTransform OffsetTransform(OffsetRotation, OffsetLocation);
					FVector Pivot = Settings->RootOriginLocation;
					const FTransform PivotOffset = FTransform(-Pivot) * OffsetTransform * FTransform(Pivot);
					if (Pending.bIsLoopBoundary)
					{
						// Conjugate so the runtime's per-frame offset reorients the
						// accumulated trajectory once instead of re-applying each loop.
						Delta = PivotOffset.Inverse() * Delta * PivotOffset;
					}
					else
					{
						Delta = Delta * PivotOffset;
					}
				}
			}

			if (Pending.Section->HasStartFrame())
			{
				// Only AccumulatedOffset chains off a prior section's value.
				const bool bAllowCrossSection =
					Settings->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::AccumulatedOffset;

				FTransform Prior = GetAccumulatedOffsetForSection(Pending.EntryStartFrame, Pending.Section, bAllowCrossSection);
				FTransform AnimSpacePrior = GetAnimSpaceAccumulatedOffsetForSection(Pending.EntryStartFrame, Pending.Section, bAllowCrossSection);

				// With no prior to chain from, anchor at InitialRoot so the stored
				// value is a full anim-space position rather than a section-relative delta.
				if (Prior.Equals(FTransform::Identity))
				{
					Prior = InitialRoot;
				}
				if (AnimSpacePrior.Equals(FTransform::Identity))
				{
					AnimSpacePrior = InitialRoot;
				}

				CacheEntry.AccumulatedOffset = Delta * Prior;
				CacheEntry.AnimSpaceAccumulatedOffset = Delta * AnimSpacePrior;
			}
			else
			{
				CacheEntry.AccumulatedOffset = Delta * InitialRoot;
				CacheEntry.AnimSpaceAccumulatedOffset = Delta * InitialRoot;
			}

			// Downstream sections don't re-apply this section's BoneMatch, so
			// bake it into anchor entries they will read.
			if (bShouldBakeOffset
				&& Settings->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::BoneMatch
				&& Settings->HasBoneMatch())
			{
				const FMovieSceneBoneMatchData BoneMatch = Settings->GetBoneMatchData();
				if (BoneMatch.bIsValid)
				{
					CacheEntry.AccumulatedOffset = CacheEntry.AccumulatedOffset * BoneMatch.MatchTransform;
					CacheEntry.AnimSpaceAccumulatedOffset = CacheEntry.AnimSpaceAccumulatedOffset * BoneMatch.MatchTransform;
				}
			}

			InsertSortedByStartFrame(CacheEntry);
		}
		else
		{
			// Non-AccumulatedOffset section: full bake with world conversion.
			// AnimationSpaceRootMotion is captured by the conversion task.
			AnimMixerBakeEvaluation::FBakeFilter Filter;
			Filter.IncludeOnlySections.Add(FObjectKey(Pending.Section));

			AnimMixerBakeEvaluation::FBakeResult Result = AnimMixerBakeEvaluation::EvaluateAtTime(
				Linker, InstanceHandle, MixerTrack, Pending.EvalTime, Filter);

			FRootMotionAccumulatedOffset CacheEntry;
			CacheEntry.StartFrame = Pending.EntryStartFrame;
			CacheEntry.AccumulatedOffset = Result.RootMotionTransform;
			CacheEntry.AnimSpaceAccumulatedOffset = Result.AnimationSpaceRootMotion;
			CacheEntry.Weight = 1.0f;
			CacheEntry.OwnerSection = Pending.OwnerSectionOverride
				? Pending.OwnerSectionOverride : Pending.Section;
			InsertSortedByStartFrame(CacheEntry);
		}
	}

	bAccumulatedOffsetCacheDirty = false;
}
