// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneRootMotionTargetDecoration.h"

#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRootMotionTargetDecoration)

UMovieSceneRootMotionTargetDecoration::UMovieSceneRootMotionTargetDecoration(const FObjectInitializer& Init)
	: Super(Init)
{
	// Invalidate the accumulated-offset cache whenever this decoration (or its hosted
	// section) is modified.
	OnSignatureChanged().AddWeakLambda(this, [this]()
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = GetTypedOuter<UMovieSceneAnimationMixerTrack>())
		{
			MixerTrack->InvalidateAccumulatedOffsetCache();
		}
	});
}

void UMovieSceneRootMotionTargetDecoration::OnDecorationAdded(UMovieSceneTrack* Track)
{
	// Create the internal root motion section if it doesn't exist
	if (!RootMotionSection)
	{
		RootMotionSection = NewObject<UMovieSceneRootMotionSection>(this, NAME_None, RF_Transactional);
	}
}

void UMovieSceneRootMotionTargetDecoration::OnDecorationRemoved()
{
}

TArrayView<TObjectPtr<UMovieSceneSection>> UMovieSceneRootMotionTargetDecoration::GetSections()
{
	return MakeArrayView(&RootMotionSection, RootMotionSection ? 1 : 0);
}

TSubclassOf<UMovieSceneSection> UMovieSceneRootMotionTargetDecoration::GetHostedSectionClass() const
{
	return UMovieSceneRootMotionSection::StaticClass();
}

