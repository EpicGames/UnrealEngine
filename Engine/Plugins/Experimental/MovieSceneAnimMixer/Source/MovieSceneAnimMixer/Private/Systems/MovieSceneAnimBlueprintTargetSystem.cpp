// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimBlueprintTargetSystem.h"

#include "AnimMixerComponentTypes.h"
#include "AnimSubsystem_SequencerMixer.h"
#include "SkeletalMeshRestoreState.h"

#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Systems/MovieSceneMixedSkeletalAnimationSystem.h"

namespace UE::MovieScene
{

	/* ------- Pre-animated state for skeletal animations ------- */

	struct FPreAnimatedAnimBlueprintMixerState
	{
		FSkeletalMeshRestoreState MeshRestoreState;
	};
	
	struct FPreAnimatedAnimBlueprintMixedSkeletalAnimationTraits : FBoundObjectPreAnimatedStateTraits
	{
		using KeyType = FObjectKey;
		using StorageType = FPreAnimatedAnimBlueprintMixerState;

		FPreAnimatedAnimBlueprintMixerState CachePreAnimatedValue(const KeyType& Object)
		{
			FPreAnimatedAnimBlueprintMixerState OutCachedValue;

			USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
			if (ensure(Component))
			{
				OutCachedValue.MeshRestoreState.SaveState(Component);
			}
			
			return OutCachedValue;
		}

		void RestorePreAnimatedValue(const KeyType& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
		{
			USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
			if (!Component || !Component->IsRegistered())
			{
				return;
			}
			
			FAnimSubsystem_SequencerMixer* MixerSubsystem = Component->GetAnimInstance()->FindSubsystem<FAnimSubsystem_SequencerMixer>();

			if (MixerSubsystem)
			{
				MixerSubsystem->ResetEvalTasks();
			}

			InOutCachedValue.MeshRestoreState.RestoreLOD();

			// Restore pose after unbinding to force the restored pose
			Component->SetUpdateAnimationInEditor(true);
			Component->SetUpdateClothInEditor(true);
			if (!Component->IsPostEvaluatingAnimation())
			{
				Component->TickAnimation(0.f, false);
				Component->RefreshBoneTransforms();
				Component->RefreshFollowerComponents();
				Component->UpdateComponentToWorld();
				Component->FinalizeBoneTransform();
				Component->MarkRenderTransformDirty();
				Component->MarkRenderDynamicDataDirty();
			}

			InOutCachedValue.MeshRestoreState.RestoreState();

			// if not game world, don't clean this up
			if (Component->GetWorld() != nullptr && Component->GetWorld()->IsGameWorld() == false)
			{
				Component->ClearMotionVector();
			}
		}
	};

	
	/** Pre-animated storage **/
	struct FPreAnimatedBlueprintMixedSkeletalAnimationStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedAnimBlueprintMixedSkeletalAnimationTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedBlueprintMixedSkeletalAnimationStorage> StorageID;
	};

	TAutoRegisterPreAnimatedStorageID<FPreAnimatedBlueprintMixedSkeletalAnimationStorage> FPreAnimatedBlueprintMixedSkeletalAnimationStorage::StorageID;
	
	/* ---------------------- System task ----------------------- */

	struct FEvaluateAndApplyAnimationTasksAnimBP
	{
	private:

		UMovieSceneEntitySystemLinker* Linker;

		TSharedPtr<FPreAnimatedBlueprintMixedSkeletalAnimationStorage> PreAnimatedStorage;

	public:

		FEvaluateAndApplyAnimationTasksAnimBP(UMovieSceneEntitySystemLinker* InLinker)
			: Linker(InLinker)
		{
			PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedBlueprintMixedSkeletalAnimationStorage>();
		}

		void ForEachEntity (
			FMovieSceneEntityID EntityID,
			FObjectComponent MeshComponent,
			const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
			TSharedPtr<FAnimNextEvaluationTask> MixerTask
		) const
		{
			// Invalid or incompatible target, return.
			if (!Target.GetPtr<FMovieSceneAnimBlueprintTarget>())
			{
				return;
			}

			// Mesh may no longer be valid
			if (!IsValid(MeshComponent.GetObject()))
			{
				return;
			}

			const FName BlueprintNodeName = Target.GetPtr<FMovieSceneAnimBlueprintTarget>()->BlueprintNodeName;

			USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(MeshComponent.GetObject());
			ensureMsgf(SkeletalMeshComponent, TEXT("Attempting to apply animation to an anim instance without a valid skeletal mesh component."));

			if (!SkeletalMeshComponent || !SkeletalMeshComponent->GetSkeletalMeshAsset() || !SkeletalMeshComponent->GetAnimInstance())
			{
				return;
			}

			PreAnimatedStorage->BeginTrackingEntity(EntityID, true, FRootInstanceHandle(), SkeletalMeshComponent);
			
			PreAnimatedStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), SkeletalMeshComponent);
			
			UAnimInstance* AnimInstance = SkeletalMeshComponent->GetAnimInstance();
			
			FAnimSubsystem_SequencerMixer* MixerSubsystem = AnimInstance->FindSubsystem<FAnimSubsystem_SequencerMixer>();

			if (MixerSubsystem)
			{
				MixerSubsystem->RegisterEvalTask(BlueprintNodeName, MixerTask);

				MixerSubsystem->StagePendingNotifyBatches(BlueprintNodeName,
					UMovieSceneAnimMixerSystem::ConsumeMixerNotifyBatches(Linker, SkeletalMeshComponent, Target));
			}

			if (!SkeletalMeshComponent->IsPostEvaluatingAnimation() && SkeletalMeshComponent->PoseTickedThisFrame())
			{
				SkeletalMeshComponent->TickAnimation(0.f, false);

				SkeletalMeshComponent->RefreshBoneTransforms();
				SkeletalMeshComponent->RefreshFollowerComponents();
				SkeletalMeshComponent->UpdateComponentToWorld();
				SkeletalMeshComponent->FinalizeBoneTransform();
				SkeletalMeshComponent->MarkRenderTransformDirty();
				SkeletalMeshComponent->MarkRenderDynamicDataDirty();
			}
		}
		
	};
}

UMovieSceneAnimBlueprintTargetSystem::UMovieSceneAnimBlueprintTargetSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;
	
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->MixerTask;
	Phase = ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneAnimMixerSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
	}
}

void UMovieSceneAnimBlueprintTargetSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace  UE::MovieScene;

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Evaluate anim tasks targeting animation blueprints.
	FTaskParams Params(TEXT("Apply Animation Tasks AnimBP"));
	Params.ForceGameThread();
	FTaskID EvaluateTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(AnimMixerComponents->MeshComponent)
		.Read(AnimMixerComponents->Target)
		.Read(AnimMixerComponents->MixerTask)
		.SetParams(Params)
		.Schedule_PerEntity<FEvaluateAndApplyAnimationTasksAnimBP>(&Linker->EntityManager, TaskScheduler, Linker);
}
