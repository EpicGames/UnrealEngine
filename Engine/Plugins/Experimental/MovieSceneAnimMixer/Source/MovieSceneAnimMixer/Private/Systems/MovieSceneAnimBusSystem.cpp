// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimBusSystem.h"

#include "AnimMixerComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "Systems/MovieSceneAnimMixerSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimBusSystem)

FMovieSceneAnimBusReadTask FMovieSceneAnimBusReadTask::Make(FName InBusName, FObjectKey InBoundObjectKey, UMovieSceneAnimMixerSystem* InMixerSystem)
{
	FMovieSceneAnimBusReadTask Task;
	Task.BusName = InBusName;
	Task.BoundObjectKey = InBoundObjectKey;
	Task.MixerSystem = InMixerSystem;
	return Task;
}

void FMovieSceneAnimBusReadTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (!MixerSystem.IsValid())
	{
		FKeyframeState RefKeyframe = VM.MakeReferenceKeyframe(false);
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(RefKeyframe)));
		return;
	}

	TSharedPtr<FMovieSceneAnimBusData> BusData = MixerSystem->ReadBusData(BoundObjectKey, BusName);
	if (BusData && BusData->IsValid())
	{
		// Execute the bus-target mixer's program directly in this VM.
		// The program pushes its result onto the keyframe stack.
		BusData->Program->Execute(VM);
	}
	else
	{
		// Bus has no writer this frame -- push identity pose
		FKeyframeState RefKeyframe = VM.MakeReferenceKeyframe(false);
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(RefKeyframe)));
	}
}

namespace UE::MovieScene
{

struct FPopulateBusReadTasks
{
	UMovieSceneAnimMixerSystem* MixerSystem;

	FPopulateBusReadTasks(UMovieSceneAnimMixerSystem* InMixerSystem)
		: MixerSystem(InMixerSystem)
	{
	}

	void ForEachEntity(
		FObjectKey BoundObjectKey,
		FName BusName,
		TSharedPtr<FAnimNextEvaluationTask>& OutTask) const
	{
		if (BusName.IsNone() || !MixerSystem)
		{
			return;
		}

		OutTask = MakeShared<FMovieSceneAnimBusReadTask>(
			FMovieSceneAnimBusReadTask::Make(BusName, BoundObjectKey, MixerSystem));
	}
};

} // namespace UE::MovieScene

UMovieSceneAnimBusSystem::UMovieSceneAnimBusSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->BusName;
	Phase = ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Run before the mixer system so bus read tasks are in place
		DefineImplicitPrerequisite(GetClass(), UMovieSceneAnimMixerSystem::StaticClass());
	}
}

void UMovieSceneAnimBusSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	UMovieSceneAnimMixerSystem* MixerSystem = Linker->FindSystem<UMovieSceneAnimMixerSystem>();

	FTaskParams Params(TEXT("Populate Bus Read Tasks"));
	Params.ForceGameThread();
	FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObjectKey)
		.Read(AnimMixerComponents->BusName)
		.Write(AnimMixerComponents->Task)
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.SetParams(Params)
		.Schedule_PerEntity<FPopulateBusReadTasks>(&Linker->EntityManager, TaskScheduler, MixerSystem);
}
