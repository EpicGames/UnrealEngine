// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimInstanceTargetSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/EvaluationVM.h"
#include "GenerationTools.h"

#include "AnimCustomInstanceHelper.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalMeshRestoreState.h"

#include "Rendering/MotionVectorSimulation.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/MovieSceneObjectPropertySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"

#include "Animation/AnimationPoseData.h"
#include "BonePose.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "AnimMixerComponentTypes.h"

#include "Graph/AnimNext_LODPose.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#endif

#include "MovieSceneAnimMixerModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimInstanceTargetSystem)


USequencerMixedAnimInstance::USequencerMixedAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

FAnimInstanceProxy* USequencerMixedAnimInstance::CreateAnimInstanceProxy()
{
	return new FSequencerMixedAnimInstanceProxy(this);
}

UAnimInstance* USequencerMixedAnimInstance::GetSourceAnimInstance()
{
	return GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().GetSourceAnimInstance();
}

/** Anim Instance Source info - created externally and used here */
void USequencerMixedAnimInstance::SetSourceAnimInstance(UAnimInstance* SourceAnimInstance)
{
	USkeletalMeshComponent* MeshComponent = GetOwningComponent();
	ensure(MeshComponent->GetAnimInstance() != SourceAnimInstance);

	if (SourceAnimInstance)
	{
		// Add the current animation instance as a linked instance
		FLinkedInstancesAdapter::AddLinkedInstance(MeshComponent, SourceAnimInstance);

		// Direct the mixed anim instance to the current animation instance to evaluate as its source (input pose)
		GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().SetSourceAnimInstance(SourceAnimInstance, UAnimInstance::GetProxyOnGameThreadStatic<FAnimInstanceProxy>(SourceAnimInstance));
	}
	else
	{
		UAnimInstance* CurrentSourceAnimInstance = GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().GetSourceAnimInstance();
		// Remove the original instances from the linked instances as it should be reinstated as the main anim instance
		FLinkedInstancesAdapter::RemoveLinkedInstance(MeshComponent, CurrentSourceAnimInstance);

		// Null out the animation instance used to as the input source for the mixed anim instance
		GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().SetSourceAnimInstance(nullptr, nullptr);
	}
}

void USequencerMixedAnimInstance::SetMixerTask(TSharedPtr<FAnimNextEvaluationTask> InEvalTask)
{
	GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().SetMixerTask(InEvalTask);
}

void USequencerMixedAnimInstance::StagePendingNotifyBatches(TArray<FSequencerMixerPendingNotifyBatch>&& InBatches)
{
	GetProxyOnGameThread<FSequencerMixedAnimInstanceProxy>().StagePendingNotifyBatches(MoveTemp(InBatches));
}

void FSequencerMixedAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);
	UpdateCounter.Reset();
}

void FSequencerMixedAnimInstanceProxy::CacheBones()
{
	if (bBoneCachesInvalidated)
	{
		if (CurrentSourceProxy)
		{
			FAnimationCacheBonesContext InputContext(CurrentSourceProxy);
			if (SourcePose.GetLinkNode())
			{
				SourcePose.CacheBones(InputContext);
			}
			else
			{
				CacheBonesInputProxy(CurrentSourceProxy);
			}
		}

		bBoneCachesInvalidated = false;
	}
}

void FSequencerMixedAnimInstanceProxy::PreEvaluateAnimation(UAnimInstance* InAnimInstance)
{
	Super::PreEvaluateAnimation(InAnimInstance);

	if (CurrentSourceAnimInstance)
	{
		CurrentSourceAnimInstance->PreEvaluateAnimation();
	}
}

bool FSequencerMixedAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	using namespace UE::MovieScene;

	bool bHasValidOutput = false;
	if (CurrentSourceProxy)
	{
		FBoneContainer& SourceRequiredBones = CurrentSourceProxy->GetRequiredBones();
		if (SourceRequiredBones.IsValid())
		{
			FPoseContext InnerOutput(CurrentSourceProxy, Output.ExpectsAdditivePose());

			// if no linked node, just use Evaluate of proxy
			if (FAnimNode_Base* InputNode = SourcePose.GetLinkNode())
			{
				CurrentSourceProxy->EvaluateAnimation_WithRoot(InnerOutput, InputNode);
			}
			else if (CurrentSourceProxy->HasRootNode())
			{
				CurrentSourceProxy->EvaluateAnimationNode(InnerOutput);
			}
			else
			{
				EvaluateInputProxy(CurrentSourceProxy, InnerOutput);
			}

			Output.Pose.MoveBonesFrom(InnerOutput.Pose);
			Output.Curve.MoveFrom(InnerOutput.Curve);
			Output.CustomAttributes.MoveFrom(InnerOutput.CustomAttributes);
			bHasValidOutput = true;
		}
		else
		{
			Output.ResetToRefPose();
		}
	}
	else
	{
		Output.ResetToRefPose();
	}

	if (MixerTask)
	{
		using namespace UE::UAF;
		FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(SkeletalMeshComponent);
		FAnimNextGraphReferencePose GraphReferencePose(RefPoseHandle);

		const UE::UAF::FReferencePose& RefPose = RefPoseHandle.GetRef<UE::UAF::FReferencePose>();
		FAnimNextGraphLODPose ResultPose;
		ResultPose.LODPose = FLODPoseHeap(RefPose, LODLevel, true, Output.ExpectsAdditivePose());

		{
			FEvaluationVM EvaluationVM(EEvaluationFlags::All, RefPose, LODLevel);

			// Use the output pose in the mixer as the 'base pose'- push this pose first.
			FKeyframeState Keyframe = EvaluationVM.MakeUninitializedKeyframe(false);
			FGenerationTools::RemapPose(Output, Keyframe.Pose);
			Keyframe.Curves.CopyFrom(Output.Curve);
			// TODO: There is not a remap attributes the other way- do we need one?
			Keyframe.Attributes.CopyFrom(Output.CustomAttributes);

			EvaluationVM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));

			MixerTask->Execute(EvaluationVM);

			TUniquePtr<FKeyframeState> EvaluatedKeyframe;
			if (EvaluationVM.PopValue(KEYFRAME_STACK_NAME, EvaluatedKeyframe))
			{
				ResultPose.LODPose.CopyFrom(EvaluatedKeyframe->Pose);
				ResultPose.Curves.CopyFrom(EvaluatedKeyframe->Curves);
				ResultPose.Attributes.CopyFrom(EvaluatedKeyframe->Attributes);
				bHasValidOutput = true;
			}

			if (!bHasValidOutput)
			{
				// We need to output a valid pose, generate one
				FKeyframeState ReferenceKeyframe = EvaluationVM.MakeReferenceKeyframe(Output.ExpectsAdditivePose());
				ResultPose.LODPose.CopyFrom(ReferenceKeyframe.Pose);
				ResultPose.Curves.CopyFrom(ReferenceKeyframe.Curves);
				ResultPose.Attributes.CopyFrom(ReferenceKeyframe.Attributes);
			}
		}

		FGenerationTools::RemapPose(ResultPose.LODPose, Output);
		Output.Curve.CopyFrom(ResultPose.Curves);
		FGenerationTools::RemapAttributes(ResultPose.LODPose, ResultPose.Attributes, Output);
	}


	return true;
}

void FSequencerMixedAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();

	if (CurrentSourceProxy)
	{
		FAnimationUpdateContext SourceContext = InContext.WithOtherProxy(CurrentSourceProxy);
		if (FAnimNode_Base* SourceNode = SourcePose.GetLinkNode())
		{
			CurrentSourceProxy->UpdateAnimation_WithRoot(SourceContext, SourceNode, TEXT("AnimGraph"));
		}
		else if (CurrentSourceProxy->HasRootNode())
		{
			CurrentSourceProxy->UpdateAnimationNode(SourceContext);
		}
		else
		{
			UpdateInputProxy(CurrentSourceProxy, SourceContext);
		}
	}

	// PreUpdateAnimation has already cleared NotifyQueue and PostUpdateAnimation has not
	// yet dispatched: the only window where adds will survive to dispatch this frame.
	for (FSequencerMixerPendingNotifyBatch& Batch : PendingNotifyBatches)
	{
		NotifyQueue.AddAnimNotifies(/*bSrcIsLeader=*/true, Batch.Notifies, Batch.Weight);
	}
	PendingNotifyBatches.Reset();
}

void FSequencerMixedAnimInstanceProxy::LinkSourcePose(UAnimInstance* InSourceInstance, FAnimInstanceProxy* InSourceProxy)
{
	UnlinkSourcePose();

	if (InSourceInstance)
	{
		CurrentSourceAnimInstance = InSourceInstance;
		CurrentSourceProxy = InSourceProxy;
		SourcePose.SetLinkNode(CurrentSourceProxy->GetRootNode());

		// reset counter, so that input proxy can restart
		ResetCounterInputProxy(CurrentSourceProxy);
	}
}

void FSequencerMixedAnimInstanceProxy::UnlinkSourcePose()
{
	CurrentSourceProxy = nullptr;
	CurrentSourceAnimInstance = nullptr;
	SourcePose.SetLinkNode(nullptr);
}


void FSequencerMixedAnimInstanceProxy::SetSourceAnimInstance(UAnimInstance* SourceAnimInstance, FAnimInstanceProxy* SourceAnimInputProxy)
{
	UnlinkSourcePose();

	if (SourceAnimInstance)
	{
		LinkSourcePose(SourceAnimInstance, SourceAnimInputProxy);
	}
}

void FSequencerMixedAnimInstanceProxy::SetMixerTask(TSharedPtr<FAnimNextEvaluationTask> InEvalTask)
{
	MixerTask = InEvalTask;
}

void FSequencerMixedAnimInstanceProxy::StagePendingNotifyBatches(TArray<FSequencerMixerPendingNotifyBatch>&& InBatches)
{
	PendingNotifyBatches = MoveTemp(InBatches);
}

namespace UE::MovieScene
{
	/** ------------------------------------------------------------------------- */

	/** Pre-animated state for skeletal animations */
	struct FPreAnimatedMixedSkeletalAnimationState
	{
		EAnimationMode::Type AnimationMode;
		TWeakObjectPtr<UAnimInstance> CachedAnimInstance;
		FSkeletalMeshRestoreState SkeletalMeshRestoreState;
	};

	/** Pre-animation traits for skeletal animations */
	struct FPreAnimatedAnimInstanceMixedSkeletalAnimationTraits : FBoundObjectPreAnimatedStateTraits
	{
		using KeyType = FObjectKey;
		using StorageType = FPreAnimatedMixedSkeletalAnimationState;

		FPreAnimatedMixedSkeletalAnimationState CachePreAnimatedValue(const KeyType& Object)
		{
			FPreAnimatedMixedSkeletalAnimationState OutCachedValue;
			USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
			if (ensure(Component))
			{
				OutCachedValue.AnimationMode = Component->GetAnimationMode();
				OutCachedValue.CachedAnimInstance = Component->AnimScriptInstance;
				OutCachedValue.SkeletalMeshRestoreState.SaveState(Component);
			}
			return OutCachedValue;
		}

		void RestorePreAnimatedValue(const KeyType& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
		{
			USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
			if (!Component || !Component->IsRegistered())
			{
				return;
			}

			FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<USequencerMixedAnimInstance>(Component);

			// Restore LOD before reinitializing anim instance
			InOutCachedValue.SkeletalMeshRestoreState.RestoreLOD();

			if (Component->GetAnimationMode() != InOutCachedValue.AnimationMode)
			{
				// this SetAnimationMode reinitializes even if the mode is same
				// if we're using same anim blueprint, we don't want to keep reinitializing it. 
				Component->SetAnimationMode(InOutCachedValue.AnimationMode);
			}

			TStrongObjectPtr<UAnimInstance> PreviousAnimInstance = InOutCachedValue.CachedAnimInstance.Pin();
			if (PreviousAnimInstance && IsValid(PreviousAnimInstance.Get()) && PreviousAnimInstance->GetSkelMeshComponent() == Component)
			{
				bool bNeedsReInit = false;

				if (Component->AnimScriptInstance != PreviousAnimInstance.Get())
				{
					Component->AnimScriptInstance = PreviousAnimInstance.Get();
					bNeedsReInit = true;
				}

				InOutCachedValue.CachedAnimInstance.Reset();

				if (Component->AnimScriptInstance)
				{
					//the skeleton may have changed so need to recalc required bones as needed.
					if (Component->GetSkeletalMeshAsset() && Component->AnimScriptInstance->CurrentSkeleton != Component->GetSkeletalMeshAsset()->GetSkeleton())
					{
						Component->AnimScriptInstance->CurrentSkeleton = Component->GetSkeletalMeshAsset()->GetSkeleton();
						bNeedsReInit = true;
					}
				}

				if (bNeedsReInit)
				{
					// When Changing anim instances or skeletons, RecalcRequiredBones and UpdateMorphTargets need to be called, and InitializeAnimation needs to be called regardless, when changing script instances
					// Playing montages does not work on uninitialized anim script instances with the EarlyOutMontageWhenUninitialized cvar enabled.
					Component->InitializeAnimScriptInstance(true);
				}
			}

			// Restore pose after unbinding to force the restored pose
			Component->SetUpdateAnimationInEditor(true);
			Component->SetUpdateClothInEditor(true);
			if (!Component->IsPostEvaluatingAnimation())
			{
				Component->TickAnimation(0.f, false);
				Component->RefreshBoneTransforms();
				Component->RefreshFollowerComponents();
				Component->UpdateComponentToWorld();
				Component->FinalizeBoneTransform();
				Component->MarkRenderTransformDirty();
				Component->MarkRenderDynamicDataDirty();
			}

			// Reset the mesh component update flag and animation mode to what they were before we animated the object
			InOutCachedValue.SkeletalMeshRestoreState.RestoreState();

			// if not game world, don't clean this up
			if (Component->GetWorld() != nullptr && Component->GetWorld()->IsGameWorld() == false)
			{
				Component->ClearMotionVector();
			}
		}
	};

	/** Pre-animation storage for skeletal animations */
	struct FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedAnimInstanceMixedSkeletalAnimationTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage> StorageID;
	};

	TAutoRegisterPreAnimatedStorageID<FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage> FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage::StorageID;

	/** Task for evaluating and applying animation tasks */
	struct FEvaluateAndApplyAnimationTasks
	{
	private:

		UMovieSceneEntitySystemLinker* Linker;

		UMovieSceneAnimInstanceTargetSystem* System;

		TSharedPtr<FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage> PreAnimatedStorage;

	public:

		FEvaluateAndApplyAnimationTasks(UMovieSceneEntitySystemLinker* InLinker, UMovieSceneAnimInstanceTargetSystem* InSystem)
			: Linker(InLinker)
			, System(InSystem)
		{
			PreAnimatedStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedAnimInstanceMixedSkeletalAnimationStorage>();
		}

		void ForEachEntity
		(
			FMovieSceneEntityID EntityID,
			FObjectComponent MeshComponent,
			TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
			const TSharedPtr<FAnimNextEvaluationTask>& MixerTask
		) const
		{
			// Invalid or incompatible target, return.
			FMovieSceneAnimInstanceTarget* AnimInstanceTarget = Target.GetMutablePtr<FMovieSceneAnimInstanceTarget>(); 
			if (!AnimInstanceTarget)
			{
				return;
			}

			USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent.GetObject());

			if (!SkeletalMeshComponent || !SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				return;
			}

			// Cache pre-animated state for this bound object before doing anything.
			// We don't yet track what entities have already started animated vs. entities that just started this frame,
			// so we just process all the currently active ones. If they are already tracked and have already had their
			// pre-animated state saved, it these calls will just early return.

			// For now, we always restore state
			PreAnimatedStorage->BeginTrackingEntity(EntityID, true, FRootInstanceHandle(), SkeletalMeshComponent);
			
			FCachePreAnimatedValueParams CacheParams;
			PreAnimatedStorage->CachePreAnimatedValue(CacheParams, SkeletalMeshComponent);


			// Setup custom anim instance, using the current anim instance as a 'source'.
			bool bWasCreated = false;
			
#if WITH_EDITOR
			TWeakObjectPtr CurrentAnimInstanceWeakPtr (SkeletalMeshComponent->GetAnimInstance());
			TWeakObjectPtr SkeletalMeshComponentWeakPtr (SkeletalMeshComponent);
#endif
			// By using the anim instance target, we are implicitly using AnimationCustomMode. We set it here explicitly to ensure the first frame initializes properly.
			if (SkeletalMeshComponent->GetAnimationMode() != EAnimationMode::AnimationCustomMode)
			{
				SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
			}

			USequencerMixedAnimInstance* SequencerInstance = FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<USequencerMixedAnimInstance>(SkeletalMeshComponent, bWasCreated);

			if (SequencerInstance)
			{
				SequencerInstance->SetMixerTask(MixerTask);

				SequencerInstance->StagePendingNotifyBatches(
					UMovieSceneAnimMixerSystem::ConsumeMixerNotifyBatches(Linker, SkeletalMeshComponent, Target));
			}
			else if (!AnimInstanceTarget->HasFiredWarningForTarget())
			{
				// Control rig also uses FAnimCustomInstanceHelper::BindToSkeletalMeshComponent to create an anim instance that it can use.
				// If there is an existing animation blueprint specified on the skeletal mesh, that can cause this to fail when both the anim mixer and control rig try to bind to it.
				UE_LOGF(LogMovieSceneAnimMixer, Warning,
				       "Unable to bind anim mixer custom instance to skeletal mesh: %ls on actor: %ls. This is usually caused by a conflict with a control rig track. Currently it is unsupported to use the Sequenccer Anim Mixer with a Custom Anim Instance target and control rig tracks, if the actor has an animation blueprint assigned. Please remove one to resolve this issue.", *SkeletalMeshComponent->GetName(), *SkeletalMeshComponent->GetOwner()->GetName())
				
				// Don't print to the log every frame.
				AnimInstanceTarget->SetHasFiredWarningForTarget(true);
			}

#if WITH_EDITOR
			if (GEditor && bWasCreated)
			{
				FDelegateHandle PreCompileHandle = GEditor->OnBlueprintPreCompile().AddLambda([CurrentAnimInstanceWeakPtr, SkeletalMeshComponentWeakPtr](const UBlueprint* InBlueprint)
				{
					if (CurrentAnimInstanceWeakPtr.IsValid() && SkeletalMeshComponentWeakPtr.IsValid() && InBlueprint && CurrentAnimInstanceWeakPtr->GetClass() == InBlueprint->GeneratedClass)
					{
						FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<USequencerMixedAnimInstance>(SkeletalMeshComponentWeakPtr.Get());
					}
				});
			
				System->PreCompileHandles.Add(PreCompileHandle);
			}
#endif
			

			// TODO: Figure out- can we do motion vector sim here with the blended anim? Or do we need to do something different
			/*if (InSkeletalAnimations.SimulatedAnimations.Num() != 0)
			{
				UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = Linker->FindSystem<UMovieSceneMotionVectorSimulationSystem>();
				if (MotionVectorSim && MotionVectorSim->IsSimulationEnabled())
				{
					ApplyAnimations(SkeletalMeshComponent, InSkeletalAnimations.SimulatedAnimations);
					SkeletalMeshComponent->TickAnimation(0.f, false);
					SkeletalMeshComponent->ForceMotionVector();

					SimulateMotionVectors(SkeletalMeshComponent, MotionVectorSim);
				}
			}*/

			if (!SkeletalMeshComponent->IsPostEvaluatingAnimation() && SkeletalMeshComponent->PoseTickedThisFrame())
			{
				SkeletalMeshComponent->TickAnimation(0.f, false);

				SkeletalMeshComponent->RefreshBoneTransforms();
				SkeletalMeshComponent->RefreshFollowerComponents();
				SkeletalMeshComponent->UpdateComponentToWorld();
				SkeletalMeshComponent->FinalizeBoneTransform();
				SkeletalMeshComponent->MarkRenderTransformDirty();
				SkeletalMeshComponent->MarkRenderDynamicDataDirty();
			}
		}
	};

} // namespace UE::MovieScene

UMovieSceneAnimInstanceTargetSystem::UMovieSceneAnimInstanceTargetSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->MixerTask;
	Phase = ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneAnimMixerSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
	}
}

#if WITH_EDITOR
UMovieSceneAnimInstanceTargetSystem::~UMovieSceneAnimInstanceTargetSystem()
{
	if (GEditor)
	{
		for (FDelegateHandle Handle : PreCompileHandles)
		{
			GEditor->OnBlueprintPreCompile().Remove(Handle);
		}
	}
}
#endif

void UMovieSceneAnimInstanceTargetSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Evaluate anim tasks targeting a custom anim instance.
	FTaskParams Params(TEXT("Apply Animation Tasks"));
	Params.ForceGameThread();
	FTaskID EvaluateTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(AnimMixerComponents->MeshComponent)
		.Write(AnimMixerComponents->Target)
		.Read(AnimMixerComponents->MixerTask)
		.SetParams(Params)
		.Schedule_PerEntity<FEvaluateAndApplyAnimationTasks>(&Linker->EntityManager, TaskScheduler,
			Linker, this);
}
