// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimMixerMaskSystem.h"

#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimationMaskDecoration.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EvaluationVM/Tasks/BlendKeyframesPerBone.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerMaskSystem)

// -----------------------------------------------------------------------
// FCachedMaskAssetData
// -----------------------------------------------------------------------

void FCachedMaskAssetData::CacheFromAsset(UUAFBlendMask* InBlendMask)
{
	if (!InBlendMask)
	{
		DampingValues.Reset();
		AttributeDampingValues.Reset();
		CurveDampingValues.Reset();
		BlendMask = nullptr;
		return;
	}

	BlendMask = InBlendMask;

	const TArray<float>& AssetBoneWeights = InBlendMask->GetSkeletonBoneWeights();
	DampingValues.SetNumUninitialized(AssetBoneWeights.Num());
	for (int32 Index = 0; Index < AssetBoneWeights.Num(); ++Index)
	{
		DampingValues[Index] = 1.0f - AssetBoneWeights[Index];
	}

	const TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight> AssetAttributeWeights = InBlendMask->GetSkeletonAttributeWeights();
	AttributeDampingValues.Reset(AssetAttributeWeights.Num());
	for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttrWeight : AssetAttributeWeights)
	{
		AttributeDampingValues.Add(FBlendProfileStandaloneCachedData::FMaskedAttributeWeight(AttrWeight.Attribute, 1.0f - AttrWeight.Weight));
	}

	CurveDampingValues.Reset();
	auto GatherCurveDamping = [this](UE::Anim::FCurveElement CurveElement)
	{
		CurveDampingValues.Add(CurveElement.Name, 1.0f - CurveElement.Value);
	};
	InBlendMask->GetSkeletonCurveWeights().ForEachElement(GatherCurveDamping);
}

bool FCachedMaskAssetData::IsStale(const UUAFBlendMask* InBlendMask) const
{
	return BlendMask.Get() != InBlendMask || !BlendMask.IsValid();
}

// -----------------------------------------------------------------------
// FMaskLayerAccumulation
// -----------------------------------------------------------------------

void FMaskLayerAccumulation::ResetAccumulation()
{
	AccumulationBoneWeights.Reset();
	CurveWeightsMap.Reset();
	AttributeWeightsMap.Reset();
	MaskSkeleton = nullptr;
	NumContributingSections = 0;
	SingleContributingMask = nullptr;
	bSingleSectionFullWeight = false;
}

void FMaskLayerAccumulation::AccumulateMask(UUAFBlendMask* BlendMask, double EasingWeight)
{
	if (!BlendMask)
	{
		return;
	}

	if (!MaskSkeleton.IsValid())
	{
		MaskSkeleton = BlendMask->GetSkeleton();
	}

	// Get or create cached data for this mask asset
	TObjectKey<UUAFBlendMask> AssetKey(BlendMask);
	FCachedMaskAssetData& CachedData = CachedAssets.FindOrAdd(AssetKey);
	if (CachedData.IsStale(BlendMask))
	{
		CachedData.CacheFromAsset(BlendMask);
	}

	++NumContributingSections;

	// Fast path: single section at full weight - can reference asset weights directly
	if (NumContributingSections == 1 && FMath::IsNearlyEqual(EasingWeight, 1.0, KINDA_SMALL_NUMBER))
	{
		bSingleSectionFullWeight = true;
		SingleContributingMask = BlendMask;
	}
	else
	{
		bSingleSectionFullWeight = false;
		SingleContributingMask = nullptr;
	}

	// Accumulate bone damping values weighted by easing
	if (CachedData.DampingValues.Num() > AccumulationBoneWeights.Num())
	{
		AccumulationBoneWeights.AddDefaulted(CachedData.DampingValues.Num() - AccumulationBoneWeights.Num());
	}

	for (int32 BoneIndex = 0; BoneIndex < CachedData.DampingValues.Num(); ++BoneIndex)
	{
		const float EasingAdjustedWeight = static_cast<float>(EasingWeight) * CachedData.DampingValues[BoneIndex];
		AccumulationBoneWeights[BoneIndex] = FMath::Clamp(AccumulationBoneWeights[BoneIndex] + EasingAdjustedWeight, 0.0f, 1.0f);
	}

	// Accumulate attribute weights
	for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttrDamping : CachedData.AttributeDampingValues)
	{
		const float EasingAdjustedWeight = static_cast<float>(EasingWeight) * AttrDamping.Weight;
		if (float* CurrentWeight = AttributeWeightsMap.Find(AttrDamping.Attribute))
		{
			*CurrentWeight = FMath::Clamp(*CurrentWeight + EasingAdjustedWeight, 0.0f, 1.0f);
		}
		else
		{
			AttributeWeightsMap.Add(AttrDamping.Attribute, FMath::Clamp(EasingAdjustedWeight, 0.0f, 1.0f));
		}
	}

	// Accumulate curve weights
	for (const TTuple<FName, float>& CurveDamping : CachedData.CurveDampingValues)
	{
		const float EasingAdjustedValue = static_cast<float>(EasingWeight) * CurveDamping.Value;
		if (float* CurrentValue = CurveWeightsMap.Find(CurveDamping.Key))
		{
			*CurrentValue = FMath::Clamp(*CurrentValue + EasingAdjustedValue, 0.0f, 1.0f);
		}
		else
		{
			CurveWeightsMap.Add(CurveDamping.Key, FMath::Clamp(EasingAdjustedValue, 0.0f, 1.0f));
		}
	}
}

TSharedPtr<FAnimNextEvaluationTask> FMaskLayerAccumulation::BuildBlendTask(float LayerWeight)
{
	if (AccumulationBoneWeights.IsEmpty())
	{
		return nullptr;
	}

	// Fast path: single section at full weight and layer weight is 1.0 -
	// reference the asset's bone weight arrays directly
	if (bSingleSectionFullWeight && FMath::IsNearlyEqual(LayerWeight, 1.0f, KINDA_SMALL_NUMBER) && SingleContributingMask.IsValid())
	{
		return MakeShared<FAnimNextBlendKeyframePerBoneWithScaleTask>(
			FAnimNextBlendKeyframePerBoneWithScaleTask::Make(SingleContributingMask.Get(), 1.0f));
	}

	// Slow path: build task from accumulated damping values.
	TaskCurveWeights.Empty();
	TaskAttributeWeights.Reset();
	TaskBoneWeights.Reset();

	for (int32 WeightIndex = 0; WeightIndex < AccumulationBoneWeights.Num(); ++WeightIndex)
	{
		TaskBoneWeights.Add(1.0f - AccumulationBoneWeights[WeightIndex]);
	}

	for (const TTuple<UE::Anim::FAttributeId, float>& CurrentAttributeWeight : AttributeWeightsMap)
	{
		TaskAttributeWeights.Add(FBlendProfileStandaloneCachedData::FMaskedAttributeWeight(CurrentAttributeWeight.Key, 1.0f - CurrentAttributeWeight.Value));
	}

	for (const TTuple<FName, float>& CurrentCurveWeight : CurveWeightsMap)
	{
		TaskCurveWeights.Add(UE::Anim::FCurveElement(CurrentCurveWeight.Key, 1.0f - CurrentCurveWeight.Value));
	}

	return MakeShared<FAnimNextBlendKeyframePerBoneWithScaleTask>(
		FAnimNextBlendKeyframePerBoneWithScaleTask::Make(MaskSkeleton.Get(), &TaskBoneWeights,
			&TaskCurveWeights, &TaskAttributeWeights, LayerWeight));
}

// -----------------------------------------------------------------------
// UMovieSceneAnimMixerMaskSystem
// -----------------------------------------------------------------------

namespace UE::MovieScene
{

struct FGatherMaskData
{
	TMap<FMaskLayerKey, FMaskLayerAccumulation>* Accumulations;
	UMovieSceneEntitySystemLinker* Linker;

	FGatherMaskData(TMap<FMaskLayerKey, FMaskLayerAccumulation>* InAccumulations, UMovieSceneEntitySystemLinker* InLinker)
		: Accumulations(InAccumulations)
		, Linker(InLinker)
	{
	}

	void PreTask()
	{
		// Reset all accumulations before gathering
		for (auto& [Key, Accum] : *Accumulations)
		{
			Accum.ResetAccumulation();
		}
	}

	void ForEachEntity(
		FObjectKey BoundObjectKey,
		TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer,
		const TInstancedStruct<FMovieSceneAnimMixerBlendProviderData>& BlendData,
		const double* WeightAndEasing) const
	{
		if (!MixerLayer)
		{
			return;
		}

		const FMovieSceneMaskBlendProviderData* MaskData = BlendData.GetPtr<FMovieSceneMaskBlendProviderData>();
		if (!MaskData || !MaskData->BlendMask.IsValid())
		{
			return;
		}

		const double Weight = WeightAndEasing ? *WeightAndEasing : 1.0;
		FMaskLayerKey Key{BoundObjectKey, MixerLayer};
		FMaskLayerAccumulation& Accum = Accumulations->FindOrAdd(Key);
		Accum.AccumulateMask(MaskData->BlendMask.Get(), Weight);
	}

	void PostTask()
	{
		FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
		for (auto& [Key, Accum] : *Accumulations)
		{
			if (!Accum.OutputEntityID)
			{
				continue;
			}

			FLayerBlendTaskBuilder Builder;

			// AccumulationBoneWeights stores damping (1 - MaskWeight); FLayerBlendTaskBuilder
			// wants the max passthrough across all bones for the notify exposure pass.
			float MaxBoneWeight = 0.0f;
			for (float Damping : Accum.AccumulationBoneWeights)
			{
				MaxBoneWeight = FMath::Max(MaxBoneWeight, 1.0f - Damping);
			}
			Builder.MaskMaxBoneWeight = MaxBoneWeight;

			if (Accum.NumContributingSections > 0)
			{
				// Build the task with ScaleFactor=1.0. The builder reconstructs it
				// each frame with the actual layer weight, pre-multiplying per-bone,
				// per-curve and per-attribute weights so the blend correctly scales
				// the mask contribution.
				TSharedPtr<FAnimNextBlendKeyframePerBoneWithScaleTask> Task =
					StaticCastSharedPtr<FAnimNextBlendKeyframePerBoneWithScaleTask>(Accum.BuildBlendTask(1.0f));
				if (Task)
				{
					auto* AccumulationsMap = Accumulations;
					FMaskLayerKey AccumKey = Key;
					Builder.BuildTask = [Task, AccumulationsMap, AccumKey](double LayerWeight) -> TSharedPtr<FAnimNextEvaluationTask>
					{
						FMaskLayerAccumulation* AccumData = AccumulationsMap->Find(AccumKey);
						if (!AccumData)
						{
							return Task;
						}

						const float Scale = static_cast<float>(LayerWeight);

						AccumData->TaskBoneWeights.Reset();
						AccumData->TaskCurveWeights.Empty();
						AccumData->TaskAttributeWeights.Reset();

						for (int32 i = 0; i < AccumData->AccumulationBoneWeights.Num(); ++i)
						{
							AccumData->TaskBoneWeights.Add((1.0f - AccumData->AccumulationBoneWeights[i]) * Scale);
						}

						for (const TTuple<FName, float>& CurveWeight : AccumData->CurveWeightsMap)
						{
							AccumData->TaskCurveWeights.Add(UE::Anim::FCurveElement(CurveWeight.Key, (1.0f - CurveWeight.Value) * Scale));
						}

						for (const TTuple<UE::Anim::FAttributeId, float>& AttrWeight : AccumData->AttributeWeightsMap)
						{
							AccumData->TaskAttributeWeights.Add(FBlendProfileStandaloneCachedData::FMaskedAttributeWeight(
								AttrWeight.Key, (1.0f - AttrWeight.Value) * Scale));
						}

						*Task = FAnimNextBlendKeyframePerBoneWithScaleTask::Make(
							AccumData->MaskSkeleton.Get(), &AccumData->TaskBoneWeights,
							&AccumData->TaskCurveWeights, &AccumData->TaskAttributeWeights, Scale);

						return Task;
					};
				}
			}

			Linker->EntityManager.WriteComponentChecked(
				Accum.OutputEntityID, AnimMixerComponents->LayerBlendTask, MoveTemp(Builder));
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneAnimMixerMaskSystem::UMovieSceneAnimMixerMaskSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->BlendProviderData;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBoundSceneComponentInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneAnimMixerSystem::StaticClass());
		DefineComponentConsumer(GetClass(), AnimMixerComponents->BlendProviderData);
		DefineComponentConsumer(GetClass(), AnimMixerComponents->MixerLayer);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObjectKey);
		DefineComponentProducer(GetClass(), AnimMixerComponents->LayerBlendTask);
	}
}

void UMovieSceneAnimMixerMaskSystem::OnLink()
{
}

void UMovieSceneAnimMixerMaskSystem::UpdateAccumulationLifetimes()
{
	using namespace UE::MovieScene;
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Count active (non-NeedsUnlink) mask section entities per key.
	// This is idempotent and safe to call multiple times per frame.
	TSet<FMaskLayerKey> ActiveKeys;

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.Read(AnimMixerComponents->MixerLayer)
	.FilterAll({ AnimMixerComponents->BlendProviderData })
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
	.Iterate_PerEntity(&Linker->EntityManager,
		[&ActiveKeys](FObjectKey BoundObjectKey, TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer)
		{
			if (MixerLayer)
			{
				ActiveKeys.Add(FMaskLayerKey{BoundObjectKey, MixerLayer});
			}
		}
	);

	// Remove accumulations whose keys have no active entities
	for (auto It = Accumulations.CreateIterator(); It; ++It)
	{
		if (!ActiveKeys.Contains(It.Key()) && It.Value().OutputEntityID)
		{
			Linker->EntityManager.AddComponent(It.Value().OutputEntityID, BuiltInComponents->Tags.NeedsUnlink);
			It.RemoveCurrent();
		}
	}
}

bool UMovieSceneAnimMixerMaskSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	return InLinker->EntityManager.ContainsAnyComponent({AnimMixerComponents->BlendProviderData});
}

void UMovieSceneAnimMixerMaskSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	if (Runner->GetCurrentPhase() != ESystemPhase::Instantiation)
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Remove output entities for keys that no longer have active mask section entities.
	UpdateAccumulationLifetimes();

	// Create output entities for newly linked mask section entities.
	// Collected first, then created outside the iteration to avoid locking issues.
	struct FPendingOutputEntity
	{
		FMaskLayerKey Key;
		FObjectKey BoundObjectKey;
		TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer;
	};
	TArray<FPendingOutputEntity> PendingOutputEntities;

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.Read(AnimMixerComponents->MixerLayer)
	.Read(AnimMixerComponents->BlendProviderData)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.Iterate_PerEntity(&Linker->EntityManager,
		[this, &PendingOutputEntities](FObjectKey BoundObjectKey, TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer, const TInstancedStruct<FMovieSceneAnimMixerBlendProviderData>& BlendData)
		{
			if (!MixerLayer)
			{
				return;
			}

			FMaskLayerKey Key{BoundObjectKey, MixerLayer};
			if (!Accumulations.Contains(Key))
			{
				PendingOutputEntities.Add({ Key, BoundObjectKey, MixerLayer });
			}
		}
	);

	for (const FPendingOutputEntity& Pending : PendingOutputEntities)
	{
		FMaskLayerAccumulation& Accum = Accumulations.FindOrAdd(Pending.Key);
		if (!Accum.OutputEntityID)
		{
			Accum.OutputEntityID = FEntityBuilder()
				.Add(BuiltInComponents->BoundObjectKey, Pending.BoundObjectKey)
				.Add(AnimMixerComponents->MixerLayer, Pending.MixerLayer)
				.Add(AnimMixerComponents->LayerBlendTask, FLayerBlendTaskBuilder())
				.CreateEntity(&Linker->EntityManager);
		}
	}
}

void UMovieSceneAnimMixerMaskSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.Read(AnimMixerComponents->MixerLayer)
	.Read(AnimMixerComponents->BlendProviderData)
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterNone({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.Ignored })
	.Schedule_PerEntity<FGatherMaskData>(&Linker->EntityManager, TaskScheduler, &Accumulations, Linker);
}

void UMovieSceneAnimMixerMaskSystem::OnCleanTaggedGarbage()
{
	UpdateAccumulationLifetimes();
}
