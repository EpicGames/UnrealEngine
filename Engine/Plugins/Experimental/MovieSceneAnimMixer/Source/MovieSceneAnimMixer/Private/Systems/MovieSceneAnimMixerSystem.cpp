// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimMixerSystem.h"

#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneRootMotionSection.h"
#include "MovieSceneRootMotionTargetDecoration.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Systems/ByteChannelEvaluatorSystem.h"
#include "Systems/MovieSceneMixerBakeTargetSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "EvaluationVM/Tasks/ExecuteProgram.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/Tasks/ApplyAdditiveKeyframe.h"
#include "EvaluationVM/Tasks/NormalizeRotations.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Algo/ForEach.h"
#include "Algo/MaxElement.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "UObject/Object.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Actor.h"
#include "Component/AnimNextComponent.h"
#include "Systems/MovieSceneAnimInstanceTargetSystem.h"
#include "Systems/MovieSceneAnimNextTargetSystem.h"
#include "Systems/MovieSceneAnimBlueprintTargetSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "AnimSubsystem_SequencerMixer.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Systems/MovieSceneMixedSkeletalAnimationSystem.h"
#include "MirroringTask.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "EvaluationVM/Tasks/TransitionEvaluationTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimBusSection.h"
#include "AnimMixerBusUtils.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerSystem)

namespace UE::MovieScene
{

// Filter for controlling which mixer entries are included in program building.
struct FProgramBuildFilter
{
	// Entries whose EntityOwner matches are skipped
	const TSet<FObjectKey>* ExcludeSections = nullptr;

	// If non-null, only entries whose EntityOwner matches are included
	const TSet<FObjectKey>* IncludeOnly = nullptr;

	// If >= 0, only entries with priority >= MinPriority are included
	int32 MinPriority = -1;

	// If >= 0, only entries with priority <= MaxPriority are included
	int32 MaxPriority = -1;

	// When true, skip ConvertRootMotionToWorldSpaceTask - result is in animation space
	bool bSkipRootMotionConversion = false;

	// When set, the conversion task writes the pre-conversion animation-space root here
	TSharedPtr<FTransform> CaptureAnimSpaceRoot;

	// When true, the first layer's inter-layer blend is skipped. The program
	// produces a self-contained keyframe without consuming a base from the
	// stack. Used for bus-target mixers whose programs are embedded inside
	// a consuming mixer's entry evaluation.
	bool bSkipFirstInterLayerBlend = false;
};

// Context needed for building evaluation programs.
struct FProgramBuildContext
{
	TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion;
	const UMovieSceneRootMotionSystem* RootMotionSystem = nullptr;
	const UMovieSceneTransformOriginSystem* TransformOriginSystem = nullptr;
	UMovieSceneEntitySystemLinker* Linker = nullptr;
	USceneComponent* Component = nullptr;
	// Layer blend task builders, keyed by MixerLayer for this specific mixer.
	// Built from entities with {BoundObjectKey, MixerLayer, LayerBlendTask} matching this mixer's BoundObjectKey.
	const TMap<TWeakObjectPtr<UMovieSceneAnimationMixerLayer>, FLayerBlendTaskBuilder>* LayerBlendTaskMap = nullptr;
	// Decoration-provided layer weights. Multiplied into LayerWeight before inter-layer blending.
	const TMap<TWeakObjectPtr<UMovieSceneAnimationMixerLayer>, double>* DecorationLayerWeightMap = nullptr;
};

// Returns true if the entry passes the filter (i.e. should be considered for inclusion)
static bool PassesFilter(const FMovieSceneAnimMixerEntry& Entry, const FProgramBuildFilter& Filter)
{
	if (Filter.ExcludeSections && Filter.ExcludeSections->Contains(Entry.EntityOwner))
	{
		return false;
	}
	if (Filter.IncludeOnly && !Filter.IncludeOnly->Contains(Entry.EntityOwner))
	{
		return false;
	}
	if (Filter.MinPriority >= 0 && Entry.Priority < Filter.MinPriority)
	{
		return false;
	}
	if (Filter.MaxPriority >= 0 && Entry.Priority > Filter.MaxPriority)
	{
		return false;
	}
	return true;
}

// Builds the ConvertRootMotionToWorldSpace task for a single mixer entry
// based on its root motion settings. Returns an empty optional if no
// conversion is needed.
static TOptional<FAnimNextConvertRootMotionToWorldSpaceTask> BuildSpaceConversionTask(
	const FMovieSceneAnimMixerEntry& MixerEntry,
	const FProgramBuildContext& Context,
	bool bRootComponentHasKeyedTransform)
{
	using ESpaceConversions = FAnimNextConvertRootMotionToWorldSpaceTask::ESpaceConversions;

	FTransform TransformOrigin  = FTransform::Identity;
	FTransform RootTransform    = FTransform::Identity;
	FVector    RootOffsetOrigin = FVector(ForceInitToZero);

	ESpaceConversions Conversion = ESpaceConversions::None;
	EMovieSceneRootMotionSpace ThisRootSpace = EMovieSceneRootMotionSpace::AnimationSpace;

	bool bUseAccumulatedOffset = false;
	FTransform ResolvedAnimSpaceAccumulatedTransform = FTransform::Identity;
	FTransform ResolvedInitialRootTransform = FTransform::Identity;

	if (MixerEntry.RootMotionSettings)
	{
		ThisRootSpace = MixerEntry.RootMotionSettings->RootMotionSpace;

		const bool bHasLocation = !MixerEntry.RootMotionSettings->RootLocation.IsZero();
		const bool bHasRotation = !MixerEntry.RootMotionSettings->RootRotation.IsIdentity();

		const EMovieSceneRootMotionTransformMode Mode = MixerEntry.RootMotionSettings->TransformMode;

		// Per-frame offsets driven by TransformMode.
		switch (Mode)
		{
			case EMovieSceneRootMotionTransformMode::Override:
				Conversion |= ESpaceConversions::RootTransformOverride;
				RootTransform = FTransform(
					MixerEntry.RootMotionSettings->RootRotation,
					MixerEntry.RootMotionSettings->RootLocation);
				break;

			case EMovieSceneRootMotionTransformMode::Offset:
			case EMovieSceneRootMotionTransformMode::AccumulatedOffset:
			case EMovieSceneRootMotionTransformMode::BoneMatch:
				if (bHasLocation || bHasRotation)
				{
					Conversion |= ESpaceConversions::RootTransformOffset;
					RootOffsetOrigin = MixerEntry.RootMotionSettings->RootOriginLocation;
					RootTransform = FTransform(
						MixerEntry.RootMotionSettings->RootRotation,
						MixerEntry.RootMotionSettings->RootLocation);
				}
				if (Mode == EMovieSceneRootMotionTransformMode::BoneMatch
					&& MixerEntry.BoneMatchTransform.IsSet())
				{
					Conversion |= ESpaceConversions::BoneMatchOffset;
				}
				break;

			case EMovieSceneRootMotionTransformMode::Asset:
			default:
				break;
		}

		// A section only uses the cache when it asked to. Otherwise a prior
		// section's entry would silently anchor this one.
		const bool bWantsCrossSectionAccumulation =
			Mode == EMovieSceneRootMotionTransformMode::AccumulatedOffset;
		const bool bWantsLoopAccumulation =
			MixerEntry.RootMotionSettings->LoopMode == EMovieSceneAnimLoopMode::Accumulated
			&& Mode != EMovieSceneRootMotionTransformMode::Override;

		UMovieSceneSection* OwnerSection = Cast<UMovieSceneSection>(MixerEntry.EntityOwner.ResolveObjectPtr());

		if (OwnerSection && Context.Linker && MixerEntry.InstanceHandle.IsValid())
		{
			if (auto* Settings = OwnerSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
			{
				if (Settings->bInitialRootTransformValid)
				{
					const FMovieSceneContext& MovieContext =
						Context.Linker->GetInstanceRegistry()->GetContext(MixerEntry.InstanceHandle);
					ResolvedInitialRootTransform = Settings->GetInitialRootTransformAtTime(
						MovieContext.GetTime(), MovieContext.GetFrameRate());
				}
			}
		}

		if (bWantsCrossSectionAccumulation || bWantsLoopAccumulation)
		{
			UMovieSceneAnimationMixerTrack* MixerTrack = OwnerSection
				? OwnerSection->GetTypedOuter<UMovieSceneAnimationMixerTrack>() : nullptr;
			UMovieSceneRootMotionSection* RootMotionSection = nullptr;
			if (MixerTrack)
			{
				if (auto* TargetDecoration = MixerTrack->FindDecoration<UMovieSceneRootMotionTargetDecoration>())
				{
					RootMotionSection = TargetDecoration->GetRootMotionSection();
				}
			}

			if (RootMotionSection && OwnerSection)
			{
				const FFrameTime SectionStart(OwnerSection->HasStartFrame()
					? OwnerSection->GetInclusiveStartFrame() : FFrameNumber(0));

				bUseAccumulatedOffset = RootMotionSection->HasApplicableAccumulatedEntry(
					SectionStart, OwnerSection, bWantsCrossSectionAccumulation);

				if (bUseAccumulatedOffset)
				{
					FFrameTime CurrentTime = SectionStart;
					if (Context.Linker && MixerEntry.InstanceHandle.IsValid())
					{
						CurrentTime = Context.Linker->GetInstanceRegistry()->GetContext(MixerEntry.InstanceHandle).GetTime();
					}

					ResolvedAnimSpaceAccumulatedTransform =
						RootMotionSection->GetAnimSpaceAccumulatedOffsetForSection(
							CurrentTime, OwnerSection, bWantsCrossSectionAccumulation);
				}
			}
		}
	}

	if (ThisRootSpace == EMovieSceneRootMotionSpace::AnimationSpace)
	{
		Conversion |= ESpaceConversions::ComponentToActorRotation;

		if (bRootComponentHasKeyedTransform)
		{
			Conversion |= ESpaceConversions::AnimationToWorld;
		}
		else if (Context.TransformOriginSystem && Context.TransformOriginSystem->GetTransformOrigin(MixerEntry.InstanceHandle, TransformOrigin))
		{
			Conversion |= ESpaceConversions::TransformOriginToWorld;
		}
	}
	else
	{
		Conversion |= ESpaceConversions::WorldSpaceComponentTransformCompensation;
	}

	if (Conversion != ESpaceConversions::None)
	{
		FAnimNextConvertRootMotionToWorldSpaceTask Task =
			FAnimNextConvertRootMotionToWorldSpaceTask::Make(
				Context.RootMotion, TransformOrigin, RootTransform, RootOffsetOrigin, Conversion);

		Task.InitialAnimRootTransform = ResolvedInitialRootTransform;
		Task.AnimSpaceAccumulatedTransform = bUseAccumulatedOffset
			? ResolvedAnimSpaceAccumulatedTransform
			: ResolvedInitialRootTransform;

		if (EnumHasAnyFlags(Conversion, ESpaceConversions::BoneMatchOffset) && MixerEntry.BoneMatchTransform.IsSet())
		{
			Task.BoneMatchTransform = MixerEntry.BoneMatchTransform.GetValue();
		}

		return Task;
	}

	return {};
}

// One section's contribution to a layer's notify dispatch. IntraLayerWeight is the
// section's normalized weight within its priority group, or for transitions the
// (1 - alpha) / alpha crossfade of from/to. Matches legacy MultiWayBlend semantics
// (non-additives normalize, additives don't).
struct FNotifyContribution
{
	TSharedPtr<FMovieSceneAnimMixerEntry> Entry;
	double IntraLayerWeight = 1.0;
};

// Per-layer notify dispatch info collected during program build, resolved into
// FSequencerMixerPendingNotifyBatch entries in a final reverse pass that walks layers
// top-to-bottom accumulating the exposure factor.
struct FNotifyLayerInfo
{
	double LayerWeight = 1.0;
	float MaskMaxBoneWeight = 1.0f;
	TArray<FNotifyContribution> Contributions;
};

// Build an evaluation program from a mixer's entries with optional filtering.
// Handles transition detection (transitions whose from/to sections are filtered out
// are ignored, and the surviving section is treated normally), priority grouping,
// blend stacks, root motion space conversion, and mirroring.
// Does NOT include FAnimNextStoreRootTransformTask - caller adds that if needed.
// If OutNotifyBatches is non-null, per-section anim-notify batches are appended with
// their effective final dispatch weight.
static TSharedPtr<UE::UAF::FEvaluationProgram> BuildProgramFromEntries(
	const FMovieSceneAnimMixer& Mixer,
	const FProgramBuildContext& Context,
	const FProgramBuildFilter& Filter = {},
	TArray<FSequencerMixerPendingNotifyBatch>* OutNotifyBatches = nullptr)
{
	// Crossfade alpha of the transition entry, captured before the transition pass
	// overwrites CombinedWeight to 1.0 (see transition handling below).
	TMap<FMovieSceneAnimMixerEntry*, double> TransitionAlphaByEntry;

	// Populated as we walk each priority/layer group; resolved into OutNotifyBatches after the
	// main loop so we can compute each layer's exposure given the higher layers above it.
	TArray<FNotifyLayerInfo> NotifyLayers;
	// First pass: determine which non-transition entries pass the filter.
	// We need this to decide which transitions are valid.
	TSet<FObjectKey> IncludedSectionOwners;
	for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer.WeakEntries)
	{
		TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin();
		if (!Entry || !Entry->EvalTask.IsValid() || Entry->bIsTransition)
		{
			continue;
		}
		if (PassesFilter(*Entry, Filter))
		{
			// Skip entries whose mixer layer row is muted/disabled on the mixer track.
			if (UMovieSceneAnimationMixerTrack* MixerTrack = Mixer.OwnerTrack.Get())
			{
				if (MixerTrack->IsRowEvalDisabled(Entry->Priority))
				{
					continue;
				}
			}
			IncludedSectionOwners.Add(Entry->EntityOwner);
		}
	}

	USceneComponent* Component = Context.Component;
	AActor* Owner = Component ? Component->GetOwner() : nullptr;
	USceneComponent* Root = Owner ? Owner->GetRootComponent() : nullptr;
	const bool bRootComponentHasKeyedTransform = (Root && Context.RootMotionSystem && Context.RootMotionSystem->IsTransformKeyed(Root));

	// Second pass: handle transitions. A transition is only valid if BOTH its from and to
	// sections are in the included set. If a transition is invalid (one side is filtered out),
	// the surviving section is treated normally (not added to SectionsInTransition).
	TSet<FObjectKey> SectionsInTransition;

	for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer.WeakEntries)
	{
		TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin();
		if (!Entry || !Entry->bIsTransition || !Entry->EvalTask.IsValid())
		{
			continue;
		}
		if (!PassesFilter(*Entry, Filter))
		{
			continue;
		}

		UMovieSceneAnimTransitionSectionBase* TransitionSection = Cast<UMovieSceneAnimTransitionSectionBase>(Entry->EntityOwner.ResolveObjectPtr());
		if (!TransitionSection || !TransitionSection->IsValid())
		{
			continue;
		}

		FObjectKey FromKey(TransitionSection->FromSection);
		FObjectKey ToKey(TransitionSection->ToSection);

		// Only activate this transition if both from/to are included
		if (!IncludedSectionOwners.Contains(FromKey) || !IncludedSectionOwners.Contains(ToKey))
		{
			continue;
		}

		SectionsInTransition.Add(FromKey);
		SectionsInTransition.Add(ToKey);

		TSharedPtr<FMovieSceneAnimMixerEntry> FromEntry = Mixer.FindEntryByOwner(FromKey);
		TSharedPtr<FMovieSceneAnimMixerEntry> ToEntry = Mixer.FindEntryByOwner(ToKey);

		// Wrap from/to tasks in mini-programs that include mirroring and
		// space conversion so the transition blends in world space. This is
		// required for AccumulatedOffset (different starting positions) and
		// is more correct in general when sections have different settings.
		auto WrapEntryTask = [&](const TSharedPtr<FMovieSceneAnimMixerEntry>& WrapEntry)
			-> TSharedPtr<FAnimNextEvaluationTask>
		{
			if (!WrapEntry || !WrapEntry->EvalTask.IsValid())
			{
				return nullptr;
			}

			bool bNeedsMirror = (WrapEntry->MirrorTable != nullptr);
			bool bNeedsExtraction = WrapEntry->RootMotionSettings.IsSet();
			TOptional<FAnimNextConvertRootMotionToWorldSpaceTask> SpaceTask;
			if (!Filter.bSkipRootMotionConversion)
			{
				SpaceTask = BuildSpaceConversionTask(*WrapEntry, Context, bRootComponentHasKeyedTransform);
			}

			if (!bNeedsMirror && !bNeedsExtraction && !SpaceTask.IsSet())
			{
				return WrapEntry->EvalTask;
			}

			TSharedPtr<UE::UAF::FEvaluationProgram> MiniProgram = MakeShared<UE::UAF::FEvaluationProgram>();
			MiniProgram->AppendTaskPtr(WrapEntry->EvalTask);

			if (bNeedsMirror)
			{
				TSharedPtr<FAnimNextEvaluationMirroringTask> MirrorTask = MakeShared<FAnimNextEvaluationMirroringTask>();
				MirrorTask->Setup.MirrorDataTable = WrapEntry->MirrorTable;
				MiniProgram->AppendTaskPtr(MirrorTask);
			}

			if (bNeedsExtraction)
			{
				MiniProgram->AppendTask(FAnimNextExtractRootMotionTask::Make(
					WrapEntry->RootMotionSettings->RootMotionSource));
			}

			if (Filter.CaptureAnimSpaceRoot && SpaceTask.IsSet())
			{
				SpaceTask->CaptureAnimSpaceRoot = Filter.CaptureAnimSpaceRoot;
			}

			if (SpaceTask.IsSet())
			{
				MiniProgram->AppendTask(MoveTemp(SpaceTask.GetValue()));
			}

			TSharedPtr<FAnimNextExecuteProgramTask> WrappedTask = MakeShared<FAnimNextExecuteProgramTask>();
			WrappedTask->Program = MiniProgram;
			return WrappedTask;
		};

		TSharedPtr<FAnimNextEvaluationTask> FromTask = WrapEntryTask(FromEntry);
		TSharedPtr<FAnimNextEvaluationTask> ToTask = WrapEntryTask(ToEntry);

		if (FAnimNextTransitionEvaluationTask* TransitionTask =
			static_cast<FAnimNextTransitionEvaluationTask*>(Entry->EvalTask.Get()))
		{
			float DeltaTime = 0.0f;
			if (Context.Linker && Entry->InstanceHandle.IsValid())
			{
				const FMovieSceneContext& MovieContext = Context.Linker->GetInstanceRegistry()->GetContext(Entry->InstanceHandle);
				const FFrameTime FrameDelta = MovieContext.GetDelta();
				const FFrameRate FrameRate = MovieContext.GetFrameRate();
				DeltaTime = static_cast<float>(FrameDelta.AsDecimal() / FrameRate.AsDecimal());
			}

			double BlendWeight = Entry->CombinedWeight;
			TransitionTask->Update(FromTask, ToTask, BlendWeight, DeltaTime);
		}

		// Compute the interpolated section weight for the layer blend.
		// The transition's CombinedWeight is the crossfade alpha (0..1).
		double TransitionAlpha = Entry->CombinedWeight;
		double FromWeight = FromEntry ? FromEntry->SectionWeight : 1.0;
		double ToWeight = ToEntry ? ToEntry->SectionWeight : 1.0;
		Entry->SectionWeight = FMath::Lerp(FromWeight, ToWeight, TransitionAlpha);
		Entry->CombinedWeight = 1.0;

		TransitionAlphaByEntry.Add(Entry.Get(), TransitionAlpha);
	}

	// Third pass: collect all entries that will actually be evaluated.
	// Non-transition entries that pass filter AND aren't handled by a transition,
	// plus transition entries whose from/to are both valid.
	TArray<TSharedPtr<FMovieSceneAnimMixerEntry>> FinalEntries;
	for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer.WeakEntries)
	{
		TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin();
		if (!Entry || !Entry->EvalTask.IsValid())
		{
			continue;
		}
		if (!PassesFilter(*Entry, Filter))
		{
			continue;
		}
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Mixer.OwnerTrack.Get())
		{
			if (MixerTrack->IsRowEvalDisabled(Entry->Priority))
			{
				continue;
			}
		}
		// Skip non-transition entries that are handled by a valid transition
		if (!Entry->bIsTransition && SectionsInTransition.Contains(Entry->EntityOwner))
		{
			continue;
		}
		// Skip transition entries whose from/to aren't both included
		if (Entry->bIsTransition)
		{
			UMovieSceneAnimTransitionSectionBase* TransitionSection = Cast<UMovieSceneAnimTransitionSectionBase>(Entry->EntityOwner.ResolveObjectPtr());
			if (!TransitionSection || !TransitionSection->IsValid())
			{
				continue;
			}
			if (!IncludedSectionOwners.Contains(FObjectKey(TransitionSection->FromSection)) ||
				!IncludedSectionOwners.Contains(FObjectKey(TransitionSection->ToSection)))
			{
				continue;
			}
		}
		FinalEntries.Add(Entry);
	}

	if (FinalEntries.Num() == 0)
	{
		return nullptr;
	}

	TSharedPtr<UE::UAF::FEvaluationProgram> Program = MakeShared<UE::UAF::FEvaluationProgram>();

	const int32 NumEntries = FinalEntries.Num();

	// Helper: determine if a priority group needs a weighted average (multiple non-additive entries).
	auto LookaheadComputeAccumulatedWeight = [&FinalEntries, NumEntries](int32 StartAtIndex, int32 Priority, bool bBackwardsCompat, float& OutAccumulatedWeight) -> bool
	{
		int32 NumAbsoluteBlends = 0;
		int32 NumSkippedBlends = 0;
		float AccumulatedWeight = 0.f;
		for (int32 NextIndex = StartAtIndex; NextIndex < NumEntries; ++NextIndex)
		{
			const TSharedPtr<FMovieSceneAnimMixerEntry>& NextEntry = FinalEntries[NextIndex];
			if (!NextEntry->EvalTask.IsValid())
			{
				continue;
			}
			if (NextEntry->Priority != Priority || NextEntry->bAdditive)
			{
				break;
			}
			const double EntryWeight = bBackwardsCompat ? NextEntry->CombinedWeight : NextEntry->EasingWeight;
			if (!FMath::IsNearlyEqual(AccumulatedWeight, AccumulatedWeight + EntryWeight, KINDA_SMALL_NUMBER))
			{
				++NumAbsoluteBlends;
				AccumulatedWeight += EntryWeight;
			}
			else
			{
				++NumSkippedBlends;
			}
		}

		if (NumAbsoluteBlends + NumSkippedBlends > 1 && !FMath::IsNearlyZero(AccumulatedWeight, KINDA_SMALL_NUMBER))
		{
			OutAccumulatedWeight = AccumulatedWeight;
			return true;
		}
		return false;
	};

	// Helper: append eval + mirror + extraction + space conversion for an entry.
	auto AppendEntryEvaluation = [&](const TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry)
	{
		// Pre-bake root motion entries (e.g. layered Control Rigs) read bone[0] from the
		// previous pose to drive their hierarchy; bake accumulated root motion into bone[0]
		// so they see the moved root rather than the identity left by the previous extract.
		// Restricted to component-space-root destinations (RootBone, Component-non-root)
		// since the attribute is in actor-relative space only in those modes.
		// Also skipped when the eval task itself populates the attribute (legacy SwapRootBone
		// override) - the bake's write would just be discarded.
		const bool bHasRootMotionOverride = MixerEntry->RootMotionSettings.IsSet()
			&& MixerEntry->RootMotionSettings->bHasRootMotionOverride;
		const bool bWantsBakedBone = MixerEntry->bPreBakeRootMotion
			&& Context.RootMotion.IsValid()
			&& Context.RootMotion->bComponentSpaceRoot
			&& !MixerEntry->bIsTransition
			&& MixerEntry->MixerLayer.IsValid()
			&& !bHasRootMotionOverride;

		if (bWantsBakedBone)
		{
			Program->AppendTask(FAnimNextBakeRootAttributeToBoneTask::Make(Context.RootMotion));
		}

		Program->AppendTaskPtr(MixerEntry->EvalTask);

		if (MixerEntry->MirrorTable)
		{
			TSharedPtr<FAnimNextEvaluationMirroringTask> MirrorTask = MakeShared<FAnimNextEvaluationMirroringTask>();
			MirrorTask->Setup.MirrorDataTable = MixerEntry->MirrorTable;
			Program->AppendTaskPtr(MirrorTask);
		}

		if (MixerEntry->RootMotionSettings.IsSet() && !MixerEntry->bIsTransition)
		{
			// Two reasons to skip extraction here:
			//   (a) A legacy root motion override (SwapRootBone on the track/section) has the eval task
			//       populate the RootTransformAttribute directly and zero the root bone, so running extract
			//       would overwrite the correct attribute value with identity.
			//   (b) Entries without a mixer layer are on the backwards-compat path (Animation track +
			//       Control Rig track with the plugin enabled but no Animation Mixer track). In that
			//       configuration, no downstream layer consumes the RootTransformAttribute, and a
			//       subsequent pose producer like a Control Rig layer peeks the top-of-stack pose to
			//       drive its hierarchy -- if we zero bone 0 here, the Control Rig sees an identity root
			//       and its controls stop following the mesh.
			const bool bEvalTaskHandledExtraction = MixerEntry->RootMotionSettings->bHasRootMotionOverride;
			const bool bBackwardsCompat = !MixerEntry->MixerLayer.IsValid();
			if (!bEvalTaskHandledExtraction && !bBackwardsCompat)
			{
				Program->AppendTask(FAnimNextExtractRootMotionTask::Make(
					MixerEntry->RootMotionSettings->RootMotionSource));
			}
		}

		if (bWantsBakedBone)
		{
			Program->AppendTask(FAnimNextResetRootBoneTask::Make());
		}

		if (!Filter.bSkipRootMotionConversion && !MixerEntry->bIsTransition && MixerEntry->RootMotionSettings.IsSet())
		{
			TOptional<FAnimNextConvertRootMotionToWorldSpaceTask> SpaceTask =
				BuildSpaceConversionTask(*MixerEntry, Context, bRootComponentHasKeyedTransform);
			if (SpaceTask.IsSet())
			{
				if (Filter.CaptureAnimSpaceRoot)
				{
					SpaceTask->CaptureAnimSpaceRoot = Filter.CaptureAnimSpaceRoot;
				}
				Program->AppendTask(MoveTemp(SpaceTask.GetValue()));
			}
		}
	};

	// Append the additive task for an entry, plus its un-normalized notify contribution to
	// the current layer (additives are not normalized within priority - matches legacy
	// AdditiveBlendNode bNormalizeAlpha=false).
	auto AppendAdditive = [&](const TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry, FNotifyLayerInfo& InNotifyLayer)
	{
		AppendEntryEvaluation(MixerEntry);
		Program->AppendTask(FAnimNextApplyAdditiveKeyframeTask::Make(MixerEntry->EasingWeight));
		if (OutNotifyBatches)
		{
			InNotifyLayer.Contributions.Add({MixerEntry, MixerEntry->EasingWeight});
		}
	};

	// Main evaluation loop- two-level: intra-layer, then inter-layer blend.
	bool bFirstLayer = true;
	for (int32 Index = 0; Index < NumEntries; )
	{
		const TSharedPtr<FMovieSceneAnimMixerEntry>& PeekEntry = FinalEntries[Index];
		const int32 Priority = PeekEntry->Priority;
		const TWeakObjectPtr<UMovieSceneAnimationMixerLayer> Layer = PeekEntry->MixerLayer;
		
		// If users are running animation tracks and control rig with the mixer plugin turned on but without using a mixer track,
		// there will be no 'layer' but we use hardcoded priorities to give the previous behavior. This requires slightly
		// different handling of weights.
		const bool bBackwardsCompat = !Layer.IsValid();

		// ---- Level 1: Intra-layer evaluation ----
		// Produce a single pose on the stack for this priority group.
		double LayerWeight = 1.0;

		// Check if this group has a transition entry
		TSharedPtr<FMovieSceneAnimMixerEntry> TransitionEntry;
		for (int32 ScanIndex = Index; ScanIndex < NumEntries; ++ScanIndex)
		{
			const TSharedPtr<FMovieSceneAnimMixerEntry>& ScanEntry = FinalEntries[ScanIndex];
			if (ScanEntry->Priority != Priority) break;
			if (ScanEntry->bIsTransition && ScanEntry->EvalTask.IsValid())
			{
				TransitionEntry = ScanEntry;
				break;
			}
		}

		FNotifyLayerInfo NotifyLayer;

		if (TransitionEntry)
		{
			// Transition: produces a single blended pose from from/to
			AppendEntryEvaluation(TransitionEntry);
			LayerWeight = TransitionEntry->SectionWeight;

			// The transition entry itself has no notifies; fire the from/to sections'
			// notifies weighted by the crossfade alpha.
			if (OutNotifyBatches)
			{
				const double TransitionAlpha = TransitionAlphaByEntry.FindRef(TransitionEntry.Get());

				UMovieSceneAnimTransitionSectionBase* TransitionSection =
					Cast<UMovieSceneAnimTransitionSectionBase>(TransitionEntry->EntityOwner.ResolveObjectPtr());
				if (TransitionSection && TransitionSection->IsValid())
				{
					if (TSharedPtr<FMovieSceneAnimMixerEntry> FromEntry = Mixer.FindEntryByOwner(FObjectKey(TransitionSection->FromSection)))
					{
						NotifyLayer.Contributions.Add({FromEntry, 1.0 - TransitionAlpha});
					}
					if (TSharedPtr<FMovieSceneAnimMixerEntry> ToEntry = Mixer.FindEntryByOwner(FObjectKey(TransitionSection->ToSection)))
					{
						NotifyLayer.Contributions.Add({ToEntry, TransitionAlpha});
					}
				}
			}

			// Process additives at this priority, skip everything else
			for ( ; Index < NumEntries; ++Index)
			{
				const TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry = FinalEntries[Index];
				if (MixerEntry->Priority != Priority)
				{
					break;
				}
				if (MixerEntry->bAdditive && MixerEntry->EvalTask.IsValid())
				{
					AppendAdditive(MixerEntry, NotifyLayer);
				}
			}
		}
		else
		{
			// Count non-additive entries for weighted average decision
			float AccumulatedGroupWeight = 1.f;
			const bool bNeedsSeparateWeightStack = LookaheadComputeAccumulatedWeight(Index, Priority, bBackwardsCompat, AccumulatedGroupWeight);

			double MaxSectionWeight = 0.0;
			bool bIsFirstAbsoluteBlend = true;

			for ( ; Index < NumEntries; ++Index)
			{
				const TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry = FinalEntries[Index];
				if (MixerEntry->Priority != Priority)
				{
					break;
				}

				if (!MixerEntry->EvalTask.IsValid())
				{
					continue;
				}

				if (MixerEntry->bAdditive)
				{
					// Additives are applied after the absolute result
					AppendAdditive(MixerEntry, NotifyLayer);
					continue;
				}

				const double IntraWeight = bBackwardsCompat ? MixerEntry->CombinedWeight : MixerEntry->EasingWeight;

				if (bNeedsSeparateWeightStack && FMath::IsNearlyZero(IntraWeight, KINDA_SMALL_NUMBER))
				{
					continue;
				}

				MaxSectionWeight = FMath::Max(MaxSectionWeight, MixerEntry->SectionWeight);

				AppendEntryEvaluation(MixerEntry);

				// Intra-layer blending (weighted average among overlapping sections)
				if (bNeedsSeparateWeightStack)
				{
					if (bIsFirstAbsoluteBlend)
					{
						bIsFirstAbsoluteBlend = false;
						Program->AppendTask(FAnimNextBlendOverwriteKeyframeWithScaleTask::Make(
							static_cast<float>(IntraWeight / AccumulatedGroupWeight)));
					}
					else
					{
						Program->AppendTask(FMovieSceneAccumulateAbsoluteBlendTask::Make(
							static_cast<float>(IntraWeight / AccumulatedGroupWeight)));
					}
				}

				// Non-additive entries normalize within their priority group when overlapping
				// (legacy MultiWayBlend bNormalizeAlpha=true); a sole entry contributes at 1.0.
				if (OutNotifyBatches)
				{
					const double NormalizedIntra = bNeedsSeparateWeightStack
						? IntraWeight / AccumulatedGroupWeight
						: 1.0;
					NotifyLayer.Contributions.Add({MixerEntry, NormalizedIntra});
				}
			}

			if (bNeedsSeparateWeightStack)
			{
				Program->AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
				LayerWeight = bBackwardsCompat ? 1.0 : MaxSectionWeight;
			}
			else
			{
				// Single entry: combined weight (easing * section weight) controls
				// the blend with the base/lower layer
				LayerWeight = PeekEntry->CombinedWeight;
			}
		}

		// ---- Level 2: Inter-layer blend ----
		// Blend this layer's result with what's below on the stack.
		// For bus-target programs, skip the first layer's blend since the
		// program will be executed inline inside a consuming mixer that
		// handles the blend with its own base keyframe.
		const bool bSkipBlend = bFirstLayer && Filter.bSkipFirstInterLayerBlend;
		bFirstLayer = false;

		if (!bSkipBlend)
		{
			// Apply decoration-provided layer weight (e.g. from a Layer Weight decoration)
			if (Layer.IsValid() && Context.DecorationLayerWeightMap)
			{
				if (const double* DecWeight = Context.DecorationLayerWeightMap->Find(Layer))
				{
					LayerWeight *= *DecWeight;
				}
			}

			TSharedPtr<FAnimNextEvaluationTask> LayerBlendTask;
			if (Layer.IsValid() && Context.LayerBlendTaskMap)
			{
				if (const FLayerBlendTaskBuilder* Builder = Context.LayerBlendTaskMap->Find(Layer))
				{
					if (Builder->IsValid())
					{
						LayerBlendTask = Builder->Build(LayerWeight);
						NotifyLayer.MaskMaxBoneWeight = Builder->MaskMaxBoneWeight;
					}
				}
			}

			if (LayerBlendTask)
			{
				Program->AppendTaskPtr(LayerBlendTask);
				Program->AppendTask(FAnimNextNormalizeKeyframeRotationsTask());
			}
			else
			{
				Program->AppendTask(FAnimNextBlendTwoKeyframesPreserveRootMotionTask::Make(LayerWeight));
			}
		}

		if (OutNotifyBatches)
		{
			NotifyLayer.LayerWeight = LayerWeight;
			NotifyLayers.Add(MoveTemp(NotifyLayer));
		}
	}

	// Walk layers top-to-bottom, accumulating Exposure as the product over upper layers of
	// (1 - upper.LayerWeight * upper.MaskMaxBoneWeight). Each contribution's dispatch weight
	// is IntraLayerWeight * LayerWeight * Exposure, gated at ZERO_ANIMWEIGHT_THRESH.
	if (OutNotifyBatches)
	{
		double Exposure = 1.0;
		for (int32 LayerIdx = NotifyLayers.Num() - 1; LayerIdx >= 0; --LayerIdx)
		{
			FNotifyLayerInfo& NotifyLayer = NotifyLayers[LayerIdx];
			for (FNotifyContribution& Contribution : NotifyLayer.Contributions)
			{
				TSharedPtr<FMovieSceneAnimMixerEntry>& Entry = Contribution.Entry;
				if (!Entry || Entry->PendingNotifies.IsEmpty())
				{
					continue;
				}

				const double EffectiveWeight = Contribution.IntraLayerWeight * NotifyLayer.LayerWeight * Exposure;
				if (EffectiveWeight > ZERO_ANIMWEIGHT_THRESH)
				{
					FSequencerMixerPendingNotifyBatch Batch;
					Batch.Notifies = MoveTemp(Entry->PendingNotifies);
					Batch.Weight = static_cast<float>(EffectiveWeight);
					OutNotifyBatches->Add(MoveTemp(Batch));
				}
				else
				{
					Entry->PendingNotifies.Reset();
				}
			}

			Exposure *= FMath::Clamp(1.0 - NotifyLayer.LayerWeight * NotifyLayer.MaskMaxBoneWeight, 0.0, 1.0);
		}
	}

	return Program;
}

} // namespace UE::MovieScene


namespace UE::MovieScene
{
	struct FUpdateTaskEntities
	{
		static void ForEachEntity(
			TSharedPtr<FAnimNextEvaluationTask> Task,
			TSharedPtr<FMovieSceneAnimMixerEntry> MixerEntry,
			const FMovieSceneRootMotionSettings* RootMotionSettings,
			const double* WeightAndEasings,
			const double* EasingResult,
			const double* WeightResult,
			const FTransform* BoneMatchTransform,
			const FFrameTime* OptEvalTime)
		{
			if (MixerEntry)
			{
				MixerEntry->EvalTask = Task;

				// The Task component holds any FAnimNextEvaluationTask derivative; gate on the
				// runtime type before downcasting since a sibling-type static_cast would be UB.
				MixerEntry->PendingNotifies.Reset();
				if (Task && Task->GetStruct() == FMovieSceneSkeletalAnimationEvaluationTask::StaticStruct())
				{
					FMovieSceneSkeletalAnimationEvaluationTask* AnimTask =
						static_cast<FMovieSceneSkeletalAnimationEvaluationTask*>(Task.Get());
					MixerEntry->PendingNotifies = MoveTemp(AnimTask->AnimationData.PendingNotifies);
				}

				if (MixerEntry->bSkipPoseWeight)
				{
					MixerEntry->CombinedWeight = 1.0;
					MixerEntry->EasingWeight = 1.0;
					MixerEntry->SectionWeight = 1.0;
				}
				else
				{
					const double Combined = WeightAndEasings ? *WeightAndEasings : 1.0;
					MixerEntry->CombinedWeight = Combined;
					MixerEntry->EasingWeight = EasingResult ? *EasingResult : Combined;
					MixerEntry->SectionWeight = WeightResult ? *WeightResult : 1.0;
				}

				if (RootMotionSettings)
				{
					MixerEntry->RootMotionSettings = *RootMotionSettings;

					// Evaluate offset channels from the decoration into RootLocation/RootRotation.
					// The channels are stored as pointers in the settings to avoid using
					// BuiltInComponents::DoubleChannel which clashes with control rig parameters.
					if (OptEvalTime)
					{
						FMovieSceneRootMotionSettings& Settings = *MixerEntry->RootMotionSettings;
						FVector Location = Settings.RootLocation;
						FRotator Rotation = Settings.RootRotation.Rotator();
						const FMovieSceneDoubleChannel* const* Channels = Settings.OffsetChannels;
						if (Channels[0]) { double Val; if (Channels[0]->Evaluate(*OptEvalTime, Val)) { Location.X = Val; } }
						if (Channels[1]) { double Val; if (Channels[1]->Evaluate(*OptEvalTime, Val)) { Location.Y = Val; } }
						if (Channels[2]) { double Val; if (Channels[2]->Evaluate(*OptEvalTime, Val)) { Location.Z = Val; } }
						if (Channels[3]) { double Val; if (Channels[3]->Evaluate(*OptEvalTime, Val)) { Rotation.Roll  = Val; } }
						if (Channels[4]) { double Val; if (Channels[4]->Evaluate(*OptEvalTime, Val)) { Rotation.Pitch = Val; } }
						if (Channels[5]) { double Val; if (Channels[5]->Evaluate(*OptEvalTime, Val)) { Rotation.Yaw   = Val; } }
						Settings.RootLocation = Location;
						Settings.RootRotation = Rotation.Quaternion();
					}
				}

				if (BoneMatchTransform)
				{
					MixerEntry->BoneMatchTransform = *BoneMatchTransform;
				}
				else
				{
					MixerEntry->BoneMatchTransform.Reset();
				}
			}
		}
	};

	struct FEvaluateAnimMixers
	{
		TMap<FMovieSceneAnimMixerKey, TSharedPtr<FMovieSceneAnimMixer>>* Mixers;
		TMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* RootMotions;
		const UMovieSceneRootMotionSystem* RootMotionSystem;
		const UMovieSceneTransformOriginSystem* TransformOriginSystem;
		UMovieSceneEntitySystemLinker* Linker;
		UMovieSceneAnimMixerSystem* MixerSystem;

		// Layer blend task builders grouped by BoundObjectKey -> {MixerLayer -> Builder}
		TMap<FObjectKey, TMap<TWeakObjectPtr<UMovieSceneAnimationMixerLayer>, FLayerBlendTaskBuilder>> LayerBlendBuilders;

		// Decoration-provided layer weights grouped by BoundObjectKey -> {MixerLayer -> Weight}
		TMap<FObjectKey, TMap<TWeakObjectPtr<UMovieSceneAnimationMixerLayer>, double>> DecorationLayerWeights;

		FEvaluateAnimMixers(TMap<FMovieSceneAnimMixerKey, TSharedPtr<FMovieSceneAnimMixer>>* InMixers, TMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* InRootMotions, UMovieSceneEntitySystemLinker* InLinker, UMovieSceneAnimMixerSystem* InMixerSystem)
			: Mixers(InMixers)
			, RootMotions(InRootMotions)
			, Linker(InLinker)
			, MixerSystem(InMixerSystem)
		{
			RootMotionSystem = InLinker->FindSystem<UMovieSceneRootMotionSystem>();
			TransformOriginSystem = InLinker->FindSystem<UMovieSceneTransformOriginSystem>();
		}

		void PreTask()
		{
			// Build layer blend task lookup from entities produced by the mask system
			// (or other blend providers).
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
			FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
			FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObjectKey)
			.Read(AnimMixerComponents->MixerLayer)
			.Read(AnimMixerComponents->LayerBlendTask)
			.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
			.Iterate_PerEntity(&Linker->EntityManager,
				[this](FObjectKey BoundObjectKey, TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer, const FLayerBlendTaskBuilder& Builder)
				{
					if (MixerLayer && Builder.IsValid())
					{
						LayerBlendBuilders.FindOrAdd(BoundObjectKey).Add(MixerLayer, Builder);
					}
				}
			);

			// Build decoration layer weight lookup from layer weight section entities.
			// These are tagged with LayerWeight and have their WeightResult evaluated
			// by the standard weight/easing system.
			FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObjectKey)
			.Read(AnimMixerComponents->MixerLayer)
			.Read(BuiltInComponents->WeightResult)
			.FilterAll({ AnimMixerComponents->Tags.LayerWeight })
			.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
			.Iterate_PerEntity(&Linker->EntityManager,
				[this](FObjectKey BoundObjectKey, TObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer, double Weight)
				{
					if (MixerLayer)
					{
						DecorationLayerWeights.FindOrAdd(BoundObjectKey).Add(MixerLayer, Weight);
					}
				}
			);

			// --- Bus evaluation: dependency-ordered pre-evaluation of bus-target mixers ---
			// Skip the full dependency graph build when no bus targets are in use.
			if (HasAnyBusTarget())
			{
				EvaluateBusTargetMixers();
			}
			else
			{
				MixerSystem->ClearBusStorage();
			}
		}

		bool HasAnyBusTarget() const
		{
			for (const auto& MixerPair : *Mixers)
			{
				if (MixerPair.Key.Target.GetPtr<FMovieSceneAnimBusTarget>())
				{
					return true;
				}
			}
			return false;
		}

		void EvaluateBusTargetMixers()
		{
			MixerSystem->ClearBusStorage();

			// Identify objects with bus targets and mark dirty caches
			TSet<FObjectKey> ObjectsWithBusTargets;
			for (const auto& MixerPair : *Mixers)
			{
				if (MixerPair.Key.Target.GetPtr<FMovieSceneAnimBusTarget>())
				{
					FObjectKey ObjKey = MixerPair.Key.BoundObjectKey;
					ObjectsWithBusTargets.Add(ObjKey);

					if (TSharedPtr<FMovieSceneAnimMixer> Mixer = MixerPair.Value)
					{
						if (Mixer->bNeedsResort)
						{
							MixerSystem->GetOrCreateBusTopology(ObjKey).bDirty = true;
						}
					}
				}
			}

			MixerSystem->RemoveStaleBusTopologies(ObjectsWithBusTargets);

			for (FObjectKey ObjKey : ObjectsWithBusTargets)
			{
				auto& Cache = MixerSystem->GetOrCreateBusTopology(ObjKey);
				if (Cache.bDirty)
				{
					// Gather owner tracks for this object only when we need to rebuild
					TArray<UMovieSceneAnimationMixerTrack*> ObjectTracks;
					for (const auto& MixerPair : *Mixers)
					{
						if (MixerPair.Key.BoundObjectKey == ObjKey)
						{
							if (TSharedPtr<FMovieSceneAnimMixer> Mixer = MixerPair.Value)
							{
								if (Mixer->OwnerTrack.IsValid())
								{
									ObjectTracks.AddUnique(Mixer->OwnerTrack.Get());
								}
							}
						}
					}
					Cache.EvalOrder = FAnimMixerBusUtils::ComputeBusEvaluationOrder(ObjectTracks);
					Cache.bDirty = false;
				}

				// Build programs and store bus data in cached order
				for (const FName& BusName : Cache.EvalOrder)
				{
					FMovieSceneAnimMixerKey MixerKey;
					MixerKey.BoundObjectKey = ObjKey;
					MixerKey.Target = TInstancedStruct<FMovieSceneAnimBusTarget>::Make(BusName);

					TSharedPtr<FMovieSceneAnimMixer> Mixer = Mixers->FindRef(MixerKey);
					if (!Mixer)
					{
						continue;
					}

					USceneComponent* Component = Cast<USceneComponent>(ObjKey.ResolveObjectPtr());
					TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = RootMotions->FindRef(ObjKey).Pin();

					FProgramBuildContext BuildContext;
					BuildContext.RootMotion = RootMotion;
					BuildContext.RootMotionSystem = RootMotionSystem;
					BuildContext.TransformOriginSystem = TransformOriginSystem;
					BuildContext.Linker = Linker;
					BuildContext.Component = Component;
					BuildContext.LayerBlendTaskMap = LayerBlendBuilders.Find(ObjKey);
					BuildContext.DecorationLayerWeightMap = DecorationLayerWeights.Find(ObjKey);

					FProgramBuildFilter BuildFilter;
					BuildFilter.bSkipFirstInterLayerBlend = true;

					// Notify batches stay on the bus mixer; consuming mixers fold them in below.
					Mixer->PendingNotifyBatches.Reset();
					Mixer->EvaluationProgram = BuildProgramFromEntries(*Mixer, BuildContext, BuildFilter, &Mixer->PendingNotifyBatches);

					// Check root motion per-frame since entries can change
					bool bHasRootMotion = false;
					if (Mixer->EvaluationProgram && RootMotion)
					{
						for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer->WeakEntries)
						{
							if (TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin())
							{
								if (Entry->RootMotionSettings.IsSet())
								{
									bHasRootMotion = true;
									break;
								}
								if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Entry->EntityOwner.ResolveObjectPtr()))
								{
									TSharedPtr<FMovieSceneAnimBusData> BusData = MixerSystem->ReadBusData(ObjKey, BusSection->BusName);
									if (BusData && BusData->bHasRootMotion)
									{
										bHasRootMotion = true;
										break;
									}
								}
							}
						}

						if (bHasRootMotion)
						{
							AActor* Owner = Component ? Component->GetOwner() : nullptr;
							USceneComponent* Root = Owner ? Owner->GetRootComponent() : nullptr;
							const bool bComponentHasKeyedTransform = (Component && RootMotionSystem && RootMotionSystem->IsTransformKeyed(Component));
							const bool bRootComponentHasKeyedTransform = (Root && RootMotionSystem && RootMotionSystem->IsTransformKeyed(Root));
							FAnimNextStoreRootTransformTask StoreTask = FAnimNextStoreRootTransformTask::Make(RootMotion, bComponentHasKeyedTransform, bRootComponentHasKeyedTransform);
							StoreTask.bForceRootBoneDestination = true;
							Mixer->EvaluationProgram->AppendTask(MoveTemp(StoreTask));
						}
					}

					if (Mixer->EvaluationProgram)
					{
						TSharedPtr<FMovieSceneAnimBusData> BusData = MakeShared<FMovieSceneAnimBusData>();
						BusData->Program = Mixer->EvaluationProgram;
						BusData->bHasRootMotion = bHasRootMotion;
						MixerSystem->StoreBusData(ObjKey, BusName, BusData);
					}
				}
			}
		}

		// Get an InstanceHandle from the mixer's entries (any valid entry will do).
		// Mixer entities themselves don't carry InstanceHandle - only section entities do.
		static FInstanceHandle GetInstanceHandleFromMixer(const FMovieSceneAnimMixer& Mixer)
		{
			for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer.WeakEntries)
			{
				if (TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin())
				{
					if (Entry->InstanceHandle.IsValid())
					{
						return Entry->InstanceHandle;
					}
				}
			}
			return FInstanceHandle();
		}

		void ForEachEntity(
			FObjectComponent MeshComponent,
			const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
			TSharedPtr<FAnimNextEvaluationTask>& MixerTask
			) const
		{
			// Bus-target mixers are handled in EvaluateBusTargetMixers. Their
			// programs are stored in BusStorage, not on the MixerTask entity.
			if (Target.GetPtr<FMovieSceneAnimBusTarget>())
			{
				return;
			}

			TSharedPtr<FMovieSceneAnimMixer> Mixer = Mixers->FindRef(FMovieSceneAnimMixerKey({ MeshComponent.GetObject(), Target }));
			if (!Mixer)
			{
				return;
			}

			USceneComponent* Component = Cast<USceneComponent>(MeshComponent.GetObject());
			TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = RootMotions->FindRef(MeshComponent.GetObject()).Pin();

			FProgramBuildContext BuildContext;
			BuildContext.RootMotion = RootMotion;
			BuildContext.RootMotionSystem = RootMotionSystem;
			BuildContext.TransformOriginSystem = TransformOriginSystem;
			BuildContext.Linker = Linker;
			BuildContext.Component = Component;

			// Look up layer blend builders for this bound object
			const TMap<TWeakObjectPtr<UMovieSceneAnimationMixerLayer>, FLayerBlendTaskBuilder>* BlendMap =
				LayerBlendBuilders.Find(MeshComponent.GetObject());
			BuildContext.LayerBlendTaskMap = BlendMap;

			// Look up decoration layer weights for this bound object
			BuildContext.DecorationLayerWeightMap = DecorationLayerWeights.Find(MeshComponent.GetObject());

			// Apply bake filter from the bake target if present.
			FProgramBuildFilter BuildFilter;
			if (const FMovieSceneMixerBakeTarget* BakeTarget = Target.GetPtr<FMovieSceneMixerBakeTarget>())
			{
				if (!BakeTarget->Filter.ExcludeSections.IsEmpty())
				{
					BuildFilter.ExcludeSections = &BakeTarget->Filter.ExcludeSections;
				}
				if (!BakeTarget->Filter.IncludeOnlySections.IsEmpty())
				{
					BuildFilter.IncludeOnly = &BakeTarget->Filter.IncludeOnlySections;
				}
				BuildFilter.MinPriority = BakeTarget->Filter.MinPriority;
				BuildFilter.MaxPriority = BakeTarget->Filter.MaxPriority;
				BuildFilter.bSkipRootMotionConversion = BakeTarget->Filter.bSkipRootMotionConversion;
				BuildFilter.CaptureAnimSpaceRoot = BakeTarget->Filter.CaptureAnimSpaceRoot;
			}

			// Build program from entries. Gap entries (null EvalTask) are naturally
			// skipped, so this returns null during a full gap.
			Mixer->PendingNotifyBatches.Reset();
			Mixer->EvaluationProgram = BuildProgramFromEntries(*Mixer, BuildContext, BuildFilter, &Mixer->PendingNotifyBatches);

			// Forward notify batches from any bus mixers feeding this consuming mixer, scaled
			// by the bus section's CombinedWeight. This does not account for the bus section's
			// layer position or exposure within the consuming mix, which is a deliberate
			// approximation - the bus pose has already been evaluated independently.
			for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer->WeakEntries)
			{
				TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin();
				if (!Entry)
				{
					continue;
				}
				UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Entry->EntityOwner.ResolveObjectPtr());
				if (!BusSection)
				{
					continue;
				}
				FMovieSceneAnimMixerKey BusKey;
				BusKey.BoundObjectKey = MeshComponent.GetObject();
				BusKey.Target = TInstancedStruct<FMovieSceneAnimBusTarget>::Make(BusSection->BusName);
				TSharedPtr<FMovieSceneAnimMixer> BusMixer = Mixers->FindRef(BusKey);
				if (!BusMixer || BusMixer->PendingNotifyBatches.IsEmpty())
				{
					continue;
				}
				const float SectionWeight = static_cast<float>(Entry->CombinedWeight);
				if (SectionWeight <= ZERO_ANIMWEIGHT_THRESH)
				{
					continue;
				}
				for (const FSequencerMixerPendingNotifyBatch& BusBatch : BusMixer->PendingNotifyBatches)
				{
					const float ForwardWeight = BusBatch.Weight * SectionWeight;
					if (ForwardWeight <= ZERO_ANIMWEIGHT_THRESH)
					{
						continue;
					}
					FSequencerMixerPendingNotifyBatch& Forwarded = Mixer->PendingNotifyBatches.AddDefaulted_GetRef();
					Forwarded.Notifies = BusBatch.Notifies;
					Forwarded.Weight = ForwardWeight;
				}
			}

			if (RootMotion)
			{
				// Check if any entry produces root motion. If none do, this is a
				// gap (either full - no real entries at all, or soft - only non-root-motion
				// sections like facial are active). In either case, apply the persisted
				// gap transform if configured.
				bool bAnyEntryHasRootMotion = false;
				for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Mixer->WeakEntries)
				{
					if (TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin())
					{
						if (Entry->RootMotionSettings.IsSet())
						{
							bAnyEntryHasRootMotion = true;
							break;
						}
						// Check if this entry is a bus section whose source has root motion
						if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Entry->EntityOwner.ResolveObjectPtr()))
						{
							TSharedPtr<FMovieSceneAnimBusData> BusData = MixerSystem->ReadBusData(MeshComponent.GetObject(), BusSection->BusName);
							if (BusData && BusData->bHasRootMotion)
							{
								bAnyEntryHasRootMotion = true;
								break;
							}
						}
					}
				}

				if (!bAnyEntryHasRootMotion
					&& RootMotion->GapBehavior == EMovieSceneRootMotionGapBehavior::PersistPreviousTransform)
				{
					UMovieSceneRootMotionSection* Section = RootMotion->WeakRootMotionSection.Get();
					FInstanceHandle EntryInstanceHandle = GetInstanceHandleFromMixer(*Mixer);
					if (Section && EntryInstanceHandle.IsValid())
					{
						const FMovieSceneContext& Context = Linker->GetInstanceRegistry()->GetContext(EntryInstanceHandle);
						FTransform GapTransform = Section->GetAccumulatedOffsetAt(Context.GetTime());
						if (!GapTransform.Equals(FTransform::Identity))
						{
							if (!Mixer->EvaluationProgram)
							{
								Mixer->EvaluationProgram = MakeShared<UE::UAF::FEvaluationProgram>();
							}
							// Set the cached gap transform directly as the root motion
							// attribute. The value is already in post-conversion world
							// space from the trajectory cache.
							Mixer->EvaluationProgram->AppendTask(
								FAnimNextSetRootTransformAttributeTask::Make(GapTransform));
						}
					}
				}

				if (Mixer->EvaluationProgram)
				{
					// Skip the store task during bake - bake captures the root
					// motion attribute directly from the result keyframe.
					if (!Target.GetPtr<FMovieSceneMixerBakeTarget>())
					{
						AActor* Owner = Component ? Component->GetOwner() : nullptr;
						USceneComponent* Root = Owner ? Owner->GetRootComponent() : nullptr;
						const bool bComponentHasKeyedTransform = (Component && RootMotionSystem->IsTransformKeyed(Component));
						const bool bRootComponentHasKeyedTransform = (Root && RootMotionSystem->IsTransformKeyed(Root));
						FAnimNextStoreRootTransformTask StoreTask = FAnimNextStoreRootTransformTask::Make(RootMotion, bComponentHasKeyedTransform, bRootComponentHasKeyedTransform);
						if (UMovieSceneAnimMixerSystem::IsForceRootBoneDestinationScopeActive())
						{
							StoreTask.bForceRootBoneDestination = true;
						}
						Mixer->EvaluationProgram->AppendTask(MoveTemp(StoreTask));
					}
				}
			}

			if (Mixer->EvaluationProgram)
			{
				if (!MixerTask.IsValid())
				{
					MixerTask = MakeShared<FAnimNextExecuteProgramTask>();
				}
				TSharedPtr<FAnimNextExecuteProgramTask> EvalProgramTask = StaticCastSharedPtr<FAnimNextExecuteProgramTask>(MixerTask);
				EvalProgramTask->Program = Mixer->EvaluationProgram;
			}
			else
			{
				MixerTask = nullptr;
			}
		}
	};
}

FAnimNextBlendTwoKeyframesPreserveRootMotionTask FAnimNextBlendTwoKeyframesPreserveRootMotionTask::Make(float InterpolationAlpha)
{
	FAnimNextBlendTwoKeyframesPreserveRootMotionTask Task;
	Task.InterpolationAlpha = InterpolationAlpha;
	return Task;
}

void FAnimNextBlendTwoKeyframesPreserveRootMotionTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		TUniquePtr<FValueBundle> DiscardedCollection;
		(void)VM.PopValue(ATTRIBUTE_STACK_NAME, DiscardedCollection);
		return;
	}

	const UE::Anim::FAttributeId RootMotionDeltaAttributeId(UE::Anim::IAnimRootMotionProvider::AttributeName, FCompactPoseBoneIndex(0));

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	TOptional<FTransform> OverrideRootTransform;
	TOptional<float>      OverrideRootWeight;
	TOptional<FTransform> OverrideRootDeltaTransform;
	bool                  bIsAuthoritative = false;

	// This task is used to perform a two way blend between 2 poses without blending root motion attributes
	//    if they do not exist in either pose. By default these attributes would blend with the identity matrix
	//    in a way that would cause undesirable effects when blending sequencer's pose with the incoming pose
	//    from the upstream graph.

	//        [ Sequencer pose A           ]  [ Sequencer poseB            ]
	//        |    + RootMotionTransform   |  |                            |
	//        [____________________________]  [____________________________]
	//                                 \           /
	//                                  \  Blend  /
	//                                   \       /
	//                                    \     /
	//                                     \   /
	//                                      \ /
	//                       [ Final Sequencer Pose       ]           [ Upstream pose (locomotion) ]
	//                       |  + RootMotionTransform     |           |    + RootMotionDelta       |
	//                       [____________________________]           [____________________________]
	//                                                    \           /
	//                                                     \  Blend  /
	//                                                      \       /
	//                                                       \     /
	//                                                        \   /
	//                                                         \ /
	//                                           [ Final Pose               ]
	//                                           |    + RootMotionTransform |
	//                                           |    + RootMotionDelta     |
	//                                           [__________________________]
	// From this final pose, FAnimNextStoreRootTransformTask is able to accurately read both Sequencer's desired world space
	//    root transform, its desired weight, and the incoming desired root motion delta from locomotion. It can then blend
	//    all these things together to form the final root motion delta which can be consumed by external systems like Mover
	//
	// If there are competing sources of Sequencer root motion, one may be authoritative, so blending will also be skipped
	//    in this case, but only for the RootMotionTransform, since Sequencer doesn't directly write to RootMotionDelta

	{
		const TUniquePtr<FKeyframeState>* KeyframeA = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
		const TUniquePtr<FKeyframeState>* KeyframeB = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 1);
		if (KeyframeA && KeyframeB)
		{
			UE::Anim::FStackAttributeContainer& AttributesA = (*KeyframeA)->Attributes;
			UE::Anim::FStackAttributeContainer& AttributesB = (*KeyframeB)->Attributes;

			const FTransformAnimationAttribute* RootMotionTransformA = AttributesA.Find<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
			const FTransformAnimationAttribute* RootMotionDeltaA     = AttributesA.Find<FTransformAnimationAttribute>(RootMotionDeltaAttributeId);
			const FIntegerAnimationAttribute* RootMotionIsAuthoritativeA = AttributesA.Find<FIntegerAnimationAttribute>(AnimMixerComponents->RootTransformIsAuthoritativeAttributeId);

			const FTransformAnimationAttribute* RootMotionTransformB = AttributesB.Find<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
			const FTransformAnimationAttribute* RootMotionDeltaB     = AttributesB.Find<FTransformAnimationAttribute>(RootMotionDeltaAttributeId);
			const FIntegerAnimationAttribute* RootMotionIsAuthoritativeB = AttributesB.Find<FIntegerAnimationAttribute>(AnimMixerComponents->RootTransformIsAuthoritativeAttributeId);

			// If Keyframe A is authoritative, or there is no transform from B, preserve only A. Otherwise it would be blended with identity, or non-authoritative source of root motion.
			const bool bAIsAuthoritative = RootMotionTransformA && RootMotionIsAuthoritativeA && !RootMotionIsAuthoritativeB;
			const bool bPreserveTransformA =  bAIsAuthoritative || (RootMotionTransformA && !RootMotionTransformB);

			// Only preserve the transform from B if it is authoritative.
			const bool bPreserveTransformB = RootMotionTransformB && RootMotionIsAuthoritativeB && !RootMotionIsAuthoritativeA;
			
			if (bPreserveTransformA)
			{
				OverrideRootTransform = RootMotionTransformA->Value;
			}
			else if (bPreserveTransformB)
			{
				OverrideRootTransform = RootMotionTransformB->Value;
			}
			
			if (!RootMotionDeltaA && RootMotionDeltaB)
			{
				// Preserve root motion *delta* from B if it's not in A (ie, don't blend it with identity!)
				OverrideRootDeltaTransform = RootMotionDeltaB->Value;
			}

			bIsAuthoritative = RootMotionIsAuthoritativeA || RootMotionIsAuthoritativeB;
		}
	}

	// Do the actual blend
	FAnimNextBlendTwoKeyframesTask::Execute(VM);

	if (OverrideRootTransform || OverrideRootDeltaTransform)
	{
		TUniquePtr<FKeyframeState> Keyframe;
		if (VM.PopValue(KEYFRAME_STACK_NAME, Keyframe) && Keyframe)
		{
			if (OverrideRootTransform)
			{
				Keyframe->Attributes.FindOrAdd<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId)->Value
					= OverrideRootTransform.GetValue();
			}
			if (OverrideRootDeltaTransform)
			{
				Keyframe->Attributes.FindOrAdd<FTransformAnimationAttribute>(RootMotionDeltaAttributeId)->Value
					= OverrideRootDeltaTransform.GetValue();
			}
			if (bIsAuthoritative)
			{
				Keyframe->Attributes.FindOrAdd<FIntegerAnimationAttribute>(AnimMixerComponents->RootTransformIsAuthoritativeAttributeId)->Value
					= 1;
			}

			VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
		}
	}
}


FMovieSceneAccumulateAbsoluteBlendTask FMovieSceneAccumulateAbsoluteBlendTask::Make(float ScaleFactor)
{
	FMovieSceneAccumulateAbsoluteBlendTask Task;
	Task.ScaleFactor = ScaleFactor;

	return Task;
}

void FMovieSceneAccumulateAbsoluteBlendTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		TUniquePtr<FValueBundle> DiscardedCollection;
		(void)VM.PopValue(ATTRIBUTE_STACK_NAME, DiscardedCollection);
		return;

		// TODO: Is this task any different from the normal Accumulate task?
	}

	// Pop our top two poses, we'll re-use the top keyframe for our result

	TUniquePtr<FKeyframeState> KeyframeA;
	if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeA))
	{
		// We have no inputs, nothing to do
		return;
	}

	TUniquePtr<FKeyframeState> KeyframeB;
	if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeB))
	{
		// We have a single input, leave it on top of the stack
		VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeA));
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		check(KeyframeA->Pose.GetNumBones() == KeyframeB->Pose.GetNumBones());

		BlendAddWithScale(KeyframeB->Pose.LocalTransforms.GetView(), KeyframeA->Pose.LocalTransforms.GetConstView(), ScaleFactor);
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
	{
		KeyframeB->Curves.Accumulate(KeyframeA->Curves, ScaleFactor);
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
	{
		UE::Anim::Attributes::AccumulateAttributes(KeyframeA->Attributes, KeyframeB->Attributes, ScaleFactor, AAT_None);
	}

	VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeB));
}

UMovieSceneAnimMixerSystem::UMovieSceneAnimMixerSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	RelevantComponent = AnimMixerComponents->Task;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;
	SystemCategories = EEntitySystemCategory::BlenderSystems;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBoundSceneComponentInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneComponentTransformSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneTransformOriginSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UByteChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObjectKey);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentConsumer(GetClass(), AnimMixerComponents->Task);
		DefineComponentConsumer(GetClass(), AnimMixerComponents->Target);
		DefineComponentConsumer(GetClass(), AnimMixerComponents->Priority);
		DefineComponentConsumer(GetClass(), AnimMixerComponents->MirrorTable);
		DefineComponentProducer(GetClass(), AnimMixerComponents->MixerTask);
	}
}

TSharedPtr<FMovieSceneMixerRootMotionComponentData> UMovieSceneAnimMixerSystem::FindRootMotion(FObjectKey InObject) const
{
	TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = RootMotionByObject.FindRef(InObject).Pin();
	if (RootMotion)
	{
		return RootMotion;
	}
	
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InObject.ResolveObjectPtr()))
	{
		AActor* Actor = SceneComponent ? SceneComponent->GetOwner() : nullptr;
		if (!Actor || SceneComponent != Actor->GetRootComponent())
		{
			return nullptr;
		}

		FObjectKey ActorKey(Actor);
		for (auto It = ActorToRootMotion.CreateConstKeyIterator(ActorKey); It; ++It)
		{
			TSharedPtr<FMovieSceneMixerRootMotionComponentData> ThisRootMotion = It.Value().Pin();
			if (ThisRootMotion && ThisRootMotion->RootDestination == EMovieSceneRootMotionDestination::Actor)
			{
				return ThisRootMotion;
			}
		}
	}
	return nullptr;
}

TSharedPtr<FMovieSceneAnimMixer> UMovieSceneAnimMixerSystem::FindMixer(const FMovieSceneAnimMixerKey& InKey) const
{
	return Mixers.FindRef(InKey);
}

TArray<FSequencerMixerPendingNotifyBatch> UMovieSceneAnimMixerSystem::ConsumeMixerNotifyBatches(
	UMovieSceneEntitySystemLinker* Linker,
	UObject* BoundObject,
	const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target)
{
	if (!Linker)
	{
		return {};
	}
	UMovieSceneAnimMixerSystem* MixerSystem = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
	if (!MixerSystem)
	{
		return {};
	}
	TSharedPtr<FMovieSceneAnimMixer> Mixer = MixerSystem->FindMixer(FMovieSceneAnimMixerKey({ FObjectKey(BoundObject), Target }));
	if (!Mixer)
	{
		return {};
	}
	return MoveTemp(Mixer->PendingNotifyBatches);
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimMixerSystem::FindMixerLayer(
	TFunctionRef<bool(UMovieSceneAnimationMixerLayer*)> Predicate) const
{
	for (const auto& [Key, Mixer] : Mixers)
	{
		if (!Mixer)
		{
			continue;
		}
		UMovieSceneAnimationMixerTrack* MixerTrack = Mixer->OwnerTrack.Get();
		if (!MixerTrack)
		{
			continue;
		}
		for (UMovieSceneAnimationMixerLayer* Layer : MixerTrack->GetLayers())
		{
			if (Layer && Predicate(Layer))
			{
				return Layer;
			}
		}
	}
	return nullptr;
}

void UMovieSceneAnimMixerSystem::AssignRootMotion(FObjectKey InObject, TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion)
{
	if (RootMotion)
	{
		RootMotionByObject.Add(InObject, RootMotion);
		USceneComponent* SceneComponent = Cast<USceneComponent>(InObject.ResolveObjectPtr());
		AActor* Actor = SceneComponent ? SceneComponent->GetOwner() : nullptr;
		if (Actor && !ActorToRootMotion.FindPair(Actor, RootMotion))
		{
			ActorToRootMotion.Add(Actor, RootMotion);
		}
	}
	else
	{
		RootMotionByObject.Remove(InObject);
	}
}

void UMovieSceneAnimMixerSystem::PreInitializeAllRootMotion()
{
	for (auto It = RootMotionByObject.CreateIterator(); It; ++It)
	{
		TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = It.Value().Pin();
		if (RootMotion)
		{
			RootMotion->bActorTransformSet = false;
		}
	}
}

UMovieSceneAnimMixerSystem::FBusTopologyCache& UMovieSceneAnimMixerSystem::GetOrCreateBusTopology(FObjectKey ObjectKey)
{
	return BusTopologyByObject.FindOrAdd(ObjectKey);
}

namespace UE::MovieScene::Private
{
	// Game-thread-only counter - all mixer program building happens on the game thread.
	static int32 GForceRootBoneDestinationScopeCount = 0;
}

void UMovieSceneAnimMixerSystem::PushForceRootBoneDestinationScope()
{
	check(IsInGameThread());
	++UE::MovieScene::Private::GForceRootBoneDestinationScopeCount;
}

void UMovieSceneAnimMixerSystem::PopForceRootBoneDestinationScope()
{
	check(IsInGameThread());
	check(UE::MovieScene::Private::GForceRootBoneDestinationScopeCount > 0);
	--UE::MovieScene::Private::GForceRootBoneDestinationScopeCount;
}

bool UMovieSceneAnimMixerSystem::IsForceRootBoneDestinationScopeActive()
{
	return UE::MovieScene::Private::GForceRootBoneDestinationScopeCount > 0;
}

void UMovieSceneAnimMixerSystem::RemoveStaleBusTopologies(const TSet<FObjectKey>& ActiveObjects)
{
	for (auto It = BusTopologyByObject.CreateIterator(); It; ++It)
	{
		if (!ActiveObjects.Contains(It.Key()))
		{
			It.RemoveCurrent();
		}
	}
}

void UMovieSceneAnimMixerSystem::InitializeAllRootMotion()
{
	for (auto It = RootMotionByObject.CreateIterator(); It; ++It)
	{
		TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = It.Value().Pin();
		if (RootMotion)
		{
			RootMotion->Initialize();
		}
	}
}

void UMovieSceneAnimMixerSystem::StoreBusData(FObjectKey BoundObject, FName BusName, TSharedPtr<FMovieSceneAnimBusData> Data)
{
	FMovieSceneAnimBusStorageKey Key;
	Key.BoundObjectKey = BoundObject;
	Key.BusName = BusName;
	BusStorage.Add(Key, MoveTemp(Data));
}

TSharedPtr<FMovieSceneAnimBusData> UMovieSceneAnimMixerSystem::ReadBusData(FObjectKey BoundObject, FName BusName) const
{
	FMovieSceneAnimBusStorageKey Key;
	Key.BoundObjectKey = BoundObject;
	Key.BusName = BusName;
	const TSharedPtr<FMovieSceneAnimBusData>* Found = BusStorage.Find(Key);
	return Found ? *Found : nullptr;
}

void UMovieSceneAnimMixerSystem::ClearBusStorage()
{
	BusStorage.Reset();
}

bool UMovieSceneAnimMixerSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return Mixers.Num() != 0;
}

void UMovieSceneAnimMixerSystem::OnLink()
{
	RootMotionSystem = Linker->LinkSystem<UMovieSceneRootMotionSystem>();
	Linker->SystemGraph.AddReference(this, RootMotionSystem);

	// Force-link the pre-animated state cache when we come online so it's in the
	// graph before the Instantiation phase starts. Without this, the cache only
	// links when RestoreState-tagged entities first appear - which can happen
	// mid-phase as section entities are imported - and the cache gets sorted
	// upstream of us after we're already running, tripping the in-flight ensure.
	UMovieSceneCachePreAnimatedStateSystem* CacheSystem = Linker->LinkSystem<UMovieSceneCachePreAnimatedStateSystem>();
	Linker->SystemGraph.AddReference(this, CacheSystem);

	PreFlushHandle = Linker->Events.PreFlush.AddUObject(this, &UMovieSceneAnimMixerSystem::HandlePreFlush);
}

void UMovieSceneAnimMixerSystem::OnUnlink()
{
	if (PreFlushHandle.IsValid() && Linker)
	{
		Linker->Events.PreFlush.Remove(PreFlushHandle);
		PreFlushHandle.Reset();
	}
}

void UMovieSceneAnimMixerSystem::HandlePreFlush(UMovieSceneEntitySystemLinker* InLinker)
{
	using namespace UE::MovieScene;

	if (bInsidePreFlush)
	{
		return;
	}
	TGuardValue<bool> ReentrancyGuard(bInsidePreFlush, true);

	// Don't rebuild the offset cache while a bake scope is active (e.g.
	// from GetOrEvaluateSectionPose during Render). The rebuild creates
	// nested FBakeEvaluationScopes that save/restore the outer scope's
	// bake target instead of the real original target, corrupting results.
	// The cache will be rebuilt on the next non-bake flush instead.
	if (UMovieSceneMixerBakeTargetSystem* BakeSystem = InLinker->FindSystem<UMovieSceneMixerBakeTargetSystem>())
	{
		if (BakeSystem->bBakeActive)
		{
			return;
		}
	}

	// Collect dirty tracks before rebuilding. The rebuild triggers re-entrant
	// flushes (via bake evaluation) which can add/remove Mixers map entries,
	// so we must not iterate the map during the rebuild.
	TArray<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>> DirtyTracks;
	TArray<FInstanceHandle> DirtyHandles;

	for (TPair<FMovieSceneAnimMixerKey, TSharedPtr<FMovieSceneAnimMixer>>& Pair : Mixers)
	{
		if (!Pair.Value)
		{
			continue;
		}
		UMovieSceneAnimationMixerTrack* Track = Pair.Value->OwnerTrack.Get();
		if (!Track || !Track->HasDirtyAccumulatedOffsetCache())
		{
			continue;
		}

		for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : Pair.Value->WeakEntries)
		{
			if (TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin())
			{
				if (Entry->InstanceHandle.IsValid())
				{
					// Resolve to the root instance handle. The bake evaluation
					// flushes the ECS which only supports root instance updates.
					FInstanceHandle Handle = Entry->InstanceHandle;
					const FSequenceInstance& Instance = InLinker->GetInstanceRegistry()->GetInstance(Handle);
					FRootInstanceHandle RootHandle = Instance.GetRootInstanceHandle();

					DirtyTracks.Add(Track);
					DirtyHandles.Add(RootHandle);
					break;
				}
			}
		}
	}

	for (int32 Index = 0; Index < DirtyTracks.Num(); ++Index)
	{
		if (UMovieSceneAnimationMixerTrack* Track = DirtyTracks[Index].Get())
		{
			Track->RebuildDirtyAccumulatedOffsetCache(InLinker, DirtyHandles[Index]);
		}
	}
}

void UMovieSceneAnimMixerSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
		FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

		// ----------------------------------------------------------------------------------------------------------------------------------------
		// Step 1 - Remove expired mixer entries.
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObjectKey)
		.Write(AnimMixerComponents->MixerEntry)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, AnimMixerComponents->Task })
		.Iterate_PerEntity(&Linker->EntityManager,
			[this](FObjectKey BoundObjectKey, TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry)
			{
				if (MixerEntry)
				{
					if (TSharedPtr<FMovieSceneAnimMixer> Mixer = MixerEntry->WeakMixer.Pin())
					{
						Mixer->bNeedsResort = true;
					}
				}
				MixerEntry = nullptr;
			}
		);

		// ----------------------------------------------------------------------------------------------------------------------------------------
		// Step 2 - Create new mixer entries for new anim tasks, gathering any new mixer entities that need to be created
		TArray<FMovieSceneAnimMixerKey> NewMixers;

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObjectKey)
		.Write(AnimMixerComponents->Target)
		.Write(AnimMixerComponents->MixerEntry)
		.Read(AnimMixerComponents->Task)
		.Read(AnimMixerComponents->Priority)
		.ReadOptional(TrackComponents->SkeletalAnimation)
		.ReadOptional(AnimMixerComponents->RootMotionSettings)
		.ReadOptional(AnimMixerComponents->MirrorTable)
		.ReadOptional(AnimMixerComponents->EntityOwner)
		.ReadOptional(AnimMixerComponents->MixerLayer)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink})
		.Iterate_PerEntity( &Linker->EntityManager,
		[this, &NewMixers, BuiltInComponents, AnimMixerComponents]
		(
			FMovieSceneEntityID EntityID,
			FInstanceHandle InstanceHandle,
			FObjectKey BoundObjectKey,
			TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
			TSharedPtr<FMovieSceneAnimMixerEntry>& InOutMixerEntry,
			TSharedPtr<FAnimNextEvaluationTask> Task,
			int32 Priority,
			const FMovieSceneSkeletalAnimationComponentData* SkelAnimData,
			const FMovieSceneRootMotionSettings* RootMotionSettings,
			const TObjectPtr<UMirrorDataTable>* OptMirrorDataTable,
			const FObjectKey* EntityOwner,
			const TObjectPtr<UMovieSceneAnimationMixerLayer>* MixerLayer)
			{
				// For new entities, we 'resolve' the animation target so if 'Automatic' is picked we choose the right one automatically.
				Target = ResolveAnimationTarget(BoundObjectKey, Target, SkelAnimData);
				FMovieSceneAnimMixerKey Key({ BoundObjectKey, Target });
				TSharedPtr<FMovieSceneAnimMixer> Mixer = Mixers.FindRef(Key);
				if (!Mixer)
				{
					Mixer = MakeShared<FMovieSceneAnimMixer>();
					Mixers.Add(Key, Mixer);
					NewMixers.Add(Key);
				}
				else if (Mixer->WeakEntries.Num() == 0)
				{
					NewMixers.Add(Key);
				}

				// Create a new mixer entry if necessary
				if (!InOutMixerEntry)
				{
					InOutMixerEntry = MakeShared<FMovieSceneAnimMixerEntry>();
				}

				FMovieSceneAnimMixerEntry* Entry = InOutMixerEntry.Get();

				Entry->InstanceHandle = InstanceHandle;
				Entry->EntityID = EntityID;
				Entry->EvalTask = Task;
				Entry->Priority = Priority;
				Entry->CombinedWeight = 0;
				Entry->EasingWeight = 0;
				Entry->SectionWeight = 1.0;
				Entry->MixerLayer = (MixerLayer && MixerLayer->Get()) ? *MixerLayer : nullptr;
				Entry->EntityOwner = EntityOwner ? *EntityOwner : FObjectKey();
				if (RootMotionSettings)
				{
					Entry->RootMotionSettings = *RootMotionSettings;

					// If we know we'll need root motion, ensure it is set up correctly with a lifetime reference that keeps it
					//     alive as long as this entry
					TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = FindRootMotion(BoundObjectKey);
					if (!RootMotion)
					{
						RootMotion = MakeShared<FMovieSceneMixerRootMotionComponentData>();

						RootMotion->RootDestination = EMovieSceneRootMotionDestination::RootBone;
						RootMotion->OriginalBoundObject = Cast<USceneComponent>(BoundObjectKey.ResolveObjectPtr());
						AssignRootMotion(BoundObjectKey, RootMotion);
					}

					// Overwrite the RootDestination for the root motion if we have a legacy setting.
					//   If any actual RootDestination components exist, they will simply override this on eval
					switch (RootMotionSettings->LegacySwapRootBone)
					{
					case ESwapRootBone::SwapRootBone_None:
						break;
					case ESwapRootBone::SwapRootBone_Component:
						RootMotion->RootDestination = EMovieSceneRootMotionDestination::Component;
						break;
					case ESwapRootBone::SwapRootBone_Actor:
						RootMotion->RootDestination = EMovieSceneRootMotionDestination::Actor;
						break;
					}

					Entry->RootMotionLifetimeReference = RootMotion;
				}
				else
				{
					Entry->RootMotionLifetimeReference = nullptr;
				}
				const FComponentMask& Type = Linker->EntityManager.GetEntityType(EntityID);
				Entry->bAdditive = Type.Contains(BuiltInComponents->Tags.AdditiveAnimation);
				Entry->bSkipPoseWeight = Type.Contains(AnimMixerComponents->Tags.SkipPoseWeight);
				Entry->bPreBakeRootMotion = Type.Contains(AnimMixerComponents->Tags.PreBakeRootMotion);
				Entry->MirrorTable = OptMirrorDataTable ? *OptMirrorDataTable : nullptr;
				Entry->bIsTransition = Type.Contains(AnimMixerComponents->Tags.Transition);

				TSharedPtr<FMovieSceneAnimMixer> ExistingMixer = Entry->WeakMixer.Pin();
				if (ExistingMixer)
				{
					if (ExistingMixer != Mixer)
					{
						ExistingMixer->WeakEntries.Remove(InOutMixerEntry);
						Entry->WeakMixer = nullptr;

						Mixer->WeakEntries.Emplace(InOutMixerEntry);
						Entry->WeakMixer = Mixer;
					}
				}
				else
				{
					Mixer->WeakEntries.Emplace(InOutMixerEntry);
					Entry->WeakMixer = Mixer;
				}

				Mixer->bNeedsResort = true;

				if (!Mixer->OwnerTrack.IsValid())
				{
					if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(Entry->EntityOwner.ResolveObjectPtr()))
					{
						Mixer->OwnerTrack = Section->GetTypedOuter<UMovieSceneAnimationMixerTrack>();
					}
				}
			});

		// ----------------------------------------------------------------------------------------------------------------------------------------
		// Step 3 - Create new mixer entities
		for (const FMovieSceneAnimMixerKey& NewMixerKey : NewMixers)
		{
			TSharedPtr<FMovieSceneAnimMixer> NewMixer = Mixers.FindChecked(NewMixerKey);
			NewMixer->MixerEntityID = FEntityBuilder()
				.Add(AnimMixerComponents->MeshComponent, FObjectComponent::Weak(NewMixerKey.BoundObjectKey.ResolveObjectPtr()))
				.Add(AnimMixerComponents->Target, NewMixerKey.Target)
				.Add(AnimMixerComponents->MixerTask, nullptr)
				.AddTag(BuiltInComponents->Tags.RestoreState) // TODO: For now we always restore state on the mixer when it gets unlinked.
				.CreateEntity(&Linker->EntityManager);
		}

		auto SortPredicate = [](const TWeakPtr<FMovieSceneAnimMixerEntry>& A, const TWeakPtr<FMovieSceneAnimMixerEntry>& B)
		{
			return *A.Pin() < *B.Pin();
		};

		// ----------------------------------------------------------------------------------------------------------------------------------------
		// Step 4 - Update mixer entities, and remove stale ones. 
		for (auto It = Mixers.CreateIterator(); It; ++It)
		{
			TSharedPtr<FMovieSceneAnimMixer> Mixer = It.Value();

			if (Mixer->bNeedsResort)
			{
				bool bNeedsRootMotion = false;

				// Remove nullptrs
				for (int32 Index = Mixer->WeakEntries.Num()-1; Index >= 0; --Index)
				{
					if (Mixer->WeakEntries[Index].Pin() == nullptr)
					{
						Mixer->WeakEntries.RemoveAtSwap(Index, 1, EAllowShrinking::No);
					}
				}

				Algo::Sort(Mixer->WeakEntries, SortPredicate);
				Mixer->bNeedsResort = false;
			}

			if (Mixer->WeakEntries.IsEmpty())
			{
				Mixer->EvaluationProgram = nullptr;

				if (Mixer->MixerEntityID)
				{
					Linker->EntityManager.AddComponent(Mixer->MixerEntityID, BuiltInComponents->Tags.NeedsUnlink, EEntityRecursion::Full);
					Mixer->MixerEntityID = FMovieSceneEntityID();
				}

				It.RemoveCurrent();
			}
		}

		for (auto It = RootMotionByObject.CreateIterator(); It; ++It)
		{
			if (It.Value().Pin() == nullptr)
			{
				It.RemoveCurrent();
			}
		}
		for (auto It = ActorToRootMotion.CreateIterator(); It; ++It)
		{
			if (It.Value().Pin() == nullptr)
			{
				It.RemoveCurrent();
			}
		}

		// ----------------------------------------------------------------------------------------------------------------------------------------
		// Step 5. Mutate transforms as necessary to include their root motion
		struct FMutateTransforms : IMovieSceneConditionalEntityMutation
		{
			const TMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* RootMotionByObject;
			const TMultiMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* ActorToRootMotion;

			FComponentTypeID AnimMixerPoseProducer;

			FMutateTransforms(
				const TMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* InRootMotionByObject,
				const TMultiMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>>* InActorToRootMotion)
				: RootMotionByObject(InRootMotionByObject)
				, ActorToRootMotion(InActorToRootMotion)
				, AnimMixerPoseProducer(FMovieSceneTracksComponentTypes::Get()->Tags.AnimMixerPoseProducer)
			{
			}

			virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
			{
				if (InOutEntityComponentTypes->Contains(AnimMixerPoseProducer))
				{
					InOutEntityComponentTypes->Remove(AnimMixerPoseProducer);
				}
				else
				{
					InOutEntityComponentTypes->Set(AnimMixerPoseProducer);
				}
			}
			virtual void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const
			{
				TComponentReader<UObject*> BoundObjects = Allocation->ReadComponents(FBuiltInComponentTypes::Get()->BoundObject);
				const bool bAlreadyRedirectingToRoot = Allocation->HasComponent(AnimMixerPoseProducer);

				const int32 NumEntities = Allocation->Num();
				for (int32 Index = 0; Index < NumEntities; ++Index)
				{
					USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObjects[Index]);

					if (!SceneComponent)
					{
						continue;
					}

					bool bNeedsTracking = RootMotionByObject->Contains(SceneComponent);
					if (!bNeedsTracking)
					{
						if (AActor* Owner = SceneComponent->GetOwner())
						{
							bNeedsTracking = Owner->GetRootComponent() == SceneComponent && ActorToRootMotion->Contains(Owner);
						}
					}

					if (bAlreadyRedirectingToRoot != bNeedsTracking)
					{
						OutEntitiesToMutate.PadToNum(Index+1, false);
						OutEntitiesToMutate[Index] = true;
					}
				}
			}
		};

		FMutateTransforms Mutation(&RootMotionByObject, &ActorToRootMotion);
		FEntityComponentFilter Filter;
		Filter.All({ BuiltInComponents->BoundObject, TrackComponents->ComponentTransform.PropertyTag, BuiltInComponents->CustomPropertyIndex });
		Linker->EntityManager.MutateConditional(Filter, Mutation);
	}
}

void UMovieSceneAnimMixerSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// TODO: We should be able to optimize here if we figure out which tasks won't contribute to the final pose. This should be doable 
	// with a little extra API and pre-examination of per-bone blend weights, etc.

	// Update mixer entry tasks- todo this is slow with the current data hierarchy
	// Maybe we want a flat map of entity id to entry, and then put indices into that into the mixer or something similar.
	FTaskID UpdateTask = FEntityTaskBuilder()
		.Read(AnimMixerComponents->Task)
		.Write(AnimMixerComponents->MixerEntry)
		.ReadOptional(AnimMixerComponents->RootMotionSettings)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.ReadOptional(BuiltInComponents->EasingResult)
		.ReadOptional(BuiltInComponents->WeightResult)
		.ReadOptional(AnimMixerComponents->BoneMatchTransform)
		.ReadOptional(BuiltInComponents->EvalTime)
		.FilterNone({BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Ignored})
		.Schedule_PerEntity<FUpdateTaskEntities>(&Linker->EntityManager, TaskScheduler);

	// For each mixer, build the evaluation program task list.
	FTaskID MixTask = FEntityTaskBuilder()
		.Read(AnimMixerComponents->MeshComponent)
		.Read(AnimMixerComponents->Target)
		.Write(AnimMixerComponents->MixerTask)
		.FilterNone({BuiltInComponents->Tags.Ignored})
		.Schedule_PerEntity<FEvaluateAnimMixers>(&Linker->EntityManager, TaskScheduler, &Mixers, &RootMotionByObject, Linker, this);

	TaskScheduler->AddPrerequisite(UpdateTask, MixTask);
}

void UMovieSceneAnimMixerSystem::OnCleanTaggedGarbage()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.Write(AnimMixerComponents->MixerEntry)
	.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, AnimMixerComponents->Task })
	.Iterate_PerEntity(&Linker->EntityManager,
		[this](FObjectKey BoundObjectKey, TSharedPtr<FMovieSceneAnimMixerEntry>& MixerEntry)
		{
			if (MixerEntry)
			{
				TSharedPtr<FMovieSceneAnimMixer> Mixer = MixerEntry->WeakMixer.Pin();
				if (Mixer)
				{
					Mixer->bNeedsResort = true;
				}
			}
			MixerEntry = nullptr;
		}
	);

	FEntityTaskBuilder()
	.Read(AnimMixerComponents->MeshComponent)
	.Read(AnimMixerComponents->Target)
	.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, AnimMixerComponents->MixerTask })
	.Iterate_PerEntity(&Linker->EntityManager,
		[this](FObjectComponent MeshComponent, const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target)
		{
			RootMotionByObject.Remove(MeshComponent.GetObject());
			Mixers.Remove(FMovieSceneAnimMixerKey({ MeshComponent.GetObject(), Target }));
		}
	);
}

TInstancedStruct<FMovieSceneMixedAnimationTarget> UMovieSceneAnimMixerSystem::ResolveAnimationTarget(FObjectKey ObjectKey, const TInstancedStruct<FMovieSceneMixedAnimationTarget>& InTarget, const FMovieSceneSkeletalAnimationComponentData* SkelAnimData)
{
	// If user has selected the default 'automatic' target, attempt to choose one automatically for them.
	if (!InTarget.IsValid() || InTarget.GetScriptStruct() == FMovieSceneMixedAnimationTarget::StaticStruct())
	{
		if (UObject* Object = ObjectKey.ResolveObjectPtr())
		{
			USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Object);
			UUAFComponent* AnimNextComponent = nullptr;
			if (!SkeletalMeshComponent)
			{
				if (AActor* Actor = Cast<AActor>(Object))
				{
					SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>();
					AnimNextComponent = SkeletalMeshComponent->GetOwner()->FindComponentByClass<UUAFComponent>();
				}
			}
			else 
			{
				AnimNextComponent = SkeletalMeshComponent->GetOwner()->FindComponentByClass<UUAFComponent>();
			}

			if (AnimNextComponent != nullptr && (!SkeletalMeshComponent || !SkeletalMeshComponent->bEnableAnimation))
			{
				// If we have an anim next component and the skeletal mesh component has animation disabled, default to anim next target
				return TInstancedStruct<FMovieSceneAnimNextInjectionTarget>::Make();
			}
			else if (SkeletalMeshComponent)
			{
				// Backwards compatibility- are they forcing custom mode?
				const bool bForceCustomMode = SkelAnimData != nullptr && SkelAnimData->Section && SkelAnimData->Section->Params.bForceCustomMode;

				if (!bForceCustomMode)
				{
					if (UAnimInstance* AnimInstance = SkeletalMeshComponent->GetAnimInstance())
					{
						if (const FAnimSubsystem_SequencerMixer* MixerSubsystem = AnimInstance->FindSubsystem<FAnimSubsystem_SequencerMixer>())
						{
							FName BlueprintNodeName = FAnimNode_SequencerMixerTarget::DefaultTargetName;
						
							// Backwards compatibility- if we have skel anim data with a 'SlotName', use that as the name.
							if (SkelAnimData != nullptr && SkelAnimData->Section && !SkelAnimData->Section->Params.SlotName.IsNone())
							{
								BlueprintNodeName = SkelAnimData->Section->Params.SlotName;
							}
						
							// We have an anim blueprint with sequencer mixer target node(s). Create a target using the default target name.
							return TInstancedStruct<FMovieSceneAnimBlueprintTarget>::Make((FMovieSceneAnimBlueprintTarget(BlueprintNodeName)));
						}
					}
				}

				// Fallback to using a custom anim instance as the target
				return TInstancedStruct<FMovieSceneAnimInstanceTarget>::Make();
			}
		}
	}
	return InTarget;
}