// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAnimNextTargetSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneTracksComponentTypes.h"
#include "SkeletalMeshRestoreState.h"
#include "MovieSceneAnimMixerModule.h"

#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "EvaluationVM/EvaluationVM.h"
#include "GenerationTools.h"

#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/MovieSceneObjectPropertySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"

#include "Animation/AnimationPoseData.h"
#include "BonePose.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "AnimMixerComponentTypes.h"
#include "MovieSceneAnimMixerSettings.h"
#include "AnimMixerUtils.h"

#include "Graph/AnimNext_LODPose.h"
#include "Injection/InjectionUtils.h"
#include "TraitInterfaces/IEvaluate.h"

#include "PrimitiveSceneProxy.h"
#include "Module/AnimNextModule.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimNextTargetSystem)

namespace UE::MovieScene
{

	/** Pre-animated state for skeletal animations */
	struct FPreAnimatedAnimNextState
	{
		// Handle to the injection request instance (used to uninject when animation done)
		UE::UAF::FInjectionRequestPtr InjectionHandle;

		TObjectPtr<UMovieSceneAnimNextTargetSystem> System;
	
		// Pointer to the UAF Component's module. Since the system sets the value on the component, this is a strong ptr to prevent GC. 	
		TStrongObjectPtr<const UUAFSystem> PreAnimatedModule;

		// Whether the anim next module started enabled or not.
		bool bAnimNextModuleEnabled = false;

		// Whether evaluation changed the module, and therefore needs to be reset.
		bool bModuleReplaced = false;
	};

	/** Pre-animation traits for skeletal animations */
	struct FPreAnimatedAnimNextMixedSkeletalAnimationTraits : FBoundObjectPreAnimatedStateTraits
	{
		using KeyType = TTuple<FObjectKey, FAnimNextVariableReference>;
		using StorageType = FPreAnimatedAnimNextState;

		void RestorePreAnimatedValue(const KeyType& ObjectAndInjectionSite, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
		{
			// Uninject
			if (InOutCachedValue.InjectionHandle.IsValid())
			{
				if (InOutCachedValue.System)
				{
					InOutCachedValue.System->CurrentTargets.RemoveAll([&InOutCachedValue](const FMovieSceneAnimNextTargetData& TargetData) { return TargetData.InjectionRequestHandle == InOutCachedValue.InjectionHandle;});
				}
				UE::UAF::FInjectionUtils::Uninject(InOutCachedValue.InjectionHandle);
			}

			if (InOutCachedValue.bModuleReplaced)
			{
				if (UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(ObjectAndInjectionSite.Key.ResolveObjectPtr()))
				{
					// TODO the animation needs to be updated, or the post needs to be reset to ref pose when restoring to a previous module.
					// Currently there is no support for this in UAF.
					// This is most pronounced in the editor, where the mesh stays in the same pose when the sequence is closed.
					FAnimMixerUtils::UnregisterUAFSystem(AnimNextComponent);
					FAnimMixerUtils::SetUAFComponentModule(AnimNextComponent, InOutCachedValue.PreAnimatedModule.Get());
					FAnimMixerUtils::RegisterUAFSystem(AnimNextComponent);
				}
			}
			
			// Reset the anim next module back to not being enabled if applicable
			if (!InOutCachedValue.bAnimNextModuleEnabled)
			{
				if (UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(ObjectAndInjectionSite.Key.ResolveObjectPtr()))
				{
					AnimNextComponent->SetActive(false);
				}
			}
		}
	};
	struct FPreAnimatedSkelMeshComponentTraits : FBoundObjectPreAnimatedStateTraits
	{
		using KeyType = FObjectKey;
		using StorageType = FSkeletalMeshRestoreState;

		StorageType CachePreAnimatedValue(const KeyType& Object)
		{
			USkeletalMeshComponent* SkelMeshComp = CastChecked<USkeletalMeshComponent>(Object.ResolveObjectPtr());

			StorageType State;
			State.SaveState(SkelMeshComp);
			if (FPrimitiveSceneProxy* Proxy = SkelMeshComp->GetSceneProxy())
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Proxy->SetCanSkipRedundantTransformUpdates(false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			return State;
		}

		void RestorePreAnimatedValue(const KeyType& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
		{
			USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
			if (SkelMeshComp != nullptr && SkelMeshComp->GetSceneProxy())
			{
				if (FPrimitiveSceneProxy* Proxy = SkelMeshComp->GetSceneProxy())
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Proxy->SetCanSkipRedundantTransformUpdates(true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				InOutCachedValue.RestoreState();
			}
		}
	};

	/** Pre-animation storage for anim next target */
	struct FPreAnimatedAnimNextMixedSkeletalAnimationStorage : TPreAnimatedStateStorage<FPreAnimatedAnimNextMixedSkeletalAnimationTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedAnimNextMixedSkeletalAnimationStorage> StorageID;

		void BeginTrackingEntity(FMovieSceneEntityID EntityID, bool bWantsRestoreState, FRootInstanceHandle RootInstanceHandle, UUAFComponent* Component, FAnimNextVariableReference InjectionSite)
		{
			if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestoreState)
			{
				return;
			}

			FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();

			KeyType Key{ Component, InjectionSite };
	
			FPreAnimatedStorageIndex       StorageIndex = this->GetOrCreateStorageIndex(Key);
			FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.MakeGroup(Component);
			FPreAnimatedStateEntry Entry { GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };

			EntityMetaData->BeginTrackingEntity(Entry, EntityID, RootInstanceHandle, bWantsRestoreState);
		}

		template<typename OnCacheValue /* StorageType(const KeyType&) */>
		void CachePreAnimatedValue(UUAFComponent* Component, FAnimNextVariableReference InjectionSite, OnCacheValue&& CacheCallback)
		{
			EPreAnimatedCaptureSourceTracking TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked;

			if (this->ShouldTrackCaptureSource(TrackingMode, Component, InjectionSite))
			{
				KeyType Key{ Component, InjectionSite };
	
				FPreAnimatedStorageIndex       StorageIndex = this->GetOrCreateStorageIndex(Key);
				FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.MakeGroup(Component);
				FPreAnimatedStateEntry Entry { GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };

				this->TrackCaptureSource(Entry, TrackingMode);

				EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
				if (!this->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
				{
					StorageType NewValue = CacheCallback(Component);
					this->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(NewValue));
				}
			}
		}

	};

	struct FPreAnimatedSkelMeshComponentStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedSkelMeshComponentTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkelMeshComponentStorage> StorageID;
	};

	TAutoRegisterPreAnimatedStorageID<FPreAnimatedAnimNextMixedSkeletalAnimationStorage> FPreAnimatedAnimNextMixedSkeletalAnimationStorage::StorageID;
	TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkelMeshComponentStorage> FPreAnimatedSkelMeshComponentStorage::StorageID;

	/** Task for evaluating and applying animation tasks */
	struct FEvaluateAnimNextTasks
	{
	private:

		UMovieSceneEntitySystemLinker* Linker;
		UMovieSceneAnimNextTargetSystem* System;

		TSharedPtr<FPreAnimatedAnimNextMixedSkeletalAnimationStorage> PreAnimatedStorage;
		TSharedPtr<FPreAnimatedSkelMeshComponentStorage> PreAnimatedSkelMeshCompStorage;

	public:

		FEvaluateAnimNextTasks(UMovieSceneEntitySystemLinker* InLinker, UMovieSceneAnimNextTargetSystem* InSystem)
			: Linker(InLinker)
			, System(InSystem)
		{
			PreAnimatedStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedAnimNextMixedSkeletalAnimationStorage>();
			PreAnimatedSkelMeshCompStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSkelMeshComponentStorage>();
		}

		void ForEachEntity
		(
			FMovieSceneEntityID EntityID,
			FObjectComponent ObjectComponent,
			const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
			TSharedPtr<FAnimNextEvaluationTask> MixerTask
		) const
		{
			// Invalid or incompatible target, return.
			const FMovieSceneAnimNextInjectionTarget* InjectionTarget = Target.GetPtr<FMovieSceneAnimNextInjectionTarget>();
			if (!InjectionTarget)
			{
				return;
			}

			// TODO: It's valid (or will be) to use Anim Next without a Skeletal Mesh Component.
			// We'll need to refactor this to be able to pass an Anim Next component here and refactor core anim tracks to match.

			UObject* BoundMesh = ObjectComponent.GetObject();
			USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundMesh);
			UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(BoundMesh);
			bool bModuleReplaced = false;

			if (!AnimNextComponent)
			{
				AActor* Actor = nullptr;
				// Could be a skeletal mesh component
				if (UActorComponent* ActorComponent = Cast<UActorComponent>(BoundMesh))
				{
					Actor = ActorComponent->GetOwner();
				}
				else
				{
					Actor = Cast<AActor>(ObjectComponent.GetObject());
				}

				if (Actor)
				{
					AnimNextComponent = Actor->FindComponentByClass<UUAFComponent>();
				}
			}

			if (AnimNextComponent)
			{
				TStrongObjectPtr<const UUAFSystem> PreAnimatedModule = TStrongObjectPtr<const UUAFSystem>(FAnimMixerUtils::GetUAFModule(AnimNextComponent));
				if (!FAnimMixerUtils::IsUAFModuleValid(AnimNextComponent))
				{
					const UMovieSceneAnimMixerSettings* AnimMixerSettings = GetDefault<UMovieSceneAnimMixerSettings>();

					FAnimMixerUtils::UnregisterUAFSystem(AnimNextComponent);
					FAnimMixerUtils::SetUAFComponentModule(AnimNextComponent, AnimMixerSettings->DefaultUAFModule.LoadSynchronous());
					FAnimMixerUtils::RegisterUAFSystem(AnimNextComponent);

					bModuleReplaced = true;
				}
				UE::UAF::FInjectionSite InjectionSite(InjectionTarget->InjectionSite);
				if (InjectionSite.DesiredSite.IsNone())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					InjectionSite.DesiredSite = FAnimNextVariableReference(GSequencerDefaultAnimNextInjectionSite);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					InjectionSite.bUseModuleFallback = true;
				}

				// If not currently injected, inject
				FMovieSceneAnimNextTargetData* TargetData = System->CurrentTargets.FindByPredicate([AnimNextComponent, InjectionSite](const FMovieSceneAnimNextTargetData& Data) { return Data.AnimNextComponent == AnimNextComponent && Data.InjectionSite.DesiredSite == InjectionSite.DesiredSite;});

				if (!TargetData)
				{
					TargetData = &System->CurrentTargets.AddDefaulted_GetRef();
					TargetData->AnimNextComponent = AnimNextComponent;
					TargetData->InjectionSite = InjectionSite;
					TargetData->Modifier = MakeShared<FMovieSceneAnimMixerEvaluationModifier>(MixerTask);

					// Inject
					TargetData->InjectionRequestHandle = UE::UAF::FInjectionUtils::InjectEvaluationModifier(AnimNextComponent, TargetData->Modifier.ToSharedRef(), TargetData->InjectionSite);

					// For now, we always restore state
					PreAnimatedStorage->BeginTrackingEntity(EntityID, true, FRootInstanceHandle(), AnimNextComponent, InjectionSite.DesiredSite);
					PreAnimatedSkelMeshCompStorage->BeginTrackingEntity(EntityID, true, FRootInstanceHandle(), SkeletalMeshComponent);

					bool bAnimNextModuleEnabled = AnimNextComponent->IsActive();

					auto OnCacheAnimatedState = [this, TargetData, bAnimNextModuleEnabled, bModuleReplaced, PreAnimatedModule](UUAFComponent*)
					{
						FPreAnimatedAnimNextState State;
						State.InjectionHandle = TargetData->InjectionRequestHandle;
						State.System = this->System;
						State.bAnimNextModuleEnabled = bAnimNextModuleEnabled;
						State.bModuleReplaced = bModuleReplaced;
						State.PreAnimatedModule = PreAnimatedModule;
						return State;
					};

					PreAnimatedStorage->CachePreAnimatedValue(AnimNextComponent, InjectionSite.DesiredSite, OnCacheAnimatedState);
					PreAnimatedSkelMeshCompStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), SkeletalMeshComponent);

					// Force enable the anim next module if not currently enabled
					if (!bAnimNextModuleEnabled)
					{
						AnimNextComponent->SetActive(true);
					}
				}
				else if(TargetData->Modifier.IsValid())
				{
					// Update Task
					TargetData->Modifier->TaskToInject = MixerTask;
				}
			}

		}
	};

} // namespace UE::MovieScene

void FMovieSceneAnimMixerEvaluationModifier::PostEvaluate(UE::UAF::FEvaluateTraversalContext& Context) const
{
	if (TaskToInject.IsValid())
	{
		Context.AppendTaskPtr(TaskToInject);
	}
}

#if WITH_EDITORONLY_DATA
bool FMovieSceneAnimNextInjectionTarget::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FMovieSceneAnimNextInjectionTarget::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InjectionSite = FAnimNextVariableReference(InjectionSiteName_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}
#endif

UMovieSceneAnimNextTargetSystem::UMovieSceneAnimNextTargetSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->MixerTask;
	Phase = ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneAnimMixerSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
	}
}

void UMovieSceneAnimNextTargetSystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMovieSceneAnimNextTargetSystem* This = static_cast<UMovieSceneAnimNextTargetSystem*>(InThis);

	Super::AddReferencedObjects(InThis, Collector);

	for (const FMovieSceneAnimNextTargetData& TargetData : This->CurrentTargets)
	{
		if (TargetData.InjectionRequestHandle)
		{
			TargetData.InjectionRequestHandle->ExternalAddReferencedObjects(Collector);
		}
	}
}

void UMovieSceneAnimNextTargetSystem::OnUnlink()
{
	// Clean up system data
	for (const FMovieSceneAnimNextTargetData& TargetData : CurrentTargets)
	{
		// Remove injection
		if (TargetData.InjectionRequestHandle.IsValid())
		{
			UE::UAF::FInjectionUtils::Uninject(TargetData.InjectionRequestHandle);
		}
	}

	CurrentTargets.Empty();
}

void UMovieSceneAnimNextTargetSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Evaluate anim tasks targeting an anim next injection point
	FTaskParams Params(TEXT("Apply AnimNext Animation Tasks"));
	Params.ForceGameThread();
	FTaskID EvaluateTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(AnimMixerComponents->MeshComponent)
		.Read(AnimMixerComponents->Target)
		.Read(AnimMixerComponents->MixerTask)
		.SetParams(Params)
		.Schedule_PerEntity<FEvaluateAnimNextTasks>(&Linker->EntityManager, TaskScheduler,
			Linker, this);
}