// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMixedSkeletalAnimationSystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieScene.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalMeshRestoreState.h"

#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Systems/MovieSceneObjectPropertySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"

#include "Components/SkeletalMeshComponent.h"
#include "Component/AnimNextComponent.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "AnimSequencerInstanceProxy.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimationInstanceScope.h"
#include "Animation/AnimNotifyQueue.h"
#include "BonePose.h"
#include "EvaluationVM/Tasks/DecompressionTools.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMixedSkeletalAnimationSystem)


namespace UE::MovieScene
{

bool GMovieSceneAnimMixerEnabled = true;
FAutoConsoleVariableRef CVarMovieSceneAnimMixerEnabled(
	TEXT("Sequencer.AnimMixer.Enabled"),
	GMovieSceneAnimMixerEnabled,
	TEXT("(Default: true) Controls whether the new Anim Mixer library is used for skeletal animation evaluation.\n"),
	ECVF_Default);

bool GAnimMixerOverrideAnimationTrack = true;
FAutoConsoleVariableRef CVarAnimMixerOverrideAnimationTrack(
	TEXT("Sequencer.AnimMixer.OverrideAnimationTrack"),
	GAnimMixerOverrideAnimationTrack,
	TEXT("(Default: true) Controls whether the new Anim Mixer library should override Animation Track behavior or not. Warning: disabling this means that Animation Tracks and Animation Mixer Tracks will fight with each other if they animate the same object.\n"),
	ECVF_Default);

// Adds required anim mixer components onto skeletal animation section entities
struct FSkeletalAnimMixerMutation : IMovieSceneEntityMutation
{
	FBuiltInComponentTypes* BuiltInComponents;
	FMovieSceneTracksComponentTypes* TrackComponents; 
	FAnimMixerComponentTypes* AnimMixerComponents;
	UMovieSceneEntitySystemLinker* Linker = nullptr;
	FSkeletalAnimMixerMutation(UMovieSceneEntitySystemLinker* InLinker)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
		TrackComponents = FMovieSceneTracksComponentTypes::Get();
		AnimMixerComponents = FAnimMixerComponentTypes::Get();
		Linker = InLinker;
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
	{
		InOutEntityComponentTypes->SetAll(
		{
			TrackComponents->Tags.AnimMixerPoseProducer,
			AnimMixerComponents->Priority,
			AnimMixerComponents->Target,
			AnimMixerComponents->Task,
			AnimMixerComponents->MixerEntry,
			AnimMixerComponents->MirrorTable
		});

		// Backwards-compat root motion settings are only synthesized for sections outside a mixer
		// track. Sections inside a mixer track get their settings from the Root Motion Settings
		// decoration instead, so an undecorated section has no settings component and extracts no
		// root motion.
		if (!InOutEntityComponentTypes->Contains(AnimMixerComponents->MixerLayer))
		{
			InOutEntityComponentTypes->Set(AnimMixerComponents->RootMotionSettings);
		}
	}

	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		const int32 Num = Allocation->Num();
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();

		// @todo: figure out restore state semantics with root motion
		const bool bWantsRestore = false;
		const bool bCapturePreAnimatedState = Linker->PreAnimatedState.IsCapturingGlobalState() || bWantsRestore;
		static const FName PreAnimatedTransformName("Transform");

		FPreAnimatedEntityCaptureSource* EntityMetaData = nullptr;
		TSharedPtr<FPreAnimatedComponentTransformStorage> ComponentTransformStorage;
		if (bCapturePreAnimatedState)
		{
			EntityMetaData = Linker->PreAnimatedState.GetOrCreateEntityMetaData();
			ComponentTransformStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedComponentTransformStorage>();
		}

		TArrayView<const FMovieSceneEntityID> EntityIDs = Allocation->GetEntityIDs();
		TComponentReader<FRootInstanceHandle> RootInstanceHandles = Allocation->ReadComponents(BuiltInComponents->RootInstanceHandle);
		TComponentReader<UObject*> BoundObjects = Allocation->ReadComponents(BuiltInComponents->BoundObject);

		TComponentReader<FMovieSceneSkeletalAnimationComponentData> SkeletalAnimationData = Allocation->ReadComponents(TrackComponents->SkeletalAnimation);
		TComponentWriter<TInstancedStruct<FMovieSceneMixedAnimationTarget>> OutAnimTargets = Allocation->WriteComponents(AnimMixerComponents->Target, WriteContext);
		TComponentWriter<TSharedPtr<FAnimNextEvaluationTask>> OutTasks = Allocation->WriteComponents(AnimMixerComponents->Task, WriteContext);
		TComponentWriter<int32> OutPriorities = Allocation->WriteComponents(AnimMixerComponents->Priority, WriteContext);
		TOptionalComponentWriter<FMovieSceneRootMotionSettings> OutRootMotionSettings = Allocation->TryWriteComponents(AnimMixerComponents->RootMotionSettings, WriteContext);
		TOptionalComponentReader<TObjectPtr<UMovieSceneAnimationMixerLayer>> OptMixerLayers = Allocation->TryReadComponents(AnimMixerComponents->MixerLayer);
		
		TComponentWriter<TObjectPtr<UMirrorDataTable>> OutMirrorTables = Allocation->WriteComponents(AnimMixerComponents->MirrorTable, WriteContext);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			// Initialize a null (empty) task, will be written during evaluation
			OutTasks[Index] = nullptr;

			const FMovieSceneSkeletalAnimationComponentData& SkeletalAnim = SkeletalAnimationData[Index];
			if (SkeletalAnim.Section)
			{
				// If we're not on a mixer layer, set a default target and priority
				if (!OptMixerLayers || !OptMixerLayers[Index])
				{
					OutAnimTargets[Index] = TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make();
					OutPriorities[Index] = -100000;
				}

				UMovieSceneCommonAnimationTrack* Track = SkeletalAnim.Section->GetTypedOuter<UMovieSceneCommonAnimationTrack>();
				UMovieSceneSkeletalAnimationTrack* SkelAnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track);

				ESwapRootBone LegacySwapRootBone = SkeletalAnim.Section->Params.SwapRootBone;
				if (LegacySwapRootBone == ESwapRootBone::SwapRootBone_None && SkelAnimTrack)
				{
					LegacySwapRootBone = SkelAnimTrack->SwapRootBone;
				}
				if (OutRootMotionSettings)
				{
					OutRootMotionSettings[Index].LegacySwapRootBone = LegacySwapRootBone;
				}
				// Only apply the legacy section MirrorDataTable if set; otherwise
				// preserve the value from the mirroring decoration (if any).
				if (SkeletalAnim.Section->Params.MirrorDataTable)
				{
					OutMirrorTables[Index] = SkeletalAnim.Section->Params.MirrorDataTable;
				}

				if (bCapturePreAnimatedState)
				{
					USceneComponent* SwapRootComponent = nullptr;
					if (USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObjects[Index]))
					{
						switch (LegacySwapRootBone)
						{
							case ESwapRootBone::SwapRootBone_None: break;
							case ESwapRootBone::SwapRootBone_Component: SwapRootComponent = BoundComponent; break;
							case ESwapRootBone::SwapRootBone_Actor:     SwapRootComponent = BoundComponent->GetOwner()->GetRootComponent(); break;
						}
					}
					if (SwapRootComponent)
					{
						FPreAnimatedStateEntry Entry = ComponentTransformStorage->MakeEntry(SwapRootComponent, PreAnimatedTransformName);
						EntityMetaData->BeginTrackingEntity(Entry, EntityIDs[Index], RootInstanceHandles[Index], bWantsRestore);
						ComponentTransformStorage->CachePreAnimatedTransform(FCachePreAnimatedValueParams(), SwapRootComponent);
					}
				}

				if (OutRootMotionSettings)
				{
					OutRootMotionSettings[Index].bHasRootMotionOverride = false;

					if (Track)
					{
						if (Track->RootMotionParams.bRootMotionsDirty)
						{
							constexpr bool bForce = true;
							Track->SetUpRootMotions(bForce);
						}

						if (Track->RootMotionParams.bHaveRootMotion)
						{
							OutRootMotionSettings[Index].bHasRootMotionOverride = true;
						}
					}

					if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(SkeletalAnim.Section ? SkeletalAnim.Section->Params.Animation.Get() : nullptr))
					{
						OutRootMotionSettings[Index].bForceRootLock = AnimSequence->bForceRootLock && AnimSequence->bEnableRootMotion;
					}
				}
			}
		}
	}
};



struct FGatherMixableRootMotion
{
	static void ForEachEntity(
		const FMovieSceneSkeletalAnimationComponentData& SkeletalAnimation,
		FFrameTime EvalTime,
		FMovieSceneRootMotionSettings& RootMotionSettings
		)
	{
		if (RootMotionSettings.bHasRootMotionOverride)
		{
			UMovieSceneSkeletalAnimationSection::FRootMotionParams RootMotionParams;

			SkeletalAnimation.Section->GetRootMotion(EvalTime, RootMotionParams);
			if (RootMotionParams.Transform.IsSet())
			{
				RootMotionSettings.RootOverrideLocation = RootMotionParams.Transform->GetLocation();
				RootMotionSettings.RootOverrideRotation = RootMotionParams.Transform->GetRotation();
				RootMotionSettings.bBlendFirstChildOfRoot = RootMotionParams.bBlendFirstChildOfRoot;
				RootMotionSettings.ChildBoneIndex = RootMotionParams.ChildBoneIndex;
			}
		}
	}
};


/** ------------------------------------------------------------------------- */
/** Task for gathering active skeletal animations and setting up their tasks. */
struct FGatherMixableSkeletalAnimations
{
	const FInstanceRegistry* InstanceRegistry;
	const UMovieSceneTransformOriginSystem* TransformOriginSystem;

	FGatherMixableSkeletalAnimations(UMovieSceneEntitySystemLinker* InLinker)
		: InstanceRegistry(InLinker->GetInstanceRegistry())
	{
		TransformOriginSystem = InLinker->FindSystem<UMovieSceneTransformOriginSystem>();
	}

	void ForEachAllocation(
		const FEntityAllocationProxy AllocationProxy,
		TRead<FMovieSceneEntityID> EntityIDs,
		TRead<FInstanceHandle> InstanceHandles,
		TRead<UObject*> BoundObjects,
		TRead<FMovieSceneSkeletalAnimationComponentData> SkeletalAnimations,
		TReadOptional<FFrameTime> OptionalEvalTimes,
		TWriteOptional<FMovieSceneRootMotionSettings> OptionalRootMotionSettings,
		TWrite<TSharedPtr<FAnimNextEvaluationTask>> EvalTask) const
	{
		// Gather all the skeletal animations currently active in all sequences.
		// We map these animations to their bound object, which means we might blend animations from different sequences
		// that have bound to the same object.
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FEntityAllocation* Allocation = AllocationProxy.GetAllocation();
		const int32 Num = Allocation->Num();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			FMovieSceneEntityID EntityID(EntityIDs[Index]);
			const FInstanceHandle& InstanceHandle(InstanceHandles[Index]);
			UObject* BoundObject(BoundObjects[Index]);
			const FMovieSceneSkeletalAnimationComponentData& SkeletalAnimation(SkeletalAnimations[Index]);

			// Get the full context, so we can get both the current and previous evaluation times.
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);
			const FMovieSceneContext& Context = SequenceInstance.GetContext();

			// Calculate the time at which to evaluate the animation
			const UMovieSceneSkeletalAnimationSection* AnimSection = SkeletalAnimation.Section;
			const FMovieSceneSkeletalAnimationParams& AnimParams = AnimSection->Params;

			// Get the bound skeletal mesh component.
			USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(BoundObject);
			UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSection->GetPlaybackAnimation());
			if (!SkeletalMeshComponent || AnimSequence == nullptr)
			{
				continue;
			}


			FFrameTime EvalFrameTime = OptionalEvalTimes ? OptionalEvalTimes[Index] : Context.GetTime();
			FFrameTime PreviousEvalFrameTime = Context.GetPreviousTime();

			FFrameNumber SectionStartTime = AnimSection->GetInclusiveStartFrame();
			FFrameNumber SectionEndTime   = AnimSection->GetExclusiveEndFrame();

			if (AnimParams.bLinearPlaybackWhenScaled)
			{
				UMovieSceneScalingAnchors* ScalingAnchors = AnimSection->GetTypedOuter<UMovieScene>()->FindDecoration<UMovieSceneScalingAnchors>();

				if (ScalingAnchors)
				{
					TOptional<FFrameTime> UnwarpedTime = ScalingAnchors->InverseRemapTimeCycled(EvalFrameTime, EvalFrameTime, FInverseTransformTimeParams());
					if (UnwarpedTime.IsSet())
					{
						EvalFrameTime = UnwarpedTime.GetValue();
					}
					TOptional<FFrameTime> PreviousUnwarpedTime = ScalingAnchors->InverseRemapTimeCycled(PreviousEvalFrameTime, PreviousEvalFrameTime, FInverseTransformTimeParams());
					if (PreviousUnwarpedTime.IsSet())
					{
						PreviousEvalFrameTime = PreviousUnwarpedTime.GetValue();
					}
					TOptional<FFrameTime> UnwarpedStartTime = ScalingAnchors->InverseRemapTimeCycled(SectionStartTime, SectionStartTime, FInverseTransformTimeParams());
					if (UnwarpedStartTime.IsSet())
					{
						SectionStartTime = UnwarpedStartTime.GetValue().RoundToFrame();
					}
					TOptional<FFrameTime> UnwarpedEndTime = ScalingAnchors->InverseRemapTimeCycled(SectionEndTime, SectionEndTime, FInverseTransformTimeParams());
					if (UnwarpedEndTime.IsSet())
					{
						SectionEndTime = UnwarpedEndTime.GetValue().RoundToFrame();
					}
				}
			}


			const float EvalTime = AnimParams.MapTimeToAnimation(SectionStartTime, SectionEndTime, EvalFrameTime, Context.GetFrameRate(), AnimSequence);
			const float PreviousEvalTime = AnimParams.MapTimeToAnimation(SectionStartTime, SectionEndTime, PreviousEvalFrameTime, Context.GetFrameRate(), AnimSequence);

			const EMovieScenePlayerStatus::Type PlayerStatus = Context.GetStatus();

			const bool bPreviewPlayback = ShouldUsePreviewPlayback(PlayerStatus, *BoundObject);

			// If the playback status is jumping, ie. one such occurrence is setting the time for thumbnail generation, disable anim notifies updates because it could fire audio.
			// If the playback status is scrubbing, we disable notifies for now because we can't properly fire them in all cases until we get evaluation range info.
			// We now layer this with the passed in notify toggle to force a disable in this case.
			const bool bFireNotifies = !bPreviewPlayback || (PlayerStatus != EMovieScenePlayerStatus::Jumping && PlayerStatus != EMovieScenePlayerStatus::Stopped && PlayerStatus != EMovieScenePlayerStatus::Scrubbing);
			const bool bPlaying = PlayerStatus == EMovieScenePlayerStatus::Playing;

			// Don't fire notifies if looping around.
			bool bLooped = false;
			if (AnimParams.bReverse)
			{
				if (PreviousEvalTime <= EvalTime)
				{
					bLooped = true;
				}
			}
			else if (PreviousEvalTime >= EvalTime)
			{
				bLooped = true;
			}

			FMixedAnimSkeletalAnimationData AnimData;
			AnimData.AnimSequence = AnimSequence;
			if (OptionalRootMotionSettings)
			{
				if (OptionalRootMotionSettings[Index].bHasRootMotionOverride)
				{
					FSkeletalAnimationRootMotionOverride NewOverride;
					NewOverride.RootMotion = FTransform(OptionalRootMotionSettings[Index].RootOverrideRotation, OptionalRootMotionSettings[Index].RootOverrideLocation);
					NewOverride.ChildBoneIndex = OptionalRootMotionSettings[Index].ChildBoneIndex;
					NewOverride.bBlendFirstChildOfRoot = OptionalRootMotionSettings[Index].bBlendFirstChildOfRoot;

					AnimData.RootMotionOverride = NewOverride;
				}
				else
				{
					AnimData.RootMotionOverride.Reset();
				}
			}
			AnimData.FromPosition = PreviousEvalTime;
			AnimData.ToPosition = EvalTime;
			AnimData.bFireNotifies = bFireNotifies && !AnimParams.bSkipAnimNotifiers && !bLooped;
			// TODO: We want to allow for additive to be set on the section properties
			AnimData.bAdditive = AnimData.AnimSequence.IsValid() ? AnimData.AnimSequence->IsValidAdditive() : false;
			AnimData.bSuppressForceRootLock = OptionalRootMotionSettings && OptionalRootMotionSettings[Index].bForceRootLock;


			AnimData.PendingNotifies.Reset();

			const float MoveDelta = EvalTime - PreviousEvalTime;
			if (AnimData.bFireNotifies && !FMath::IsNearlyZero(MoveDelta))
			{
				// Per-section player ID so UAnimInstance's ActiveAnimNotifyState diff can
				// distinguish two sections playing the same UAnimSequence on the same mesh.
				const uint32 AssetPlayerInstanceID = HashCombine(
					GetTypeHash(FObjectKey(AnimSection)),
					GetTypeHash(FObjectKey(SkeletalMeshComponent)));

				// Populate the notify references' context data directly. FAnimTickRecord::MakeContextData
				// would be more natural but it inlines a private Engine helper we can't link against.
				FAnimTickRecord StubRecord;
				StubRecord.SourceAsset = AnimSequence;

				FAnimNotifyContext NotifyContext(StubRecord);
				AnimSequence->GetAnimNotifies(PreviousEvalTime, MoveDelta, NotifyContext);

				for (FAnimNotifyEventReference& Ref : NotifyContext.ActiveNotifies)
				{
					Ref.AddContextData<UE::Anim::FAnimNotifyAssetPlayerInstanceContext>(AssetPlayerInstanceID);
				}
				AnimData.PendingNotifies = MoveTemp(NotifyContext.ActiveNotifies);
			}

			if (!EvalTask[Index].IsValid())
			{ 
				// Create task if not yet created
				EvalTask[Index] = MakeShared<FMovieSceneSkeletalAnimationEvaluationTask>();
			}
			TSharedPtr<FMovieSceneSkeletalAnimationEvaluationTask> AnimTask = StaticCastSharedPtr<FMovieSceneSkeletalAnimationEvaluationTask>(EvalTask[Index]);
			AnimTask->AnimationData = AnimData;
		}
	}

private:

	static bool ShouldUsePreviewPlayback(EMovieScenePlayerStatus::Type PlayerStatus, UObject& RuntimeObject)
	{
		// We also use PreviewSetAnimPosition in PIE when not playing, as we can preview in PIE.
		bool bIsNotInPIEOrNotPlaying = (RuntimeObject.GetWorld() && !RuntimeObject.GetWorld()->HasBegunPlay()) || PlayerStatus != EMovieScenePlayerStatus::Playing;
		return GIsEditor && bIsNotInPIEOrNotPlaying;
	}
};

} // namespace UE::MovieScene

void FMovieSceneSkeletalAnimationEvaluationTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		FValueBundleStack Collection(VM.GetActiveNamedSet());
		Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		return;
	}

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	if (const UAnimSequence* AnimSequencePtr = AnimationData.AnimSequence.Get())
	{
		const bool bIsAdditive = AnimSequencePtr->IsValidAdditive();
		FDeltaTimeRecord DeltaTime;
		DeltaTime.Set(AnimationData.FromPosition, AnimationData.ToPosition);

		const bool bExtractRootMotion = AnimationData.RootMotionOverride.IsSet();
		FAnimExtractContext ExtractionContext(AnimationData.ToPosition, bExtractRootMotion, DeltaTime, false);
#if WITH_EDITOR
		// When the extraction task will handle root motion, suppress Force Root Lock
		// during pose evaluation so the root bone retains its actual motion.
		ExtractionContext.bIgnoreRootLock = AnimationData.bSuppressForceRootLock;
#endif

		FKeyframeState Keyframe = VM.MakeUninitializedKeyframe(bIsAdditive);

		const bool bUseRawData = FDecompressionTools::ShouldUseRawData(AnimSequencePtr, ExtractionContext);

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			FDecompressionTools::GetAnimationPose(AnimSequencePtr, ExtractionContext, Keyframe.Pose, bUseRawData);

			// Legacy root motion override: the common animation track pre-computes root motion
			// transforms and passes them in via RootMotionOverride.
			if (AnimationData.RootMotionOverride.IsSet())
			{
				if (!AnimationData.RootMotionOverride->bBlendFirstChildOfRoot)
				{
					// Standard root motion: apply the override to bone 0 and store it
					// as the RootTransformAttribute so FAnimNextExtractRootMotionTask
					// (which should not be created for legacy overrides) is not needed.
					const int32 PoseIndex = Keyframe.Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
					if (PoseIndex != INDEX_NONE)
					{
						Keyframe.Pose.LocalTransformsView[PoseIndex] = AnimationData.RootMotionOverride->RootMotion;
					}

					FTransformAnimationAttribute* RootMotionAttribute = Keyframe.Attributes.FindOrAdd<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
					FFloatAnimationAttribute* RootMotionWeight = Keyframe.Attributes.FindOrAdd<FFloatAnimationAttribute>(AnimMixerComponents->RootTransformWeightAttributeId);
					RootMotionAttribute->Value = AnimationData.RootMotionOverride->RootMotion;
					RootMotionWeight->Value = 1.f;

					const int32 RootIndex = Keyframe.Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
					if (RootIndex != INDEX_NONE)
					{
						Keyframe.Pose.LocalTransformsView[RootIndex] = FTransform::Identity;
					}
				}
				else if (AnimationData.RootMotionOverride->ChildBoneIndex != INDEX_NONE)
				{
					// Blend-first-child-of-root: the pre-computed transform targets a
					// child bone, not the root. Apply the override to the child bone.
					// The root bone's animated value is extracted separately as root
					// motion so SwapRootBone still works (the old code path applied
					// SwapRootBone independently after setting the child bone).
					const int32 ChildPoseIndex = Keyframe.Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(AnimationData.RootMotionOverride->ChildBoneIndex);
					if (ChildPoseIndex != INDEX_NONE)
					{
						Keyframe.Pose.LocalTransformsView[ChildPoseIndex] = AnimationData.RootMotionOverride->RootMotion;
					}

					// Extract the root bone's animated value as root motion and zero
					// it, mirroring what SwapRootBone did in the old code path.
					const int32 RootIndex = Keyframe.Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
					if (RootIndex != INDEX_NONE)
					{
						FTransformAnimationAttribute* RootMotionAttribute = Keyframe.Attributes.FindOrAdd<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
						FFloatAnimationAttribute* RootMotionWeight = Keyframe.Attributes.FindOrAdd<FFloatAnimationAttribute>(AnimMixerComponents->RootTransformWeightAttributeId);
						RootMotionAttribute->Value = Keyframe.Pose.LocalTransformsView[RootIndex];
						RootMotionWeight->Value = 1.f;

						Keyframe.Pose.LocalTransformsView[RootIndex] = FTransform::Identity;
					}
				}
			}
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
		{
			FDecompressionTools::GetAnimationCurves(AnimSequencePtr, ExtractionContext, Keyframe.Curves, bUseRawData);
		}

		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
		{
			FDecompressionTools::GetAnimationAttributes(AnimSequencePtr, ExtractionContext, Keyframe.Pose.GetRefPose(), Keyframe.Attributes, bUseRawData);
		}

		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
	}
	else
	{
		constexpr bool bIsAdditive = false;
		FKeyframeState Keyframe = VM.MakeReferenceKeyframe(bIsAdditive);
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));
	}
}

UMovieSceneMixedSkeletalAnimationSystem::UMovieSceneMixedSkeletalAnimationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	RelevantComponent = TrackComponents->SkeletalAnimation;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;
	SystemCategories = EEntitySystemCategory::BlenderSystems;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get(); 

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBoundSceneComponentInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneTransformOriginSystem::StaticClass(), GetClass());
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentConsumer(GetClass(), TrackComponents->SkeletalAnimation);
		DefineComponentProducer(GetClass(), TrackComponents->Tags.AnimMixerPoseProducer);
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
		DefineComponentProducer(GetClass(), AnimMixerComponents->Task);
		DefineComponentProducer(GetClass(), AnimMixerComponents->Target);
		DefineComponentProducer(GetClass(), AnimMixerComponents->Priority);

	}
	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();
}

void UMovieSceneMixedSkeletalAnimationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	if (!GMovieSceneAnimMixerEnabled)
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		FEntityComponentFilter Filter;
		Filter.All({ BuiltInComponents->BoundObject, TrackComponents->SkeletalAnimation, BuiltInComponents->Tags.NeedsLink });
		Filter.None({ BuiltInComponents->Tags.ImportedEntity });

		if (GAnimMixerOverrideAnimationTrack == false)
		{
			// If we're not overriding animation tracks, constrain the mutation to things that have already been marked as AnimMixerPoseProducers
			//   This happens for the Anim Mixer Track inside UMovieSceneAnimationSectionDecoration::ExtendEntityImpl
			Filter.All({ TrackComponents->Tags.AnimMixerPoseProducer });
		}

		// Initialize components for skeletal animation mixing
		FSkeletalAnimMixerMutation SkeletalAnimMixerMutation(Linker);
		Linker->EntityManager.MutateAll(Filter, SkeletalAnimMixerMutation);
	}
}

void UMovieSceneMixedSkeletalAnimationSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// Gather all skel anim section entities that have been marked with the AnimMixerPoseProducer tag
	FTaskID GatherRootMotionTask = FEntityTaskBuilder()
		.Read(TrackComponents->SkeletalAnimation)
		.Read(BuiltInComponents->EvalTime)
		.Write(AnimMixerComponents->RootMotionSettings)
		.SetDesiredThread(ENamedThreads::GameThread_Local)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerEntity<FGatherMixableRootMotion>(&Linker->EntityManager, TaskScheduler);

	// Gather all skel anim section entities that have been marked with the AnimMixerPoseProducer tag
	FTaskID GatherTask = FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.Read(TrackComponents->SkeletalAnimation)
		.ReadOptional(BuiltInComponents->EvalTime)
		.WriteOptional(AnimMixerComponents->RootMotionSettings)
		.Write(AnimMixerComponents->Task)
		.FilterAll( { TrackComponents->Tags.AnimMixerPoseProducer })
		.FilterNone({ BuiltInComponents->Tags.Ignored})
		.Schedule_PerAllocation<FGatherMixableSkeletalAnimations>(&Linker->EntityManager, TaskScheduler,
			Linker);

	TaskScheduler->AddPrerequisite(GatherRootMotionTask, GatherTask);
}
