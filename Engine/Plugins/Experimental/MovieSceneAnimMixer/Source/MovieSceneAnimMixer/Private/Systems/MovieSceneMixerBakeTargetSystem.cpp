// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMixerBakeTargetSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneTracksComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Components/SkeletalMeshComponent.h"

#include "Animation/BuiltInAttributeTypes.h"
#include "Misc/MemStack.h"
#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationVM.h"
#include "GenerationTools.h"
#include "Graph/AnimNext_LODPose.h"
#include "BonePose.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMixerBakeTargetSystem)

namespace UE::MovieScene
{

// Task that executes mixer programs in a temporary VM, capturing the
// resulting keyframe and root motion transform instead of applying them.
struct FCaptureBakeResults
{
	UMovieSceneEntitySystemLinker* Linker;
	UMovieSceneMixerBakeTargetSystem* BakeSystem;

	FCaptureBakeResults(UMovieSceneEntitySystemLinker* InLinker, UMovieSceneMixerBakeTargetSystem* InBakeSystem)
		: Linker(InLinker)
		, BakeSystem(InBakeSystem)
	{
	}

	void ForEachEntity(
		FObjectComponent MeshComponent,
		const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target,
		const TSharedPtr<FAnimNextEvaluationTask>& MixerTask) const
	{
		if (!BakeSystem->bBakeActive || !MixerTask.IsValid())
		{
			return;
		}

		// Only process entities targeted with FMovieSceneMixerBakeTarget.
		// During bake evaluation the track's target is temporarily swapped
		// to this type, so only bake-targeted entities end up here.
		if (!Target.GetPtr<FMovieSceneMixerBakeTarget>())
		{
			return;
		}

		USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(MeshComponent.GetObject());
		if (!SkelMeshComp || !SkelMeshComp->GetSkeletalMeshAsset())
		{
			return;
		}

		using namespace UE::UAF;

		FMemMark Mark(FMemStack::Get());

		// Set up a temporary VM with the skeletal mesh's reference pose
		FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(SkelMeshComp);
		const FReferencePose& RefPose = RefPoseHandle.GetRef<FReferencePose>();
		int32 LODLevel = SkelMeshComp->GetPredictedLODLevel();

		FEvaluationVM TempVM(EEvaluationFlags::All, RefPose, LODLevel);

		// Push a reference keyframe as the initial pose
		FKeyframeState InitialKeyframe = TempVM.MakeReferenceKeyframe(false);
		TempVM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(InitialKeyframe)));

		// Execute the mixer task
		MixerTask->Execute(TempVM);

		// Pop the resulting keyframe. The store root transform task is
		// deliberately skipped during bake (see FEvaluateAnimMixers), so the
		// RootTransformAttribute is still present on the keyframe with the
		// world-space root motion value from the convert task.
		UMovieSceneMixerBakeTargetSystem::FBakedResult Result;

		TUniquePtr<FKeyframeState> ResultKeyframe;
		if (TempVM.PopValue(KEYFRAME_STACK_NAME, ResultKeyframe))
		{
			const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
			const FTransformAnimationAttribute* RootAttr =
				ResultKeyframe->Attributes.Find<FTransformAnimationAttribute>(
					AnimMixerComponents->RootTransformAttributeId);
			if (RootAttr)
			{
				Result.RootMotionTransform = RootAttr->Value;
			}

			// Copy from stack-allocated keyframe to heap-backed storage
			// before FMemMark frees the evaluation mem stack.
			Result.Pose.PrepareForLOD(RefPose, LODLevel);
			Result.Pose.CopyFrom(ResultKeyframe->Pose);
			Result.Curves.CopyFrom(ResultKeyframe->Curves);
			Result.Attributes.CopyFrom(ResultKeyframe->Attributes);
		}

		BakeSystem->BakedResults.Add(MoveTemp(Result));
	}
};

} // namespace UE::MovieScene

UMovieSceneMixerBakeTargetSystem::UMovieSceneMixerBakeTargetSystem(const FObjectInitializer& Init)
	: Super(Init)
{
	using namespace UE::MovieScene;

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->MixerTask;
	Phase = ESystemPhase::Scheduling;

	// Excluded from interrogation
	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneAnimMixerSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneMixerBakeTargetSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FTaskParams Params(TEXT("Capture Bake Results"));
	Params.ForceGameThread();
	// Skip entities tagged for unlink within this same Flush - they still
	// carry the MixerTask component during Schedule but their mixer is
	// being removed in the same pass.
	FEntityTaskBuilder()
		.Read(AnimMixerComponents->MeshComponent)
		.Read(AnimMixerComponents->Target)
		.Read(AnimMixerComponents->MixerTask)
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.SetParams(Params)
		.Schedule_PerEntity<FCaptureBakeResults>(&Linker->EntityManager, TaskScheduler,
			Linker, this);
}
