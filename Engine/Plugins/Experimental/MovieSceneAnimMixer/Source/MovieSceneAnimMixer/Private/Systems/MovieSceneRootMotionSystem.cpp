// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneRootMotionSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneRootMotionSection.h"

#include "EvaluationVM/EvaluationTask.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Component/AnimNextComponent.h"

#include "Systems/MovieSceneAnimMixerSystem.h"
#include "AnimMixerComponentTypes.h"
#include "Module/AnimNextModuleInstance.h"
#include "Animation/AnimRootMotionProvider.h"

#include "SceneInterface.h"

#include "Components/SkeletalMeshComponent.h"
#include "Graph/AnimNext_LODPose.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/ByteChannelEvaluatorSystem.h"
#include "Misc/MemStack.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "Misc/ScopeRWLock.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRootMotionSystem)

namespace UE::MovieScene
{
	struct FGatherRootDestinations
	{
		UMovieSceneAnimMixerSystem* AnimMixer;

		FGatherRootDestinations(UMovieSceneAnimMixerSystem* InAnimMixer)
			: AnimMixer(InAnimMixer)
		{}

		void ForEachEntity(
			FObjectKey BoundObjectKey,
			uint8 ByteResult,
			EMovieSceneRootMotionDestination& OutRootDestination,
			TSharedPtr<FMovieSceneMixerRootMotionComponentData>& OutRootMotion
			) const
		{
			OutRootDestination = (EMovieSceneRootMotionDestination)(ByteResult);

			// Fall back to the mixer system's root motion map if the entity's
			// component is null (can happen with default-constructed child entities).
			if (!OutRootMotion)
			{
				OutRootMotion = AnimMixer->FindRootMotion(BoundObjectKey);
			}

			if (OutRootMotion)
			{
				OutRootMotion->RootDestination = OutRootDestination;
			}
		}
	};

	struct FInitializeRootMotionTask
	{
		UMovieSceneAnimMixerSystem* AnimMixer;

		FInitializeRootMotionTask(UMovieSceneAnimMixerSystem* InAnimMixer)
			: AnimMixer(InAnimMixer)
		{}

		void PreTask()
		{
			AnimMixer->PreInitializeAllRootMotion();
		}

		void ForEachEntity(UObject* BoundObject,
			const double* InLocationX, const double* InLocationY, const double* InLocationZ,
			const double* InRotationX, const double* InRotationY, const double* InRotationZ,
			const double* InScaleX, const double* InScaleY, const double* InScaleZ) const
		{
			USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject);
			if (!SceneComponent)
			{
				return;
			}

			USceneComponent* RootComponent  = SceneComponent ? SceneComponent->GetOwner()->GetRootComponent() : SceneComponent;
			if (BoundObject == RootComponent)
			{
				FObjectKey BoundObjectKey(BoundObject);

				TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = AnimMixer->FindRootMotion(BoundObjectKey);
				if (RootMotion && RootMotion->RootDestination == EMovieSceneRootMotionDestination::Actor)
				{
					FVector  Location = RootMotion->ActorTransform.GetLocation();
					FRotator Rotation = RootMotion->ActorTransform.GetRotation().Rotator();
					FVector  Scale    = RootMotion->ActorTransform.GetScale3D();

					if (InLocationX) { Location.X = *InLocationX; }
					if (InLocationY) { Location.Y = *InLocationY; }
					if (InLocationZ) { Location.Z = *InLocationZ; }

					if (InRotationX) { Rotation.Roll  = *InRotationX; }
					if (InRotationY) { Rotation.Pitch = *InRotationY; }
					if (InRotationZ) { Rotation.Yaw   = *InRotationZ; }

					if (InScaleX) { Scale.X = *InScaleX; }
					if (InScaleY) { Scale.Y = *InScaleY; }
					if (InScaleZ) { Scale.Z = *InScaleZ; }

					RootMotion->ActorTransform = FTransform(Rotation, Location, Scale);
					RootMotion->bActorTransformSet = true;
					return;
				}
			}

			FTransform Transform = SceneComponent->GetRelativeTransform();
			FIntermediate3DTransform IntermediateTransform(Transform.GetLocation(), Transform.GetRotation().Rotator(), Transform.GetScale3D());

			if (InLocationX) { IntermediateTransform.T_X = *InLocationX; }
			if (InLocationY) { IntermediateTransform.T_Y = *InLocationY; }
			if (InLocationZ) { IntermediateTransform.T_Z = *InLocationZ; }

			if (InRotationX) { IntermediateTransform.R_X = *InRotationX; }
			if (InRotationY) { IntermediateTransform.R_Y = *InRotationY; }
			if (InRotationZ) { IntermediateTransform.R_Z = *InRotationZ; }

			if (InScaleX) { IntermediateTransform.S_X = *InScaleX; }
			if (InScaleY) { IntermediateTransform.S_Y = *InScaleY; }
			if (InScaleZ) { IntermediateTransform.S_Z = *InScaleZ; }

			UE::MovieScene::FIntermediate3DTransform::ApplyTransformTo(SceneComponent, IntermediateTransform);
		}

		void PostTask()
		{
			AnimMixer->InitializeAllRootMotion();
		}
	};

	struct FIsActorBeingMovedStatics
	{
#if !WITH_EDITOR
		constexpr bool IsActive() const
		{
			return false;
		}

		constexpr void Initialize()
		{
		}

#else

		bool IsActive() const
		{
			return bIsActorBeingMoved.load(std::memory_order_relaxed);
		}

		void Initialize()
		{
			static bool bFirst = true;
			if (bFirst)
			{
				bFirst = false;
				if (GEditor)
				{
					GEditor->OnBeginObjectMovement().AddRaw(this, &FIsActorBeingMovedStatics::Set, true);
					GEditor->OnEndObjectMovement().AddRaw(this, &FIsActorBeingMovedStatics::Set, false);
				}
			}
		}

	private:

		void Set(UObject&, bool bIsActive)
		{
			ensure(IsInGameThread());
			bIsActorBeingMoved.exchange(bIsActive, std::memory_order_relaxed);
		}

		mutable std::atomic<bool> bIsActorBeingMoved = false;
#endif // WITH_EDITOR

	} GActorMovementTracker;

	// Convert a world-space root motion attribute value to the actor-relative space
	// expected on bone[0]. No-op when the destination keeps the root in world space.
	static FTransform ConvertRootAttributeToComponentSpace(const FTransform& InAttributeValue, const FMovieSceneMixerRootMotionComponentData& InRootMotionData)
	{
		if (!InRootMotionData.bComponentSpaceRoot)
		{
			return InAttributeValue;
		}

		FTransform Result = InAttributeValue * InRootMotionData.ActorTransform.Inverse();
		TOptional<FQuat> InverseMeshToActorRotation = InRootMotionData.GetInverseMeshToActorRotation();
		if (InverseMeshToActorRotation.IsSet())
		{
			Result.SetTranslation(InverseMeshToActorRotation.GetValue().Inverse() * Result.GetTranslation());
		}
		return Result;
	}

} // namespace UE::MovieScene



TOptional<FQuat> FMovieSceneMixerRootMotionComponentData::GetInverseMeshToActorRotation() const
{
	UE::TReadScopeLock ScopeLock(RootMotionLock);
	return InverseMeshToActorRotation;
}

void FMovieSceneMixerRootMotionComponentData::Initialize()
{
	UE::TWriteScopeLock ScopeLock(RootMotionLock);

	InverseMeshToActorRotation.Reset();

	USceneComponent* BoundObject = OriginalBoundObject.Get();

	if (!BoundObject)
	{
		Target = nullptr;
		// Leave the last known component and actor transform
		return;
	}

	// If we're applying to the actor or a root custom attribute, we need to factor out the actor->compont rotation
	const bool bNeedInverseMeshRotation = RootDestination == EMovieSceneRootMotionDestination::Actor
			|| RootDestination == EMovieSceneRootMotionDestination::Attribute;

	USceneComponent* RootComponent = BoundObject->GetOwner()->GetRootComponent();
	if (RootComponent)
	{
		ComponentToActorTransform = BoundObject->GetComponentTransform().GetRelativeTransform(RootComponent->GetComponentTransform()).Inverse();

		if (bNeedInverseMeshRotation && RootComponent != BoundObject)
		{
			InverseMeshToActorRotation = RootComponent->GetComponentTransform().GetRelativeTransformReverse(BoundObject->GetComponentTransform()).GetRotation();
		}

		if (!bActorTransformSet)
		{
			ActorTransform = RootComponent->GetRelativeTransform();
		}
	}

	switch (RootDestination)
	{
	case EMovieSceneRootMotionDestination::Discard:
	case EMovieSceneRootMotionDestination::RootBone:
	case EMovieSceneRootMotionDestination::Attribute:
		Target = nullptr;
		break;

	case EMovieSceneRootMotionDestination::Component:
		Target = BoundObject;
		break;

	case EMovieSceneRootMotionDestination::Actor:
		Target = RootComponent;
		break;
	}

	// If we want to swap the root bone with the component transform, but that component is the root component,
	//     that is the same behavior as swapping with the actor, so don't perform any inverse component transformations
	bComponentSpaceRoot = (RootDestination == EMovieSceneRootMotionDestination::RootBone)
		|| (RootDestination == EMovieSceneRootMotionDestination::Component && RootComponent != Target);
}

FAnimNextConvertRootMotionToWorldSpaceTask::FAnimNextConvertRootMotionToWorldSpaceTask(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, const FTransform& InTransformOrigin, const FTransform& InRootTransform, const FVector& InRootOffsetOrigin, ESpaceConversions InConversions)
	: WeakRootMotionData(InRootMotionData)
	, RootTransform(InRootTransform)
	, TransformOrigin(InTransformOrigin)
	, RootOffsetOrigin(InRootOffsetOrigin)
	, Conversions(InConversions)
{
}

FAnimNextConvertRootMotionToWorldSpaceTask FAnimNextConvertRootMotionToWorldSpaceTask::Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, const FTransform& InTransformOrigin, const FTransform& InRootTransform, const FVector& InRootOffsetOrigin, ESpaceConversions InConversions)
{
	return FAnimNextConvertRootMotionToWorldSpaceTask(InRootMotionData, InTransformOrigin, InRootTransform, InRootOffsetOrigin, InConversions);
}

void FAnimNextConvertRootMotionToWorldSpaceTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	// Default so early-return paths leave a deterministic value.
	if (CaptureAnimSpaceRoot)
	{
		*CaptureAnimSpaceRoot = FTransform::Identity;
	}

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FTransformAnimationAttribute* RootMotionAttribute = (*Keyframe)->Attributes.Find<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
	if (!RootMotionAttribute)
	{
		return;
	}

	FTransform RelativeTransform = EnumHasAnyFlags(Conversions, ESpaceConversions::RootTransformOverride)
		? RootTransform
		: RootMotionAttribute->Value;

	TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionData = WeakRootMotionData.Pin();

	// Offsets rotate the section-local delta around RootOffsetOrigin, then the
	// result is anchored back through AnimSpaceAccumulatedTransform (which
	// holds the accumulated chain when accumulating, or InitialAnimRootTransform
	// when not). BoneMatch post-multiplies to map the rotated trajectory into
	// the matched frame.
	FTransform Delta = RelativeTransform * InitialAnimRootTransform.Inverse();

	if (EnumHasAnyFlags(Conversions, ESpaceConversions::RootTransformOffset))
	{
		Delta *= FTransform(-RootOffsetOrigin);
		Delta = Delta * RootTransform;
		Delta *= FTransform(RootOffsetOrigin);
	}

	RelativeTransform = Delta * AnimSpaceAccumulatedTransform;

	if (EnumHasAnyFlags(Conversions, ESpaceConversions::BoneMatchOffset))
	{
		RelativeTransform = RelativeTransform * BoneMatchTransform;
	}

	if (CaptureAnimSpaceRoot)
	{
		*CaptureAnimSpaceRoot = RelativeTransform;
	}

	if (RootMotionData && EnumHasAnyFlags(Conversions, ESpaceConversions::ComponentToActorRotation))
	{
		TOptional<FQuat> InverseMeshToActorRotation = RootMotionData->GetInverseMeshToActorRotation();
		if (InverseMeshToActorRotation.IsSet())
		{
			RelativeTransform.SetTranslation(InverseMeshToActorRotation.GetValue() * RelativeTransform.GetTranslation());
		}
	}

	if (RootMotionData && EnumHasAnyFlags(Conversions, ESpaceConversions::AnimationToWorld))
	{
		RelativeTransform *= RootMotionData->ActorTransform;
	}
	else if (EnumHasAnyFlags(Conversions, ESpaceConversions::TransformOriginToWorld))
	{
		RelativeTransform *= TransformOrigin;
	}

	if (EnumHasAnyFlags(Conversions, ESpaceConversions::WorldSpaceComponentTransformCompensation) && RootMotionData)
	{
		RelativeTransform = RootMotionData->ComponentToActorTransform * RelativeTransform;
	}

	RootMotionAttribute->Value = RelativeTransform;
}

FAnimNextStoreRootTransformTask::FAnimNextStoreRootTransformTask(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, bool bInComponentHasKeyedTransform, bool bInRootComponentHasKeyedTransform)
	: WeakRootMotionData(InRootMotionData)
	, bComponentHasKeyedTransform(bInComponentHasKeyedTransform)
	, bRootComponentHasKeyedTransform(bInRootComponentHasKeyedTransform)
{

}

FAnimNextStoreRootTransformTask FAnimNextStoreRootTransformTask::Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, bool bInComponentHasKeyedTransform, bool bInRootComponentHasKeyedTransform)
{
	return FAnimNextStoreRootTransformTask(InRootMotionData, bInComponentHasKeyedTransform, bInRootComponentHasKeyedTransform);
}


void FAnimNextStoreRootTransformTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionData = WeakRootMotionData.Pin();
	if (!RootMotionData.IsValid() || !EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	FTransformAnimationAttribute* RootMotionAttribute = (*Keyframe)->Attributes.Find<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
	if (!RootMotionAttribute)
	{
		return;
	}

	const EMovieSceneRootMotionDestination FinalDestination = bForceRootBoneDestination
		? EMovieSceneRootMotionDestination::RootBone
		: RootMotionData->RootDestination;

	FTransform RootTransform = ConvertRootAttributeToComponentSpace(RootMotionAttribute->Value, *RootMotionData);

	const int32 RootBoneIndex = (*Keyframe)->Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);

	switch (FinalDestination)
	{
	case EMovieSceneRootMotionDestination::Discard:
		if (RootBoneIndex != INDEX_NONE)
		{
			(*Keyframe)->Pose.LocalTransformsView[RootBoneIndex] = FTransform::Identity;
		}
		return;

	case EMovieSceneRootMotionDestination::RootBone:
		if (RootBoneIndex != INDEX_NONE)
		{
			(*Keyframe)->Pose.LocalTransformsView[RootBoneIndex] = RootTransform;

			if (bForceRootBoneDestination)
			{
				// Remove the attribute so the outer mixer's extract can cleanly
				// pull root motion from the root bone without double-applying.
				(*Keyframe)->Attributes.Remove<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
				(*Keyframe)->Attributes.Remove<FFloatAnimationAttribute>(AnimMixerComponents->RootTransformWeightAttributeId);
				(*Keyframe)->Attributes.Remove<FIntegerAnimationAttribute>(AnimMixerComponents->RootTransformIsAuthoritativeAttributeId);
			}
		}
		return;

	case EMovieSceneRootMotionDestination::Component:
	case EMovieSceneRootMotionDestination::Actor:
	case EMovieSceneRootMotionDestination::Attribute:
		break;
	}


	const FFloatAnimationAttribute* RootMotionWeight = (*Keyframe)->Attributes.Find<FFloatAnimationAttribute>(AnimMixerComponents->RootTransformWeightAttributeId);
	const float TransformWeight = RootMotionWeight ? RootMotionWeight->Value : 1.f;


	// If we have a root motion provider, convert our prospective root motion transform into a delta so that
	//   pose history and locomotion react correctly. This code also supports blending into and out of gameplay
	//   animation
	if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
	{
		FTransform RootMotionDelta;
		if (RootMotionProvider->ExtractRootMotion((*Keyframe)->Attributes, RootMotionDelta))
		{
			const FTransform ActorHeadingTransform = RootMotionData->ActorTransform;

			TOptional<FQuat> InverseMeshToActorRotation = RootMotionData->GetInverseMeshToActorRotation();
			if (InverseMeshToActorRotation.IsSet())
			{
				// Rotate the translation of the root motion delta to compensate for component rotation
				RootMotionDelta.SetTranslation(InverseMeshToActorRotation.GetValue() * RootMotionDelta.GetTranslation());
			}

			// Blend the root motion delta with Sequencer's desired delta based on its weight
			const FTransform LocomotionDelta  = RootMotionDelta;
			const FTransform DesiredRootDelta = RootTransform * ActorHeadingTransform.Inverse();

			FTransform BlendResult(FMath::Lerp(RootMotionDelta.GetTranslation(), DesiredRootDelta.GetTranslation(), TransformWeight));
			BlendResult.SetRotation(FQuat::Slerp(RootMotionDelta.GetRotation(), DesiredRootDelta.GetRotation(), TransformWeight));

			// Assign the final result in component space
			RootMotionDelta = BlendResult;
			if (InverseMeshToActorRotation.IsSet())
			{
				// Unrotate the translation of the root motion delta to compensate for component rotation
				RootMotionDelta.SetTranslation(InverseMeshToActorRotation->Inverse() * RootMotionDelta.GetTranslation());
			}

#if ENABLE_VISUAL_LOG
			if (FVisualLogger::IsRecording())
			{
				static const TCHAR* LogName = TEXT("MovieSceneRootMotion");
				USceneComponent* Component = RootMotionData->OriginalBoundObject.Get();

				auto DrawMarker = [Component](const FTransform& Transform, const FColor& Color, auto&&... Args)
				{
					FVector Dir(50.f, 0.f, 0.f);

					UE_VLOG_CIRCLE(Component, LogName, Display, Transform.GetLocation(), FVector::UpVector, 10.f, Color, Args...);
					UE_VLOG_ARROW(Component, LogName, Display, Transform.GetLocation(), Transform.GetLocation() + Transform.GetRotation()*Dir, Color, TEXT(""));
				};

				DrawMarker(ActorHeadingTransform,                   FColorList::Black, TEXT("Actor"));
				DrawMarker(RootTransform,                           FColorList::Blue,  TEXT("Sequencer (%.2f%%"),      TransformWeight *100.f);
				DrawMarker(LocomotionDelta * ActorHeadingTransform, FColorList::Red,   TEXT("Locomotion %.2f%%"), (1.f-TransformWeight)*100.f);
				DrawMarker(BlendResult     * ActorHeadingTransform, FColorList::Green, TEXT("Result"));
			}
#endif

			RootMotionProvider->OverrideRootMotion(RootMotionDelta, (*Keyframe)->Attributes);
		}
	}

	// Store the root motion offset so the transform track editor can subtract it during auto-key
	RootMotionData->AppliedRootMotionOffset = RootTransform * RootMotionData->ActorTransform.Inverse();

	if (FinalDestination != EMovieSceneRootMotionDestination::Attribute)
	{
		TWeakPtr<FMovieSceneMixerRootMotionComponentData> WeakRootMotion = RootMotionData;

		auto ApplyRootMotion = [RootTransform, WeakTarget = RootMotionData->Target]
		{
			SCOPED_NAMED_EVENT(UAF_StoreRootTransformTask_ApplyRootMotion, FColor::Orange);
			USceneComponent* TargetComponent = WeakTarget.Get();
			if (TargetComponent && !GActorMovementTracker.IsActive())
			{
				TargetComponent->SetRelativeLocationAndRotation(RootTransform.GetLocation(), RootTransform.GetRotation().Rotator());
			}
		};
		FAnimNextModuleInstance::RunTaskOnGameThread(MoveTemp(ApplyRootMotion));
	}
}

FAnimNextSetRootTransformAttributeTask FAnimNextSetRootTransformAttributeTask::Make(const FTransform& InTransform)
{
	FAnimNextSetRootTransformAttributeTask Task;
	Task.Transform = InTransform;
	return Task;
}

void FAnimNextSetRootTransformAttributeTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// FindOrAdd creates the attribute if it doesn't already exist on the pose
	FTransformAnimationAttribute* RootMotionAttribute =
		(*Keyframe)->Attributes.FindOrAdd<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
	RootMotionAttribute->Value = Transform;
}

FAnimNextBakeRootAttributeToBoneTask FAnimNextBakeRootAttributeToBoneTask::Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData)
{
	FAnimNextBakeRootAttributeToBoneTask Task;
	Task.WeakRootMotionData = InRootMotionData;
	return Task;
}

void FAnimNextBakeRootAttributeToBoneTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (VM.GetActiveNamedSet())
	{
		return;
	}

	if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	const FTransformAnimationAttribute* RootMotionAttribute =
		(*Keyframe)->Attributes.Find<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
	if (!RootMotionAttribute)
	{
		return;
	}

	const int32 RootBoneIndex = (*Keyframe)->Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
	if (RootBoneIndex == INDEX_NONE)
	{
		return;
	}

	FTransform RootTransform = RootMotionAttribute->Value;
	if (TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionData = WeakRootMotionData.Pin())
	{
		RootTransform = ConvertRootAttributeToComponentSpace(RootTransform, *RootMotionData);
	}

	(*Keyframe)->Pose.LocalTransformsView[RootBoneIndex] = RootTransform;
}

FAnimNextResetRootBoneTask FAnimNextResetRootBoneTask::Make()
{
	return FAnimNextResetRootBoneTask();
}

void FAnimNextResetRootBoneTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		return;
	}

	if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 1);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	const int32 RootBoneIndex = (*Keyframe)->Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
	if (RootBoneIndex != INDEX_NONE)
	{
		(*Keyframe)->Pose.LocalTransformsView[RootBoneIndex] = FTransform::Identity;
	}
}

FAnimNextExtractRootMotionTask FAnimNextExtractRootMotionTask::Make(EMovieSceneRootMotionSource InSource)
{
	FAnimNextExtractRootMotionTask Task;
	Task.Source = InSource;
	return Task;
}

void FAnimNextExtractRootMotionTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;

	if (VM.GetActiveNamedSet())
	{
		return;
	}

	if (!EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones | EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
	{
		return;
	}

	TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe || !Keyframe->IsValid())
	{
		return;
	}

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	const int32 RootLODIndex = (*Keyframe)->Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
	if (RootLODIndex == INDEX_NONE)
	{
		return;
	}

	const int32 NumBones = (*Keyframe)->Pose.LocalTransformsView.Num();
	int32 SourceLODIndex = RootLODIndex;

	if (Source == EMovieSceneRootMotionSource::FirstAnimatedChild)
	{
		// Find the first direct child of root that has a non-zero-translation transform
		const TArrayView<const FBoneIndexType> ParentMap = (*Keyframe)->Pose.GetLODBoneIndexToParentLODBoneIndexMap();

		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			if (ParentMap.IsValidIndex(BoneIdx)
				&& ParentMap[BoneIdx] == RootLODIndex)
			{
				FTransform BoneTransform = (*Keyframe)->Pose.LocalTransformsView[BoneIdx];
				if (!BoneTransform.GetTranslation().IsNearlyZero())
				{
					SourceLODIndex = BoneIdx;
					break;
				}
			}
		}
	}

	if (SourceLODIndex < 0 || SourceLODIndex >= NumBones)
	{
		return;
	}

	const auto& RefPose = (*Keyframe)->Pose.GetRefPose();
	FTransform SourceBoneTransform = (*Keyframe)->Pose.LocalTransformsView[SourceLODIndex];

	// When extracting from a child bone, use the delta from the rest pose
	// so the root motion only captures locomotion, not the skeletal offset
	// (e.g., pelvis height). Restore the ref pose on the child bone so
	// the skeleton retains its correct shape.
	FTransform RootMotionTransform;
	if (SourceLODIndex != RootLODIndex)
	{
		FTransform RefTransform = RefPose.ReferenceLocalTransforms[SourceLODIndex];
		RootMotionTransform = RefTransform.Inverse() * SourceBoneTransform;

		// Keep only yaw. The child bone's pitch and roll belong to the rendered
		// pose, not the actor's transform.
		FQuat Swing, Twist;
		RootMotionTransform.GetRotation().ToSwingTwist(FVector::UpVector, Swing, Twist);
		RootMotionTransform.SetRotation(Twist);

		(*Keyframe)->Pose.LocalTransformsView[SourceLODIndex] = RefTransform;
	}
	else
	{
		RootMotionTransform = SourceBoneTransform;
		(*Keyframe)->Pose.LocalTransformsView[SourceLODIndex] = FTransform::Identity;
	}

	FTransformAnimationAttribute* RootMotionAttribute =
		(*Keyframe)->Attributes.FindOrAdd<FTransformAnimationAttribute>(AnimMixerComponents->RootTransformAttributeId);
	FFloatAnimationAttribute* RootMotionWeight =
		(*Keyframe)->Attributes.FindOrAdd<FFloatAnimationAttribute>(AnimMixerComponents->RootTransformWeightAttributeId);

	RootMotionAttribute->Value = RootMotionTransform;
	RootMotionWeight->Value = 1.f;
}

UMovieSceneRootMotionSystem::UMovieSceneRootMotionSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	RelevantComponent = AnimMixerComponents->MixerRootMotion;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// This must be run before the anim mixer to ensure that the anim mixer sets up its program correctly with root motion
		DefineImplicitPrerequisite(GetClass(), UMovieSceneAnimMixerSystem::StaticClass());

		// Must run after bound object instantiation so that bound child entities
		// (which carry RootDestination + MixerRootMotion) exist when we process them
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());

		DefineImplicitPrerequisite(UByteChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneComponentTransformSystem::StaticClass(), GetClass());
	}
}

bool UMovieSceneRootMotionSystem::IsTransformKeyed(const FObjectKey& Object) const
{
	return ObjectsWithTransforms.Contains(Object);
}

void UMovieSceneRootMotionSystem::OnLink()
{
	UMovieSceneAnimMixerSystem* AnimMixer = Linker->LinkSystem<UMovieSceneAnimMixerSystem>();
	Linker->SystemGraph.AddReference(AnimMixer, this);

	UE::MovieScene::GActorMovementTracker.Initialize();
}

void UMovieSceneRootMotionSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		UMovieSceneAnimMixerSystem* AnimMixer = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
		if (!AnimMixer)
		{
			// Should never exist without the anim mixer
			return;
		}

		// Remove expiring root motions
		FEntityTaskBuilder()
		.Write(AnimMixerComponents->MixerRootMotion)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity( &Linker->EntityManager,
			[](TSharedPtr<FMovieSceneMixerRootMotionComponentData>& OutRootMotion)
			{
				OutRootMotion = nullptr;
			}
		);

		// Set up root motion behaviors
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->BoundObjectKey)
		.Read(BuiltInComponents->BoundObject)
		.ReadOptional(AnimMixerComponents->GapBehavior)
		.ReadOptional(AnimMixerComponents->EntityOwner)
		.Write(AnimMixerComponents->Target)
		.Write(AnimMixerComponents->MixerRootMotion)
		.FilterAll({ AnimMixerComponents->RootDestination, BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerAllocation( &Linker->EntityManager,
			[this, AnimMixer, BuiltInComponents](
				FEntityAllocationIteratorItem Item,
				TRead<FMovieSceneEntityID> EntityIDs,
				TRead<FRootInstanceHandle> RootInstanceHandles,
				TRead<FObjectKey> BoundObjectKeys,
				TRead<UObject*> BoundObjects,
				TReadOptional<EMovieSceneRootMotionGapBehavior> GapBehaviors,
				TReadOptional<FObjectKey> EntityOwners,
				TWrite<TInstancedStruct<FMovieSceneMixedAnimationTarget>> OutTargets,
				TWrite<TSharedPtr<FMovieSceneMixerRootMotionComponentData>> OutRootMotions)
			{
				const FEntityAllocation* Allocation = Item.GetAllocation();
				const int32              Num        = Allocation->Num();


				// @todo: figure out restore state semantics with root motion
				const bool bWantsRestore = false; 
				const bool bCapturePreAnimatedState = Linker->PreAnimatedState.IsCapturingGlobalState() || bWantsRestore;

				FPreAnimatedEntityCaptureSource* EntityMetaData = nullptr;
				TSharedPtr<FPreAnimatedComponentTransformStorage> ComponentTransformStorage;
				if (bCapturePreAnimatedState)
				{
					EntityMetaData = Linker->PreAnimatedState.GetOrCreateEntityMetaData();
					ComponentTransformStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedComponentTransformStorage>();
				}

				static const FName PreAnimatedTransformName("Transform");
				for (int32 Index = 0; Index < Num; ++Index)
				{
					USceneComponent* BoundObject = Cast<USceneComponent>(BoundObjects[Index]);
					if (!BoundObject)
					{
						continue;
					}

					if (bCapturePreAnimatedState)
					{
						const FMovieSceneEntityID EntityID           = EntityIDs[Index];
						const FRootInstanceHandle RootInstanceHandle = RootInstanceHandles[Index];
						const FCachePreAnimatedValueParams CacheParams;

						// Track transform for the component and the root component
						FPreAnimatedStateEntry Entry = ComponentTransformStorage->MakeEntry(BoundObject, PreAnimatedTransformName);
						EntityMetaData->BeginTrackingEntity(Entry, EntityID, RootInstanceHandle, bWantsRestore);
						ComponentTransformStorage->CachePreAnimatedTransform(CacheParams, BoundObject);

						USceneComponent* RootComponent = BoundObject->GetOwner()->GetRootComponent();
						if (RootComponent && RootComponent != BoundObject)
						{
							FPreAnimatedStateEntry RootEntry = ComponentTransformStorage->MakeEntry(RootComponent, PreAnimatedTransformName);
							EntityMetaData->BeginTrackingEntity(RootEntry, EntityID, RootInstanceHandle, bWantsRestore);
							ComponentTransformStorage->CachePreAnimatedTransform(CacheParams, RootComponent);
						}
					}

					FObjectKey BoundObjectKey = BoundObjectKeys[Index];
					TInstancedStruct<FMovieSceneMixedAnimationTarget>& OutTarget = OutTargets[Index];
					TSharedPtr<FMovieSceneMixerRootMotionComponentData>& OutRootMotion = OutRootMotions[Index];

					// For new entities, we 'resolve' the animation target so if 'Automatic' is picked we choose the right one automatically.
					OutTarget = UMovieSceneAnimMixerSystem::ResolveAnimationTarget(BoundObjectKey, OutTarget);

					FMovieSceneAnimMixerKey Key{BoundObjectKey, OutTarget};

					if (OutRootMotion)
					{
						AnimMixer->AssignRootMotion(BoundObjectKey, OutRootMotion);
					}
					else if (TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion = AnimMixer->FindRootMotion(BoundObjectKey))
					{
						OutRootMotion = RootMotion;
					}
					else
					{
						// Make a new one
						OutRootMotion = MakeShared<FMovieSceneMixerRootMotionComponentData>();
						AnimMixer->AssignRootMotion(BoundObjectKey, OutRootMotion);
					}

					USceneComponent* Target = Cast<USceneComponent>(BoundObjectKey.ResolveObjectPtr());
					OutRootMotion->OriginalBoundObject = Target;

					// Store gap behavior data from the root motion section entity
					if (GapBehaviors.IsValid())
					{
						OutRootMotion->GapBehavior = GapBehaviors[Index];
					}
					if (EntityOwners.IsValid())
					{
						UMovieSceneRootMotionSection* Section = Cast<UMovieSceneRootMotionSection>(EntityOwners[Index].ResolveObjectPtr());
						OutRootMotion->WeakRootMotionSection = Section;
					}
				}
			}
		);
	}

	ObjectsWithTransforms.Reset();

	// Gather which objects have transforms
	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.FilterAll({ TrackComponents->ComponentTransform.PropertyTag, BuiltInComponents->CustomPropertyIndex })
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
	.Iterate_PerAllocation(&Linker->EntityManager,
		[this](const FEntityAllocation* Allocation, const FObjectKey* RootMotionTargets)
		{
			const int32 Num = Allocation->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				this->ObjectsWithTransforms.Add(RootMotionTargets[Index]);
			}
		}
	);

	ObjectsWithTransforms.Shrink();
}

void UMovieSceneRootMotionSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	UMovieSceneAnimMixerSystem* AnimMixer = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
	if (!AnimMixer)
	{
		return;
	}

	FTaskID WaitForAllTransforms = FEntityTaskBuilder()
	.Write(BuiltInComponents->CustomPropertyIndex)
	.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
	.Fork_PerAllocation<FNoopTask>(&Linker->EntityManager, TaskScheduler);

	// Gather root destination results
	FTaskID GatherRootDestinationTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObjectKey)
	.Read(BuiltInComponents->ByteResult)
	.Write(AnimMixerComponents->RootDestination)
	.Write(AnimMixerComponents->MixerRootMotion)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.Schedule_PerEntity<FGatherRootDestinations>(&Linker->EntityManager, TaskScheduler, AnimMixer);

	// Reset root motion data and gather current component/actor transforms
	FTaskParams InitialzeParams(TEXT("Initialize Root Motion"));
	InitialzeParams.ForceGameThread();
	InitialzeParams.bForcePropagateDownstream = true;
	InitialzeParams.bForcePrePostTask = true;

	FTaskID InitializeRootMotion = FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.ReadAnyOf(
		BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5],
		BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]
	)
	.FilterAll({ TrackComponents->ComponentTransform.PropertyTag, BuiltInComponents->CustomPropertyIndex, TrackComponents->Tags.AnimMixerPoseProducer })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetParams(InitialzeParams)
	.Schedule_PerEntity<FInitializeRootMotionTask>(&Linker->EntityManager, TaskScheduler, AnimMixer);

	TaskScheduler->AddPrerequisite(WaitForAllTransforms,      InitializeRootMotion);
	TaskScheduler->AddPrerequisite(GatherRootDestinationTask, InitializeRootMotion);
}
