// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AttributesRuntime.h"
#include "LODPose.h"
#include "Misc/FrameTime.h"
#include "UObject/ObjectKey.h"

class UMovieSceneEntitySystemLinker;
class UMovieSceneAnimationMixerTrack;
class USkeletalMeshComponent;

namespace UE::MovieScene { struct FInstanceHandle; }

namespace UE::MovieScene::AnimMixerBakeEvaluation
{

// Filter for controlling which mixer entries are included during bake evaluation.
struct FBakeFilter
{
	// If non-empty, only include entries whose EntityOwner matches
	TSet<FObjectKey> IncludeOnlySections;

	// Entries whose EntityOwner matches are excluded
	TSet<FObjectKey> ExcludeSections;

	// If >= 0, only entries with priority >= this value are included
	int32 MinPriority = -1;

	// If >= 0, only entries with priority <= this value are included
	int32 MaxPriority = -1;

	// When true, the ConvertRootMotionToWorldSpaceTask is omitted from the
	// evaluation program. The resulting RootMotionTransform will be in
	// animation space rather than world space.
	bool bSkipRootMotionConversion = false;

	// When set, the conversion task writes the pre-conversion animation-space
	// root here. Populated automatically by FBakeEvaluationScope.
	TSharedPtr<FTransform> CaptureAnimSpaceRoot;
};

struct FBakeResult
{
	FFrameTime Time;
	FTransform RootMotionTransform = FTransform::Identity;
	FTransform AnimationSpaceRootMotion = FTransform::Identity;
	UE::UAF::FLODPoseHeap Pose;
	FBlendedHeapCurve Curves;
	UE::Anim::FHeapAttributeContainer Attributes;

	bool IsValid() const { return Pose.IsValid(); }
};

// Evaluate the mixer at a single time, capturing results through the bake target system.
// Temporarily swaps the track's target, sets a bake filter on the mixer system,
// then re-evaluates the ECS and restores original state.
MOVIESCENEANIMMIXER_API FBakeResult EvaluateAtTime(
	UMovieSceneEntitySystemLinker* Linker,
	FInstanceHandle InstanceHandle,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	FFrameTime Time,
	const FBakeFilter& Filter = {});

// Evaluate the mixer at regular intervals from StartTime (inclusive) for NumSamples.
// More efficient than calling EvaluateAtTime in a loop because save/restore only
// happens once around the entire batch.
MOVIESCENEANIMMIXER_API TArray<FBakeResult> EvaluateRange(
	UMovieSceneEntitySystemLinker* Linker,
	FInstanceHandle InstanceHandle,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	FFrameTime StartTime,
	FFrameTime FrameStep,
	int32 NumSamples,
	const FBakeFilter& Filter = {});

} // namespace UE::MovieScene::AnimMixerBakeEvaluation
