// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Misc/FrameTime.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"

class UMovieSceneEntitySystemLinker;
class UMovieSceneAnimationMixerTrack;
class UMovieSceneSection;

struct FMovieSceneBoneMatchData;

namespace UE::MovieScene::AnimMixerBoneMatching
{

struct FBoneMatchingContext
{
	UMovieSceneEntitySystemLinker* Linker = nullptr;
	FInstanceHandle InstanceHandle;
	UMovieSceneAnimationMixerTrack* MixerTrack = nullptr;

	// Sequencer playhead time, used when MatchTimeMode == AtCurrentTime
	FFrameTime CurrentTime;
};

// Compute the bone match transform for a target section.
// Evaluates the ECS at the resolved match time to get both the underlying
// pose (everything except the target, capped at the target's priority) and
// the target section's isolated pose, then computes the selective per-axis
// match offset between them.
//
// Returns a copy of Settings with MatchTransform, bIsValid, and bIsDirty updated.
MOVIESCENEANIMMIXER_API FMovieSceneBoneMatchData ComputeBoneMatch(
	UMovieSceneSection* TargetSection,
	const FMovieSceneBoneMatchData& Settings,
	const FBoneMatchingContext& Context);

// Returns true when an edit to UpdatedSection could affect Other's bone match
// transform. Considers UpdatedSection's row index (must sit at or below Other's,
// to fall within ComputeBoneMatch's underlying-pose filter) and its start frame
// relative to MatchKeyTime. Always returns true when UpdatedSection is Other's
// explicit ReferenceSection.
MOVIESCENEANIMMIXER_API bool DoesSectionAffectBoneMatch(
	UMovieSceneSection* UpdatedSection,
	UMovieSceneSection* Other,
	FFrameNumber MatchKeyTime,
	UMovieSceneSection* OtherReferenceSection);

// Propagate bone match rebakes from UpdatedSection to every bone-matched section
// on the mixer track whose underlying pose depends on it. Recurses through the
// dependency chain using Visited to prevent cycles.
MOVIESCENEANIMMIXER_API void PropagateRematch(
	UMovieSceneSection* UpdatedSection,
	const FBoneMatchingContext& Context,
	TSet<UMovieSceneSection*>& Visited);

} // namespace UE::MovieScene::AnimMixerBoneMatching
