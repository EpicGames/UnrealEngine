// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMovieSceneSequence;
class UMovieSceneAnimationMixerTrack;

// Dependency graph for bus-target mixers. Callers populate Nodes, then
// call TopologicalSort or IsReachable.
struct MOVIESCENEANIMMIXER_API FBusDependencyGraph
{
	struct FNode
	{
		FName WritesToBus;
		TArray<FName> ReadsFromBuses;

		// Set by TopologicalSort
		int32 InDegree = 0;
		TArray<int32> Dependents;
	};

	TArray<FNode> Nodes;

	// Kahn's algorithm. Returns sorted node indices; nodes in cycles keep
	// InDegree > 0 and are excluded. Self-references are skipped.
	TArray<int32> TopologicalSort();

	// True if TargetBus is reachable from StartBus through dependency edges.
	bool IsReachable(FName StartBus, FName TargetBus) const;
};

struct MOVIESCENEANIMMIXER_API FAnimMixerBusValidationResult
{
	TArray<FString> Errors;
	TArray<FString> Warnings;

	bool HasErrors() const { return !Errors.IsEmpty(); }
	bool HasWarnings() const { return !Warnings.IsEmpty(); }
	bool IsClean() const { return Errors.IsEmpty() && Warnings.IsEmpty(); }
};

// Bus topology utilities. Functions that take mixer tracks expect all tracks
// for a single bound object.
struct MOVIESCENEANIMMIXER_API FAnimMixerBusUtils
{
	// Gather all bus names across the sequence hierarchy
	static TArray<FName> GatherBusNamesFromSequence(UMovieSceneSequence* RootSequence);

	// Build a dependency graph from mixer tracks and return topologically
	// sorted bus names. Cyclic buses are excluded (with an ensure).
	static TArray<FName> ComputeBusEvaluationOrder(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks);

	// Validate bus topology: cycles, self-references, duplicate writers, orphans.
	static FAnimMixerBusValidationResult ValidateBusTopology(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks);

	// Would adding a bus section reading BusName to Track create a cycle?
	static bool WouldBusSectionCreateCycle(FName BusName, UMovieSceneAnimationMixerTrack* Track,
		TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks);

	// Would setting Track's target to BusName create a cycle?
	static bool WouldBusTargetCreateCycle(FName BusName, UMovieSceneAnimationMixerTrack* Track,
		TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks);
};
