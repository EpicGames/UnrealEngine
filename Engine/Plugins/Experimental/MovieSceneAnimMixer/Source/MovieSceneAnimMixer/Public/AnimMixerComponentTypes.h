// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertyMetaData.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyTraits.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "StructUtils/InstancedStruct.h"
#include "EvaluationVM/EvaluationTask.h"
#include "AnimSequencerInstanceProxy.h"
#include "MovieSceneAnimMixerBlendProvider.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "Channels/MovieSceneInterpolation.h"

enum class EMovieSceneRootMotionDestination : uint8;

struct FMovieSceneAnimMixerEntry;

class UMovieSceneAnimationMixerLayer;

// Builder for layer-level blend tasks. Stored as an ECS component on entities
// produced by blend type systems (e.g., the mask system). Called by the mixer
// at evaluation time with the computed layer weight.
struct FLayerBlendTaskBuilder
{
	TFunction<TSharedPtr<FAnimNextEvaluationTask>(double LayerWeight)> BuildTask;

	// Maximum mask passthrough across all bones in the layer's accumulated mask, in [0,1].
	// 1.0 = at least one bone fully exposes the layer (or no mask at all); 0.0 = fully masked out.
	// Used by the mixer's anim-notify exposure pass to decide how much a layer suppresses notifies
	// from lower layers.
	float MaskMaxBoneWeight = 1.0f;

	bool IsValid() const { return static_cast<bool>(BuildTask); }
	TSharedPtr<FAnimNextEvaluationTask> Build(double LayerWeight) const { return BuildTask ? BuildTask(LayerWeight) : nullptr; }
};

namespace UE::MovieScene
{
	struct FAnimMixerComponentTypes
	{
		MOVIESCENEANIMMIXER_API static FAnimMixerComponentTypes* Get();
		MOVIESCENEANIMMIXER_API static void Destroy();

		// A root motion transform in animation space to be applied (after space conversion)
		MOVIESCENEANIMMIXER_API static const UE::Anim::FAttributeId RootTransformAttributeId;
		MOVIESCENEANIMMIXER_API static const UE::Anim::FAttributeId RootTransformWeightAttributeId;
		// Internal flag marking a section as authoritative source of root motion.
		// Some sections, e.g. stitch sections, should not have their root motion blended with others, since it's using motion matching to blend into the animation already.
		MOVIESCENEANIMMIXER_API static const UE::Anim::FAttributeId RootTransformIsAuthoritativeAttributeId;

		TComponentTypeID<int32> Priority;
		TComponentTypeID<TInstancedStruct<FMovieSceneMixedAnimationTarget>> Target;
		TComponentTypeID<TSharedPtr<FAnimNextEvaluationTask>> Task;
		TComponentTypeID<TSharedPtr<FAnimNextEvaluationTask>> MixerTask;
		TComponentTypeID<TInstancedStruct<FMovieSceneAnimMixerBlendProviderData>> BlendProviderData;
		TComponentTypeID<TSharedPtr<FMovieSceneAnimMixerEntry>> MixerEntry;
		TComponentTypeID<FMovieSceneRootMotionSettings> RootMotionSettings;
		TComponentTypeID<EMovieSceneRootMotionDestination> RootDestination;
		TComponentTypeID<FObjectComponent> MeshComponent;
		TComponentTypeID<TSharedPtr<FMovieSceneMixerRootMotionComponentData>> MixerRootMotion;
		TComponentTypeID<TObjectPtr<UMovieSceneAnimationMixerLayer>> MixerLayer;
		TComponentTypeID<TObjectPtr<UMirrorDataTable>> MirrorTable;

		TComponentTypeID<FTransform> BoneMatchTransform;
		TComponentTypeID<EMovieSceneRootMotionGapBehavior> GapBehavior;

		// The object (section or track) that owns/created this entity
		TComponentTypeID<FObjectKey> EntityOwner;

		// Layer-level blend task builder. Provided by mask system or simple blend decorations.
		// Entities with this + MixerLayer + BoundObjectKey define how a layer blends with layers below.
		// The builder is called with the computed layer weight and returns the blend task.
		TComponentTypeID<FLayerBlendTaskBuilder> LayerBlendTask;

		// Bus name for bus sections. Identifies which named bus this section reads from.
		TComponentTypeID<FName> BusName;

		struct
		{
			FComponentTypeID Transition;
			FComponentTypeID SkipPoseWeight;
			FComponentTypeID LayerWeight;
			FComponentTypeID PreBakeRootMotion;
		} Tags;

	private:
		FAnimMixerComponentTypes();
	};

}