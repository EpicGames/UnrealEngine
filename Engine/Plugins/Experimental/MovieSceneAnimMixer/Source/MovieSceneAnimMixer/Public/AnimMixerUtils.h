// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Component/AnimNextComponent.h"
#include "Module/AnimNextModule.h"
#include "Module/UAFWeakSystemReference.h"

struct FAnimMixerUtils
{
	static void RegisterUAFSystem(UUAFComponent* InComponent)
	{
		if (InComponent)
		{
			InComponent->RegisterSystem();
		}
	}

	static void UnregisterUAFSystem(UUAFComponent* InComponent)
	{
		if (InComponent)
		{
			InComponent->UnregisterSystem();
		}
	}

	static void SetUAFComponentModule(UUAFComponent* InComponent, TObjectPtr<const UUAFSystem> InModule)
	{
		if (InComponent)
		{
			InComponent->SetAssetFromObject(InModule);
		}
	}

	static bool IsUAFModuleValid(UUAFComponent* InComponent)
	{
		return InComponent ? InComponent->IsModuleValid() : false;
	}

	static TObjectPtr<const UUAFSystem> GetUAFModule(UUAFComponent* InComponent)
	{
		return InComponent ? InComponent->GetSystemReference().GetSystem() : nullptr;
	}
	
};
