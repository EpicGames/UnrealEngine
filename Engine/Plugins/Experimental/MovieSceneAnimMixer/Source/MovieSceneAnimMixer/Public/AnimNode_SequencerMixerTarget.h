// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"

#include "AnimNode_SequencerMixerTarget.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SequencerMixerTarget final : public FAnimNode_Base
{
	GENERATED_BODY()

	static MOVIESCENEANIMMIXER_API const FName DefaultTargetName;

	// The source input. This is passed through if there is no animation received from the level sequence, otherwise it's blended in.
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	// The target name for the level sequence to match with when applying its animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Setttings, meta=(CustomizePropery))
	FName TargetName;

	// We expose this so that the anim slot node can (through IAnimGraph_SequencerMixerTargetConnector) be backwards compatible with the mixer when the mixer plugin is enabled
	static void ApplySequencerMixedPose(FPoseContext& Output, FName InTargetName);

	// FAnimNode_Base overrides
	MOVIESCENEANIMMIXER_API FAnimNode_SequencerMixerTarget();
	MOVIESCENEANIMMIXER_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	MOVIESCENEANIMMIXER_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	MOVIESCENEANIMMIXER_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	MOVIESCENEANIMMIXER_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
};
