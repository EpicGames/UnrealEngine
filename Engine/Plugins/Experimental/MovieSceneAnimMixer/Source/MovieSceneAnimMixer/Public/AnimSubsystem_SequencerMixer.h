// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "EvaluationVM/EvaluationTask.h"
#include "Animation/AnimSubsystemInstance.h"
#include "Animation/AnimInstance.h"
#include "MovieSceneAnimMixerNotifyTypes.h"

#include "AnimSubsystem_SequencerMixer.generated.h"

/** Allows sequencer's anim mixer system to communicate with mixer nodes in an anim blueprint*/
USTRUCT()
struct FAnimSubsystem_SequencerMixer : public FAnimSubsystemInstance
{
	GENERATED_BODY()

	// FAnimSubsystemInstance interface
	MOVIESCENEANIMMIXER_API virtual void Initialize_WorkerThread() override;

	void RegisterEvalTask(FName TargetName, TSharedPtr<FAnimNextEvaluationTask> EvalTask);
	
	const TSharedPtr<FAnimNextEvaluationTask>* GetEvalTask(FName TargetName) const;

	void ResetEvalTasks();

	void StagePendingNotifyBatches(FName TargetName, TArray<FSequencerMixerPendingNotifyBatch>&& InBatches);

	TArray<FSequencerMixerPendingNotifyBatch> ConsumePendingNotifyBatches(FName TargetName);

private:
	TMap<FName, TSharedPtr<FAnimNextEvaluationTask>> TargetToTaskMap;
	TMap<FName, TArray<FSequencerMixerPendingNotifyBatch>> TargetToPendingNotifiesMap;

};
