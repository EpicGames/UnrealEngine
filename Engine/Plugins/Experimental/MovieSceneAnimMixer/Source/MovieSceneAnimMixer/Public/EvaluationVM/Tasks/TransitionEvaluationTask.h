// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "TransitionEvaluationTask.generated.h"

/**
 * FAnimNextTransitionEvaluationTask
 *
 * Abstract base class for all Sequencer transition evaluation tasks.
 * Provides common from/to task references and a virtual Update method
 * that the mixer system calls each frame before execution.
 *
 * Derived classes must implement Execute() and can override Update() to
 * perform additional per-frame setup, such as managing state for stateful
 * transitions like dead blending.
 */
USTRUCT()
struct MOVIESCENEANIMMIXER_API FAnimNextTransitionEvaluationTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextTransitionEvaluationTask)

	FAnimNextTransitionEvaluationTask() = default;
	virtual ~FAnimNextTransitionEvaluationTask() = default;

	/**
	 * Update the task with the current frame's data.
	 * Called by the mixer system each frame before the task is executed.
	 *
	 * The base implementation stores the from/to tasks and blend weight.
	 * Derived classes should call the base implementation and then perform
	 * any additional per-frame setup.
	 */
	virtual void Update(
		const TSharedPtr<FAnimNextEvaluationTask>& InFromTask,
		const TSharedPtr<FAnimNextEvaluationTask>& InToTask,
		double InBlendWeight,
		float InDeltaTime);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override { LowLevelFatalError(TEXT("Pure virtual not implemented (FAnimNextTransitionEvaluationTask::Execute)")); }

protected:

	/** The task that evaluates the "from" section's pose */
	TSharedPtr<FAnimNextEvaluationTask> FromTask;

	/** The task that evaluates the "to" section's pose */
	TSharedPtr<FAnimNextEvaluationTask> ToTask;

	/** Current blend weight (0 = full from, 1 = full to) */
	double BlendWeight = 0.0;

	/** Delta time since last update */
	float DeltaTime = 0.0f;
};
