// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneAnimationMixerItemInterface.h"
#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"

#include "MovieSceneAnimBusSection.generated.h"

// Section type for reading a stored pose from a named bus into a mixer layer.
// Place this on a mixer layer to pull in the pose that another mixer wrote to
// the specified bus via a bus target. Defaults to infinite range but can be
// trimmed to only pull from the bus during part of the sequence.
UCLASS(MinimalAPI, DisplayName="Bus")
class UMovieSceneAnimBusSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
public:

	UMovieSceneAnimBusSection();

	GENERATED_BODY()

	// Name of the bus to read the pose from
	UPROPERTY(EditAnywhere, Category="Animation Bus")
	FName BusName;

	// Weight channel controlling how much the bus pose contributes to the mix
	UPROPERTY()
	FMovieSceneFloatChannel Weight;


	virtual FColor GetMixerItemTint() const override
	{
		return BusSectionTintColor;
	}

private:

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	static constexpr FColor BusSectionTintColor = FColor(140, 70, 180);
};
