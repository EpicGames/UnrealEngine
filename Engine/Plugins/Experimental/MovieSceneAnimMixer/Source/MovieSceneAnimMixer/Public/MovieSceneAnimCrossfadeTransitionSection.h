// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneAnimCrossfadeTransitionSection.generated.h"

/**
 * Crossfade transition section that blends between two animation sections using a blend curve.
 * The blend curve controls the interpolation: 0.0 = full FromSection, 1.0 = full ToSection.
 */
UCLASS(MinimalAPI, DisplayName="Crossfade Transition")
class UMovieSceneAnimCrossfadeTransitionSection : public UMovieSceneAnimTransitionSectionBase
{
	GENERATED_BODY()

public:

	UMovieSceneAnimCrossfadeTransitionSection(const FObjectInitializer& ObjInit);

	/**
	 * The blend curve controlling the transition.
	 * Value of 0.0 = full FromSection pose
	 * Value of 1.0 = full ToSection pose
	 */
	UPROPERTY()
	FMovieSceneFloatChannel BlendCurve;

	/** Initialize the blend curve with a default linear ramp from 0 to 1 */
	MOVIESCENEANIMMIXER_API virtual void InitializeDefaultCurve() override;

	//~ UMovieSceneAnimTransitionSectionBase interface
	virtual TSharedPtr<FAnimNextTransitionEvaluationTask> CreateTransitionTask() const override;
	virtual FName GetTransitionIconStyleName() const override;
	virtual FText GetTransitionDisplayName() const override;

protected:

	//~ UMovieSceneAnimTransitionSectionBase interface
	virtual void RebuildChannelProxy(FMovieSceneChannelProxyData& Channels) override;
	virtual const FMovieSceneFloatChannel* GetBlendWeightChannel() const override { return &BlendCurve; }
};
