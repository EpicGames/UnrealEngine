// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneAnimInertialDeadBlendTransitionSection.generated.h"

class UCurveFloat;

/**
 * Inertial dead blend transition section that provides smooth, velocity-aware blending
 * between two animation sections.
 * 
 * Unlike crossfade which simply interpolates between poses, dead blending:
 * - Captures the velocity of the outgoing animation
 * - Extrapolates that motion forward during the blend
 * - Provides smoother transitions especially for fast-moving animations
 */
UCLASS(MinimalAPI, DisplayName="Inertial Dead Blend Transition")
class UMovieSceneAnimInertialDeadBlendTransitionSection : public UMovieSceneAnimTransitionSectionBase
{
	GENERATED_BODY()

public:

	UMovieSceneAnimInertialDeadBlendTransitionSection(const FObjectInitializer& ObjInit);

	// ---- Blend Parameters ----

	/** The blend mode to use for the alpha curve */
	UPROPERTY(EditAnywhere, Category = "Blending")
	EAlphaBlendOption BlendMode = EAlphaBlendOption::HermiteCubic;

	/** Optional custom blend curve (used when BlendMode is set to Custom) */
	UPROPERTY(EditAnywhere, Category = "Blending")
	TObjectPtr<UCurveFloat> CustomBlendCurve = nullptr;

	// ---- Extrapolation Parameters ----

	/**
	 * The average half-life of decay in seconds to use when extrapolating the animation.
	 * This value is scaled based on how much the outgoing velocity aligns with the
	 * direction toward the target pose.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ExtrapolationHalfLife = 1.0f;

	/**
	 * The minimum half-life of decay in seconds. Used when velocities are very small
	 * or moving away from the target pose.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ExtrapolationHalfLifeMin = 0.05f;

	/**
	 * The maximum half-life of decay in seconds. Limits how long extrapolation can
	 * continue when velocities are small but aligned with the target.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ExtrapolationHalfLifeMax = 1.0f;

	/**
	 * Maximum translation velocity in cm/s. Velocities above this are clamped
	 * to prevent extreme extrapolation.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MaximumTranslationVelocity = 500.0f;

	/**
	 * Maximum rotation velocity in degrees/s. Velocities above this are clamped
	 * to prevent extreme extrapolation.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MaximumRotationVelocity = 360.0f;

	/**
	 * Maximum scale velocity. Velocities above this are clamped
	 * to prevent extreme extrapolation.
	 */
	UPROPERTY(EditAnywhere, Category = "Extrapolation", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float MaximumScaleVelocity = 4.0f;

	//~ UMovieSceneAnimTransitionSectionBase interface
	virtual TSharedPtr<FAnimNextTransitionEvaluationTask> CreateTransitionTask() const override;
	virtual FName GetTransitionIconStyleName() const override;
	virtual FText GetTransitionDisplayName() const override;

protected:

	//~ UMovieSceneAnimTransitionSectionBase interface
	virtual void RebuildChannelProxy(FMovieSceneChannelProxyData& Channels) override;
};
