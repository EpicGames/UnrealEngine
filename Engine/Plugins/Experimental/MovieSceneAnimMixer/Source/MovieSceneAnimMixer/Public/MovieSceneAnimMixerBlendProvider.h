// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "UObject/Interface.h"

#include "MovieSceneAnimMixerBlendProvider.generated.h"

USTRUCT()
struct FMovieSceneAnimMixerBlendProviderData
{
	GENERATED_BODY()
};

UINTERFACE(MinimalAPI)
class ULayerBlendDecoration : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Marker interface for layer blend decorations. Only one layer blend decoration
 * is allowed per mixer layer (mutually exclusive). Used to enforce this constraint
 * in the UI and at runtime.
 */
class ILayerBlendDecoration
{
public:
	GENERATED_BODY()
};
