// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneAnimationMixerItemInterface.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"

#include "MovieSceneAnimMixerLayerWeightSection.generated.h"

// Section for the layer weight decoration. Imports an entity tagged with
// LayerWeight so the mixer can read the evaluated weight channel from
// the owning decoration.
UCLASS(MinimalAPI, DisplayName="Layer Weight")
class UMovieSceneAnimMixerLayerWeightSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
public:

	UMovieSceneAnimMixerLayerWeightSection();

	GENERATED_BODY()

	void SetAnimMixerLayer(UMovieSceneAnimationMixerLayer* InLayer) { OwningLayer = InLayer; }

#if WITH_EDITOR
	virtual bool SupportsVerticalDragging() const override { return false; }
	virtual bool IsVisibleInAddSectionMenu() const override { return false; }
#endif

private:

	UPROPERTY()
	TObjectPtr<UMovieSceneAnimationMixerLayer> OwningLayer;

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	FColor LayerWeightTintColor = FColor(25, 80, 110);
};
