// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include <UAF/BlendMask/UAFBlendMask.h>

#include "MovieSceneAnimationMixerItemInterface.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"

#include "MovieSceneAnimMixerMaskSection.generated.h"

#define UE_API MOVIESCENEANIMMIXER_API

struct FMovieSceneMaskBlendProvider;

UCLASS(MinimalAPI, DisplayName="Mask")
class UMovieSceneAnimMixerMaskSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
public:

	UMovieSceneAnimMixerMaskSection();

	GENERATED_BODY()

	void SetAnimMixerLayer(UMovieSceneAnimationMixerLayer* InLayer) { OwningLayer = InLayer; }

	TObjectKey<UUAFBlendMask> GetBlendMask() { return BlendMask; }

	UE_API void SetBlendMask(UObject* InMask);

#if WITH_EDITOR
	virtual bool SupportsVerticalDragging() const override { return false; }

	virtual bool IsVisibleInAddSectionMenu() const override {return false; }
#endif

private:

	UPROPERTY(EditAnywhere, Category="Default")
	TObjectPtr<UUAFBlendMask> BlendMask = nullptr;

	TSharedPtr<FMovieSceneMaskBlendProvider> BlendProvider;

	UPROPERTY()
	TObjectPtr<UMovieSceneAnimationMixerLayer> OwningLayer;

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	FColor MaskTintColor = FColor(110, 25, 50);
};

#undef UE_API