// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimGraphRuntime_SequencerMixerTargetConnector.h"
#include "Modules/ModuleInterface.h"
#include "HAL/IConsoleManager.h"

MOVIESCENEANIMMIXER_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieSceneAnimMixer, Log, All);

struct FAnimationUpdateContext;

namespace UE::UAF
{
	struct FEvaluationVM;
}

DECLARE_MULTICAST_DELEGATE_TwoParams(FPreEvaluateMixerTasks, const FAnimationUpdateContext&, UE::UAF::FEvaluationVM&);

namespace UE::MovieScene
{
	extern MOVIESCENEANIMMIXER_API FName GSequencerDefaultAnimNextInjectionSite;
	
	extern MOVIESCENEANIMMIXER_API FAutoConsoleVariableRef CVarDefaultAnimNextInjectionSite;

class FMovieSceneAnimMixerModule : public IModuleInterface, public IAnimGraphRuntime_SequencerMixerTargetConnector
{
	// IModuleInterface interface
	MOVIESCENEANIMMIXER_API virtual void StartupModule() override;
	MOVIESCENEANIMMIXER_API virtual void ShutdownModule() override;
	
	virtual void ApplySequencerMixedPose(FPoseContext& Output, FName InTargetName) const override;

};

}
