// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifyQueue.h"
#include "Containers/Array.h"

// One section's notifies plus the blend weight at which to fire them.
// One batch per section so FAnimNotifyQueue::AddAnimNotifies receives the per-section
// InstanceWeight rather than a single combined weight across the mix.
struct FSequencerMixerPendingNotifyBatch
{
	TArray<FAnimNotifyEventReference> Notifies;
	float Weight = 1.f;
};
