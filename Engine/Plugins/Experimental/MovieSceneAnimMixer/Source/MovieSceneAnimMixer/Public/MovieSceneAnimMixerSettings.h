// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "MovieSceneAnimMixerSettings.generated.h"

class UUAFSystem;

UCLASS(config = Engine, defaultconfig, meta=(DisplayName="Anim Mixer"), MinimalAPI)
class UMovieSceneAnimMixerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// Default injection site for the anim mixer to use, when no site is specified in their target. Can be overridden with the console variable Sequencer.AnimNext.DefaultInjectionSite
	UPROPERTY(Config, EditAnywhere, Category="Animation", meta = (AllowedType = "FAnimNextAnimGraph"))
	FName DefaultInjectionSite;

	// The default UAF module to use when a mixer is animating a mesh with an AnimNext component
	UPROPERTY(Config, EditAnywhere, Category="Module")
	TSoftObjectPtr<UUAFSystem> DefaultUAFModule = nullptr;

	MOVIESCENEANIMMIXER_API virtual FName GetCategoryName() const override;

	

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	MOVIESCENEANIMMIXER_API virtual FText GetSectionText() const override;
#endif
	
};
