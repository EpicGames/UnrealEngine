// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Internationalization/Text.h"
#include "MovieSceneSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimTransitionSectionBase.generated.h"

class UMovieSceneAnimationMixerTrack;
class UMovieSceneAnimationMixerLayer;
struct FAnimNextEvaluationTask;

struct FAnimNextTransitionEvaluationTask;

/**
 * Abstract base class for all Anim Mixer transition section types.
 * Transition sections handle blending between two overlapping animation sections on the same layer.
 *
 * Transition tasks follow a create-once, update-each-frame pattern:
 * 1. CreateTransitionTask() is called once during entity import to create the task
 * 2. The mixer system calls Update() on the task each frame to set from/to tasks and timing
 * 3. The task persists as an entity component, allowing it to maintain state across frames
 */
UCLASS(Abstract, MinimalAPI)
class UMovieSceneAnimTransitionSectionBase
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
	GENERATED_BODY()

public:

	UMovieSceneAnimTransitionSectionBase(const FObjectInitializer& ObjInit);

	/** The section that this transition blends FROM (earlier in time) */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> FromSection;

	/** The section that this transition blends TO (later in time) */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> ToSection;

	/** Check if this transition is structurally valid (sections exist, same row, bounded, overlap exists, neither fully contained) */
	MOVIESCENEANIMMIXER_API bool IsValid() const;

	/** Update this transition's bounds to match the current overlap of from/to sections */
	MOVIESCENEANIMMIXER_API bool UpdateBoundsFromSourceSections();

	/** Scale blend weight keys proportionally from old range to new range */
	MOVIESCENEANIMMIXER_API void ScaleKeysToRange(const TRange<FFrameNumber>& OldRange, const TRange<FFrameNumber>& NewRange);

	/** Get the computed overlap range between from and to sections */
	MOVIESCENEANIMMIXER_API TRange<FFrameNumber> ComputeOverlapRange() const;

	/** Get transition progress [0,1] at the given time. 0 = start of transition, 1 = end */
	MOVIESCENEANIMMIXER_API float GetTransitionProgress(FFrameTime InTime) const;

	/** Called after the section range is set to initialize any default curves/channels */
	virtual void InitializeDefaultCurve() {}

	/**
	 * Create the evaluation task for this transition.
	 * Called once during entity import. The task will persist as an entity component
	 * and have its Update() method called each frame by the mixer system.
	 */
	virtual TSharedPtr<FAnimNextTransitionEvaluationTask> CreateTransitionTask() const
		PURE_VIRTUAL(UMovieSceneAnimTransitionSectionBase::CreateTransitionTask, return nullptr;);

	/**
	 * Get the blend weight channel for evaluation.
	 * The result will be used as the blend weight (0 = from, 1 = to).
	 */
	virtual const FMovieSceneFloatChannel* GetBlendWeightChannel() const { return nullptr; }

	/**
	 * Get the icon style name for this transition type.
	 * Used by the editor to display a small icon in the transition section.
	 */
	virtual FName GetTransitionIconStyleName() const { return NAME_None; }

	/**
	 * Get a short display name for this transition type.
	 * Used for tooltips and other UI elements.
	 */
	virtual FText GetTransitionDisplayName() const { return FText::GetEmpty(); }

	//~ IMovieSceneAnimationMixerItemInterface
	virtual FColor GetMixerItemTint() const override { return TransitionTintColor; }

	virtual bool SupportsVerticalDragging() const override { return false; }

	virtual bool IsVisibleInAddSectionMenu() const override {return false; }

	//~ UMovieSceneSection interface
	virtual EMovieSceneChannelProxyType CacheChannelProxy() override final;
#if WITH_EDITOR
	MOVIESCENEANIMMIXER_API virtual bool IsLocalEvalDisabled() const override;
#endif

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

protected:

	/** Subclasses implement to register their specific channels */
	virtual void RebuildChannelProxy(FMovieSceneChannelProxyData& Channels) PURE_VIRTUAL(UMovieSceneAnimTransitionSectionBase::RebuildChannelProxy, );

	/** Tint color for transition sections in the timeline */
	FColor TransitionTintColor = FColor(120, 80, 160, 200); 
};
