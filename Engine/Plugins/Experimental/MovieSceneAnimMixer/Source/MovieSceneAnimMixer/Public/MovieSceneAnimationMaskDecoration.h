// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneAnimMixerBlendProvider.h"
#include "MovieSceneSignedObject.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"

#include "MovieSceneAnimationMaskDecoration.generated.h"

class UMovieSceneAnimationMixerLayer;
class UUAFBlendMask;
class UMovieSceneTrack;

USTRUCT()
struct FMovieSceneMaskBlendProviderData : public FMovieSceneAnimMixerBlendProviderData
{
	GENERATED_BODY()
	TWeakObjectPtr<UUAFBlendMask> BlendMask;
};

/**
 * Decorator for associating a mixer layer with per-bone blend masks (UAF Blend Mask).
 * The mask system will accumulate bone weights from all mask sections on the layer
 * and produce a per-bone blend task for inter-layer blending.
 *
 * Bone, attribute and curve weights can be blended using sequencer easing.
 */
UCLASS(MinimalAPI, DisplayName="Mask")
class UMovieSceneAnimationMaskDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneSectionProviderDecoration
	, public ILayerBlendDecoration
{
public:
	GENERATED_BODY()

	UMovieSceneAnimationMaskDecoration(const FObjectInitializer& ObjInit);

	virtual TArrayView<TObjectPtr<UMovieSceneSection>> GetSections() override { return MaskSections;}

	virtual UMovieSceneSection* CreateNewSection() override;

	MOVIESCENEANIMMIXER_API virtual void AddSection(UMovieSceneSection* InSection) override;

	MOVIESCENEANIMMIXER_API virtual void RemoveSection(UMovieSceneSection& SectionToRemove) override;

	MOVIESCENEANIMMIXER_API virtual bool SupportsMultipleSections() override { return true; }

	MOVIESCENEANIMMIXER_API virtual TSubclassOf<UMovieSceneSection> GetHostedSectionClass() const override;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> MaskSections;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};
