// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "MovieSceneAnimationMixerLayer.generated.h"

class UMovieSceneSection;
class UMovieSceneTrack;
class UMovieSceneAnimationMixerTrack;

/**
 * Represents a single layer in an Animation Mixer track.
 * Layers can contain either:
 * - Multiple sections (of any type/mixed types that can produce animation poses)
 * - A single child track (e.g., Control Rig track) that occupies the entire layer
 *
 * Layers store references to sections/tracks but do not own them. 
 * - Sections are owned by the outer Animation Mixer Track.
 * - Child tracks are owned by the Object Binding as regular tracks are, but they contain a pointer to their parent track and
 *   they will not be shown underneath the Object Binding
 * 
 * This allows efficient reorganization (moving sections between layers) without changing object outers, and allows
 * easier discovery by other code.
 *
 * The layer index is determined by the layer's position in the parent track's Layers array.
 *
 * Inherits from UMovieSceneDecorationContainerObject to support features like:
 * - Masks
 * - Custom blend modes
 */
UCLASS(MinimalAPI)
class UMovieSceneAnimationMixerLayer : public UMovieSceneDecorationContainerObject
{
	GENERATED_BODY()

public:
	UMovieSceneAnimationMixerLayer(const FObjectInitializer& ObjInit);

	/** Check if this layer contains a child track (vs sections) */
	MOVIESCENEANIMMIXER_API bool HasChildTrack() const { return ChildTrack != nullptr; }

	/** Get the child track occupying this layer (if any) */
	MOVIESCENEANIMMIXER_API UMovieSceneTrack* GetChildTrack() const { return ChildTrack; }

	/** Set the child track for this layer */
	MOVIESCENEANIMMIXER_API void SetChildTrack(UMovieSceneTrack* InTrack);

	/** Get all section references in this layer */
	MOVIESCENEANIMMIXER_API const TArray<TObjectPtr<UMovieSceneSection>>& GetSections() const { return Sections; }

	/** Add a section reference to this layer  */
	MOVIESCENEANIMMIXER_API void AddSection(UMovieSceneSection* Section);

	/** Remove a section reference from this layer */
	MOVIESCENEANIMMIXER_API void RemoveSection(UMovieSceneSection* Section);

	/** Check if this layer contains the given section */
	MOVIESCENEANIMMIXER_API bool ContainsSection(UMovieSceneSection* Section) const;

	/** Get the number of sections in this layer */
	MOVIESCENEANIMMIXER_API int32 GetNumSections() const { return Sections.Num(); }

	/** Check if this layer is empty (no sections, no child track) */
	MOVIESCENEANIMMIXER_API bool IsEmpty() const { return Sections.IsEmpty() && ChildTrack == nullptr; }

	MOVIESCENEANIMMIXER_API int32 GetLayerIndex() const;

	MOVIESCENEANIMMIXER_API const UMovieSceneAnimationMixerTrack* GetMixerTrack() const;

#if WITH_EDITORONLY_DATA
	/** Get the display name for this layer */
	MOVIESCENEANIMMIXER_API FText GetDisplayName() const;

	/** Set a custom display name for this layer */
	MOVIESCENEANIMMIXER_API void SetDisplayName(const FText& InDisplayName);

	/** Check if this layer has a custom display name */
	MOVIESCENEANIMMIXER_API bool HasCustomDisplayName() const { return !DisplayName.IsEmpty(); }
#endif

private:
	/** Section references contained in this layer (mutually exclusive with ChildTrack) */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Child track reference occupying this layer (mutually exclusive with Sections) */
	UPROPERTY()
	TObjectPtr<UMovieSceneTrack> ChildTrack;

#if WITH_EDITORONLY_DATA
	/** Optional custom display name for this layer */
	UPROPERTY()
	FText DisplayName;
#endif
};
