// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Decorations/IMovieSceneSectionDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/IMovieSceneTrackDecoration.h"
#include "EntitySystem/IMovieSceneEntityDecorator.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneAnimationMixerItemInterface.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "StructUtils/InstancedStruct.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "MovieSceneTrack.h"
#include "MovieSceneBoneMatchChannel.h"
#include "MovieSceneAnimationMixerTrack.generated.h"

class UMovieSceneAnimationMixerLayer;
class UMovieSceneAnimTransitionSectionBase;
class UMovieSceneAnimCrossfadeTransitionSection;

UENUM()
enum class EMovieSceneRootMotionKeepState : uint8
{
	DontKeep UMETA(ToolTip = "Root motion offsets are discarded when the section ends; the actor returns to its pre-section pose."),
	KeepState UMETA(DisplayName = "Keep", ToolTip = "Root motion offsets accumulated during the section are preserved after it ends; the actor remains at the moved-to location.")
};

USTRUCT()
struct FMovieSceneByteChannelDefaultOnly : public FMovieSceneByteChannel
{
	GENERATED_BODY()

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FMovieSceneByteChannelDefaultOnly> : public TStructOpsTypeTraitsBase2<FMovieSceneByteChannelDefaultOnly>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneByteChannelDefaultOnly> : TMovieSceneChannelTraitsBase<FMovieSceneByteChannelDefaultOnly>
{
	/** Default-only channels cannot be keyed - they only support default values */
	enum { SupportsKeying = false };

#if WITH_EDITOR

	/** Byte channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<uint8> ExtendedEditorDataType;

#endif
};

UCLASS(MinimalAPI)
class UMovieSceneAnimationSectionDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneEntityDecorator
	, public IMovieSceneSectionDecoration
	, public IMovieSceneAnimationMixerItemInterface
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TObjectPtr<UMovieSceneAnimationMixerTrack> MixerTrack;

protected: 

	virtual void ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	virtual FColor GetMixerItemTint() const override
	{
		return MixerTintOverride;
	}

private:
	FColor MixerTintOverride = FColor();
};

// Section that hosts channels from a root motion settings decoration when it lives
// on a track rather than a section. Delegates CacheChannelProxy to the decoration.
UCLASS(MinimalAPI)
class UMovieSceneRootMotionHostSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:
	UMovieSceneRootMotionHostSection()
	{
		bSupportsInfiniteRange = true;
	}

	MOVIESCENEANIMMIXER_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

	MOVIESCENEANIMMIXER_API void SetOwnerDecoration(UObject* InOwnerDecoration);

	UPROPERTY()
	TObjectPtr<UObject> OwnerDecoration;
};

/**
 * Decoration that provides root motion offset settings.
 * Can live on a section (skeletal animation) or on a track (control rig child track).
 * When on a section, channels are hosted by the parent section via IMovieSceneChannelDecoration.
 * When on a track, an internal host section is created via IMovieSceneSectionProviderDecoration.
 */
UCLASS(MinimalAPI, collapseCategories, meta=(DisplayName="Root Motion Settings"))
class UMovieSceneRootMotionSettingsDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneChannelDecoration
	, public IMovieSceneEntityDecorator
	, public IMovieSceneSectionProviderDecoration
	, public IMovieSceneSectionDecoration
	, public IMovieSceneTrackDecoration
{
public:

	GENERATED_BODY()

	UMovieSceneRootMotionSettingsDecoration(const FObjectInitializer& ObjInit);

	MOVIESCENEANIMMIXER_API EMovieSceneRootMotionTransformMode GetRootTransformMode() const;
	MOVIESCENEANIMMIXER_API EMovieSceneRootMotionSource GetRootMotionSource() const;

	// Returns the first loop boundary time for the owning section, or TOptional empty
	// if the section doesn't loop. Uses the section's timing transform to find the
	// actual loop wrap point (preserving sub-frames).
	MOVIESCENEANIMMIXER_API TOptional<FFrameTime> GetFirstLoopBoundary(const FFrameRate& TickResolution) const;

	// Returns the appropriate initial root transform for the given time - either
	// InitialRootTransform for the first loop iteration, or LoopStartRootTransform
	// for subsequent iterations.
	MOVIESCENEANIMMIXER_API FTransform GetInitialRootTransformAtTime(FFrameTime Time, const FFrameRate& TickResolution) const;

	virtual EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData) override;
	virtual void ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	// IMovieSceneSectionProviderDecoration - provides an internal section when on a track
	virtual TArrayView<TObjectPtr<UMovieSceneSection>> GetSections() override;

	// IMovieSceneSectionDecoration - auto-creates Root Motion Target when added to a section
	virtual void OnDecorationAdded(UMovieSceneSection* Section) override;

	// IMovieSceneTrackDecoration - auto-creates Root Motion Target when added to a child track
	virtual void OnDecorationAdded(UMovieSceneTrack* Track) override;

	// Creates the internal host section when the decoration is added to a track
	MOVIESCENEANIMMIXER_API void EnsureHostSection();

	// Transform Offset Channels

	UPROPERTY()
	FMovieSceneDoubleChannel Location[3];

	UPROPERTY()
	FMovieSceneDoubleChannel Rotation[3];

	UPROPERTY(EditAnywhere, Category="Root Transform")
	FVector RootOriginLocation;

	UPROPERTY()
	FMovieSceneByteChannelDefaultOnly RootMotionSpace;

	UPROPERTY()
	FMovieSceneByteChannelDefaultOnly RootMotionSourceChannel;

	UPROPERTY()
	FMovieSceneByteChannelDefaultOnly TransformMode;

	// Bone Matching

	UPROPERTY()
	FMovieSceneBoneMatchChannel BoneMatchChannel;

	// Read the first key from the channel, or return a default (invalid) match
	FMovieSceneBoneMatchData GetBoneMatchData() const
	{
		TMovieSceneChannelData<const FMovieSceneBoneMatchData> Data = BoneMatchChannel.GetData();
		TArrayView<const FMovieSceneBoneMatchData> Values = Data.GetValues();
		return Values.Num() > 0 ? Values[0] : FMovieSceneBoneMatchData();
	}

	bool HasBoneMatch() const
	{
		return BoneMatchChannel.GetNumKeys() > 0;
	}

	// Keep State

	UPROPERTY()
	FMovieSceneByteChannelDefaultOnly KeepStateChannel;

	bool GetKeepStateAfterSectionEnds() const
	{
		return KeepStateChannel.GetDefault().Get((uint8)EMovieSceneRootMotionKeepState::DontKeep)
			== (uint8)EMovieSceneRootMotionKeepState::KeepState;
	}

	UPROPERTY()
	FMovieSceneByteChannelDefaultOnly LoopModeChannel;

	EMovieSceneAnimLoopMode GetLoopMode() const
	{
		return (EMovieSceneAnimLoopMode)LoopModeChannel.GetDefault().Get((uint8)EMovieSceneAnimLoopMode::Accumulated);
	}

	// Cached initial root transform at the section's start time, computed via
	// isolated bake evaluation. Used by AccumulatedOffset mode to compute the
	// delta from the animation's starting position.
	UPROPERTY()
	FTransform InitialRootTransform = FTransform::Identity;

	// Root transform at the animation's natural start (frame 0). When the section
	// is extended left, the first loop starts mid-animation so InitialRootTransform
	// differs from the animation start. After the first loop wraps, all subsequent
	// iterations start from this transform instead.
	UPROPERTY()
	FTransform LoopStartRootTransform = FTransform::Identity;

	UPROPERTY()
	bool bInitialRootTransformValid = false;

	// Internal section for hosting channels when the decoration lives on a track
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> HostSection;

#if WITH_EDITORONLY_DATA
	// When true, the edit mode draws this section's skeleton in the viewport. Useful for bone matching.
	UPROPERTY(Transient)
	bool bShowSkeleton = false;
#endif
};

// Returns true if setting TargetSection to reference CandidateSection
// would create a bone match dependency cycle
MOVIESCENEANIMMIXER_API bool WouldCreateBoneMatchCycle(UMovieSceneSection* CandidateSection, UMovieSceneSection* TargetSection);

/**
 *
 */
UCLASS(MinimalAPI, HideCategories=("RootMotions"))
class UMovieSceneAnimationMixerTrack
	: public UMovieSceneCommonAnimationTrack
{
	GENERATED_BODY()

public:

	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerTrack(const FObjectInitializer& Init);

	virtual bool FixRowIndices() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneDecorationContainerObject* GetDecorationContainerForRow(int32 RowIndex) const override;

	//~ IMovieSceneTrackVirtualAPI interface
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const override;

	virtual void OnSectionAddedImpl(UMovieSceneSection* Secton) override;
	virtual void OnSectionRemovedImpl(UMovieSceneSection* Section) override;
	virtual void OnRemovedFromMovieSceneImpl() override;
#if WITH_EDITOR
	virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params) override;
#endif
	virtual EMovieSceneTrackEasingSupportFlags SupportsEasing(FMovieSceneSupportsEasingParams& Params) const override;

	virtual void UpdateEasing() override;

	MOVIESCENEANIMMIXER_API virtual UMovieSceneSection* AddNewAnimationOnRow(FFrameNumber KeyTime, UAnimSequenceBase* AnimSequence, int32 RowIndex) override;
	MOVIESCENEANIMMIXER_API virtual void AddSectionWithAutoRow(UMovieSceneSection* Section) override;
	MOVIESCENEANIMMIXER_API virtual int32 ComputeNextAvailableRowIndex() const override;

	// Invalidate the accumulated offset cache on the root motion section.
	// Call whenever sections change (add/remove/move) to mark the cache for rebuild.
	MOVIESCENEANIMMIXER_API void InvalidateAccumulatedOffsetCache();

	MOVIESCENEANIMMIXER_API bool HasDirtyAccumulatedOffsetCache() const;

	// Rebuild the accumulated offset cache on the root motion section if dirty.
	// Must be called from outside the ECS evaluation pipeline — the rebuild uses
	// isolated bake evaluations that re-enter the ECS.
	MOVIESCENEANIMMIXER_API void RebuildDirtyAccumulatedOffsetCache(
		class UMovieSceneEntitySystemLinker* Linker,
		UE::MovieScene::FInstanceHandle InstanceHandle);

#if WITH_EDITORONLY_DATA
	virtual FText GetTrackRowDisplayName(int32 RowIndex) const override;
	virtual FText GetDefaultDisplayName() const override;
	virtual bool CanRename() const override;

#endif

	MOVIESCENEANIMMIXER_API virtual UMovieSceneTrack* AddChildTrack(FGuid ObjectBinding, TSubclassOf<UMovieSceneTrack> TrackClass, int32 RowIndex) override;
	MOVIESCENEANIMMIXER_API int32 GetNextChildTrackRowIndex() const;
	MOVIESCENEANIMMIXER_API virtual void GetAllChildTracks(TArray<TObjectPtr<UMovieSceneTrack>>& OutChildTracks) const override;
	MOVIESCENEANIMMIXER_API void RemoveChildTrack(UMovieSceneTrack* Track);

	MOVIESCENEANIMMIXER_API UMovieSceneTrack* GetChildTrackAtRow(int32 RowIndex) const;

	MOVIESCENEANIMMIXER_API bool IsChildTrack(UMovieSceneTrack* Track) const;
	MOVIESCENEANIMMIXER_API int32 GetChildTrackRow(UMovieSceneTrack* Track) const;
	MOVIESCENEANIMMIXER_API void SetChildTrackRow(UMovieSceneTrack* Track, int32 NewRowIndex);

	// Layer Management
	// Note: Layer index is implicitly the row index. Sections use Section->GetRowIndex(),
	// Child tracks are referenced by individual layers, and cached in ChildTracks map. 
	// Layers array is kept sorted by row index.

	/** Create or get a layer at the specified row index */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* GetOrCreateLayer(int32 RowIndex);

	/** Get the layer at the specified row index (returns null if doesn't exist) */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* GetLayer(int32 RowIndex) const;

	/** Get all layers (sorted by row index) */
	MOVIESCENEANIMMIXER_API const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& GetLayers() const { return Layers; }

	/** Get the layer containing the given section (uses section's row index) */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* GetLayerForSection(UMovieSceneSection* Section) const;

	/** Get the layer containing the given child track */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* GetLayerForChildTrack(UMovieSceneTrack* Track) const;

	/** Insert a new empty layer at the specified index, shifting existing layers down */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* InsertLayer(int32 InsertIndex);

	// Transition Section Management

	/**
	 * Create a transition section between two overlapping sections on the same layer.
	 */
	template<typename TransitionClass>
	TransitionClass* CreateTransitionSection(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection);

	/**
	 * Create a transition section of the specified type between two overlapping sections (runtime class selection).
	 */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimTransitionSectionBase* CreateTransitionSectionOfType(
		UMovieSceneSection* FromSection,
		UMovieSceneSection* ToSection,
		TSubclassOf<UMovieSceneAnimTransitionSectionBase> TransitionClass);

	/** Find transition sections that reference the given section as either from or to */
	MOVIESCENEANIMMIXER_API TArray<UMovieSceneAnimTransitionSectionBase*> FindTransitionsForSection(UMovieSceneSection* Section) const;

	/** Find sections that overlap with the given section on the same layer (excluding transitions) */
	MOVIESCENEANIMMIXER_API TArray<UMovieSceneSection*> FindOverlappingSections(UMovieSceneSection* Section) const;

	/**
	 * Returns true if the given overlap range is large enough to warrant a transition.
	 * Sub-frame overlaps (from section endpoints that don't align to display frame
	 * boundaries) are rejected so that abutting sections don't produce zero-width transitions.
	 */
	MOVIESCENEANIMMIXER_API static bool IsOverlapLargeEnoughForTransition(const TRange<FFrameNumber>& Overlap, const UObject* ContextObject);

	/** Count how many non-transition sections overlap the given range on the specified row */
	MOVIESCENEANIMMIXER_API int32 CountSectionsOverlappingRange(const TRange<FFrameNumber>& Range, int32 RowIndex) const;

	/** Check if a transition already exists for the given from/to section pair */
	MOVIESCENEANIMMIXER_API UMovieSceneAnimTransitionSectionBase* FindTransitionForPair(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection) const;

	/** Auto-create transitions for any overlapping sections involving the given section */
	MOVIESCENEANIMMIXER_API void AutoCreateTransitionsForSection(UMovieSceneSection* Section);

	/**
	 * Validates and updates all transitions on the specified row.
	 * This removes invalid transitions and creates new ones for valid overlaps.
	 * Call this after any operation that might affect transition validity on a row
	 * (section added, moved, removed, or row changed).
	 */
	MOVIESCENEANIMMIXER_API bool UpdateTransitionsForRow(int32 RowIndex);

	/**
	 * Validates and updates all transitions on the specified rows.
	 * Convenience method that calls UpdateTransitionsForRow for each row.
	 */
	MOVIESCENEANIMMIXER_API bool UpdateTransitionsForRows(const TSet<int32>& RowIndices);

public:
	UPROPERTY(EditAnywhere, Category = "Mixed Animation")
	TInstancedStruct<FMovieSceneMixedAnimationTarget> MixedAnimationTarget;

private:
#if WITH_EDITOR
	/** Handle when a transition section is moved - adjusts from/to sections and constrains row changes */
	EMovieSceneSectionMovedResult HandleTransitionSectionMoved(UMovieSceneAnimTransitionSectionBase& TransitionSection);
#endif
	// Row assignment for child tracks
	UPROPERTY()
	TMap<TObjectPtr<UMovieSceneTrack>, int32> ChildTracks;

	// Layers organizing sections and child tracks by row index
	// Array is kept sorted by row index for efficient lookup
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>> Layers;
};

template<typename TransitionClass>
TransitionClass* UMovieSceneAnimationMixerTrack::CreateTransitionSection(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection)
{
	static_assert(TIsDerivedFrom<TransitionClass, UMovieSceneAnimTransitionSectionBase>::Value,
		"TransitionClass must derive from UMovieSceneAnimTransitionSectionBase");

	if (!FromSection || !ToSection)
	{
		return nullptr;
	}

	// Sections must be on the same row (layer)
	if (FromSection->GetRowIndex() != ToSection->GetRowIndex())
	{
		return nullptr;
	}

	// Compute the overlap range
	TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(
		FromSection->GetRange(),
		ToSection->GetRange()
	);

	if (Overlap.IsEmpty())
	{
		return nullptr;
	}

	Modify();

	// Create the transition section
	TransitionClass* TransitionSection = NewObject<TransitionClass>(this, TransitionClass::StaticClass(), NAME_None, RF_Transactional);
	TransitionSection->SetRange(Overlap);
	TransitionSection->SetRowIndex(FromSection->GetRowIndex());
	TransitionSection->FromSection = FromSection;
	TransitionSection->ToSection = ToSection;

	// Ensure transition sections always draw on top of regular sections
	TransitionSection->SetOverlapPriority(INT32_MAX);

	// Initialize any default curves now that the range is set
	TransitionSection->InitializeDefaultCurve();

	// Add via FSectionParameter to ensure proper lifecycle hooks are called
	UMovieSceneTrack::AddSection(FSectionParameter{*TransitionSection});

	return TransitionSection;
}
