// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneTrack.h"
#include "Decorations/IMovieSceneTrackDecoration.h"
#include "EntitySystem/IMovieSceneEntityDecorator.h"
#include "MovieSceneAnimationTrackDecoration.generated.h"


class UMovieSceneAnimationMixerTrack;
/**
 * Track decoration that adds animation mixer components to child track entities.
 * Priority is determined by the track's layer assignment within the mixer track.
 */
UCLASS(MinimalAPI)
class UMovieSceneAnimationTrackDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneTrackDecoration
	, public IMovieSceneEntityDecorator
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TObjectPtr<UMovieSceneAnimationMixerTrack> MixerTrack;

	// IMovieSceneEntityDecorator interface
	virtual void ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
};
