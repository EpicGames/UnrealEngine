// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "Misc/Optional.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneBoneMatchData.h"
#include "MovieSceneBoneMatchChannel.generated.h"

// Channel that stores FMovieSceneBoneMatchData as discrete (non-interpolated) keys.
// Typically holds 0 or 1 keys: one bone match per section.
USTRUCT()
struct FMovieSceneBoneMatchChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FMovieSceneBoneMatchData CurveValueType;

	TMovieSceneChannelData<FMovieSceneBoneMatchData> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneBoneMatchData>(&Times, &Values, this, &KeyHandles);
	}

	TMovieSceneChannelData<const FMovieSceneBoneMatchData> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneBoneMatchData>(&Times, &Values);
	}

	// FMovieSceneChannel interface
	MOVIESCENEANIMMIXER_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENEANIMMIXER_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENEANIMMIXER_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENEANIMMIXER_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENEANIMMIXER_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENEANIMMIXER_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENEANIMMIXER_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENEANIMMIXER_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENEANIMMIXER_API virtual int32 GetNumKeys() const override;
	MOVIESCENEANIMMIXER_API virtual void Reset() override;
	MOVIESCENEANIMMIXER_API void NotifyReferenceChanged(FFrameNumber NewResolvedTime);
	MOVIESCENEANIMMIXER_API void ApplyOwningSectionRange(TOptional<FFrameNumber> NewStart, TOptional<FFrameNumber> NewEnd);
	MOVIESCENEANIMMIXER_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENEANIMMIXER_API virtual void ClearDefault() override;
	MOVIESCENEANIMMIXER_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENEANIMMIXER_API virtual int32 GetIndex(FKeyHandle Handle) override;

private:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneBoneMatchData> Values;

	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneBoneMatchChannel> : TMovieSceneChannelTraitsBase<FMovieSceneBoneMatchChannel>
{
	enum { SupportsDefaults = false };
};
