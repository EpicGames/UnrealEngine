// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Transform.h"
#include "Misc/FrameNumber.h"
#include "UObject/WeakObjectPtr.h"
#include "MovieSceneBoneMatchData.generated.h"

class UMovieSceneSection;

UENUM()
enum class EBoneMatchTimeMode : uint8
{
	AtCurrentTime,
	AtStartOfSelectedSection,
	AtEndOfSelectedSection,
	AtStartOfReferenceSection,
	AtEndOfReferenceSection,
	InBetween
};

USTRUCT()
struct FMovieSceneBoneMatchData
{
	GENERATED_BODY()

	// Computed match transform offset
	UPROPERTY(VisibleAnywhere, Category="Bone Matching")
	FTransform MatchTransform = FTransform::Identity;

	// Bone to match
	UPROPERTY(EditAnywhere, Category="Bone Matching")
	FName BoneName;

	// Reference section for determining match time
	UPROPERTY()
	TWeakObjectPtr<UMovieSceneSection> ReferenceSection;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	EBoneMatchTimeMode MatchTimeMode = EBoneMatchTimeMode::AtCurrentTime;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchLocationX = true;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchLocationY = true;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchLocationZ = false;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchRotationX = false;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchRotationY = false;

	UPROPERTY(EditAnywhere, Category="Bone Matching")
	bool bMatchRotationZ = true;

	UPROPERTY()
	bool bIsValid = false;

	// Set when section moves after a valid match
	UPROPERTY()
	bool bIsDirty = false;

	// Resolved match time (frame number) filled in by ComputeBoneMatch.
	// Used by the editor to place the key on the channel.
	UPROPERTY()
	FFrameNumber MatchTime;

	// Compares user-specified settings and validity state. Computed results
	// (MatchTransform, MatchTime) and transient state (bIsDirty) are excluded
	// since they change on every re-evaluation.
	friend bool operator==(const FMovieSceneBoneMatchData& A, const FMovieSceneBoneMatchData& B)
	{
		return A.BoneName == B.BoneName
			&& A.ReferenceSection == B.ReferenceSection
			&& A.MatchTimeMode == B.MatchTimeMode
			&& A.bIsValid == B.bIsValid
			&& A.bMatchLocationX == B.bMatchLocationX
			&& A.bMatchLocationY == B.bMatchLocationY
			&& A.bMatchLocationZ == B.bMatchLocationZ
			&& A.bMatchRotationX == B.bMatchRotationX
			&& A.bMatchRotationY == B.bMatchRotationY
			&& A.bMatchRotationZ == B.bMatchRotationZ;
	}

	friend bool operator!=(const FMovieSceneBoneMatchData& A, const FMovieSceneBoneMatchData& B)
	{
		return !(A == B);
	}
};
