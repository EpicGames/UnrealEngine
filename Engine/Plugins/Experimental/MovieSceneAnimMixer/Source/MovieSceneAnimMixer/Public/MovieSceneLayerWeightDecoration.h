// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSignedObject.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"

#include "MovieSceneLayerWeightDecoration.generated.h"

class UMovieSceneAnimationMixerLayer;

// Decoration that adds a keyframeable weight channel to a mixer layer.
// The weight (default 1.0) multiplies with the section weight before
// inter-layer blending, scaling the layer's contribution to the final pose.
UCLASS(MinimalAPI, DisplayName="Layer Weight")
class UMovieSceneLayerWeightDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneSectionProviderDecoration
	, public IMovieSceneChannelDecoration
{
public:
	GENERATED_BODY()

	UMovieSceneLayerWeightDecoration(const FObjectInitializer& ObjInit);

	// IMovieSceneSectionProviderDecoration
	virtual TArrayView<TObjectPtr<UMovieSceneSection>> GetSections() override { return WeightSections; }
	virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENEANIMMIXER_API virtual void AddSection(UMovieSceneSection* InSection) override;
	MOVIESCENEANIMMIXER_API virtual void RemoveSection(UMovieSceneSection& SectionToRemove) override;
	MOVIESCENEANIMMIXER_API virtual bool SupportsMultipleSections() override { return false; }
	MOVIESCENEANIMMIXER_API virtual TSubclassOf<UMovieSceneSection> GetHostedSectionClass() const override;

	// IMovieSceneChannelDecoration
	virtual EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData) override;

	const FMovieSceneFloatChannel& GetWeightChannel() const { return WeightChannel; }

	UPROPERTY()
	FMovieSceneFloatChannel WeightChannel;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> WeightSections;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};
