// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSignedObject.h"
#include "Decorations/IMovieSceneSectionDecoration.h"
#include "Decorations/IMovieSceneTrackDecoration.h"
#include "EntitySystem/IMovieSceneEntityDecorator.h"
#include "MovieSceneMirroringDecoration.generated.h"

class UMirrorDataTable;

/**
 * Decoration that applies mirroring to animation poses produced by sections or tracks in the Anim Mixer.
 *
 */
UCLASS(MinimalAPI, collapseCategories, meta=(DisplayName="Mirroring"))
class UMovieSceneMirroringDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneSectionDecoration
	, public IMovieSceneTrackDecoration
	, public IMovieSceneEntityDecorator
{
	GENERATED_BODY()

public:

	/** The mirror data table used to mirror the animation pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mirroring")
	TObjectPtr<UMirrorDataTable> MirrorDataTable;

	// IMovieSceneEntityDecorator interface
	virtual void ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
};
