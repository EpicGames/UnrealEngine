// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Channels/MovieSceneByteChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneRootMotionSection.generated.h"

struct FMovieSceneSequenceTransform;

UENUM()
enum class EMovieSceneRootMotionDestination : uint8
{
	/** Throw away any transform on the root bone */
	Discard,
	/** Leave the root bone with whatever transform it ended up with after evaluation */
	RootBone,
	/** Copy the root bone's transform onto the owning Component, and reset the root transform */
	Component,
	/** Copy the root bone's transform onto the owning Actor, and reset the root transform */
	Actor,
	/** Leave the root motion transform on an attribute for external systems to pick up */
	Attribute,
};

/** Behavior during gaps where no root motion sections are active */
UENUM()
enum class EMovieSceneRootMotionGapBehavior : uint8
{
	/** Reset to origin during gaps - current default behavior */
	ResetToOrigin,
	/** Persist the transform from the previous section that had KeepState enabled */
	PersistPreviousTransform
};

/** Cached accumulated offset from sections with KeepState enabled.
 *  For non-AccumulatedOffset sections, AccumulatedOffset holds world-space
 *  and AnimSpaceAccumulatedOffset holds animation-space. For AccumulatedOffset
 *  sections, both fields hold animation-space values because the world
 *  conversion is applied at runtime by the conversion task. */
USTRUCT()
struct FRootMotionAccumulatedOffset
{
	GENERATED_BODY()

	/** Time at which this accumulated offset takes effect */
	UPROPERTY()
	FFrameTime StartFrame;

	/** The accumulated transform offset at this point (world space) */
	UPROPERTY()
	FTransform AccumulatedOffset = FTransform::Identity;

	/** The accumulated transform offset in animation space (pre-world conversion) */
	UPROPERTY()
	FTransform AnimSpaceAccumulatedOffset = FTransform::Identity;

	/** Weight of this offset for blending */
	UPROPERTY()
	float Weight = 1.0f;

	/** The section that generated this cache entry */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> OwnerSection = nullptr;
};

/**
 * Section that controls where root motion is applied for animation mixer tracks.
 * Created internally by UMovieSceneRootMotionTargetDecoration - not added directly.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Root Motion Target", Hidden))
class UMovieSceneRootMotionSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
	GENERATED_BODY()

public:

	UMovieSceneRootMotionSection(const FObjectInitializer& Init);

	virtual int32 GetRowSortOrder() const;

	virtual FColor GetMixerItemTint() const override
	{
		return MixerTintOverride;
	}

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	/** Get the accumulated offset at a given time (world space) */
	MOVIESCENEANIMMIXER_API FTransform GetAccumulatedOffsetAt(FFrameTime Time) const;

	/** Section-filtered accumulated offset lookup. bAllowCrossSection lets the
	 *  query see entries owned by other sections that fall before this section's
	 *  start, for AccumulatedOffset-style chaining. */
	MOVIESCENEANIMMIXER_API FTransform GetAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const;

	/** Get the accumulated offset at a given time in animation space (pre-world conversion) */
	MOVIESCENEANIMMIXER_API FTransform GetAnimSpaceAccumulatedOffsetAt(FFrameTime Time) const;

	/** Anim-space variant of GetAccumulatedOffsetForSection. */
	MOVIESCENEANIMMIXER_API FTransform GetAnimSpaceAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const;

	/** Existence check using the same filter rules as GetAccumulatedOffsetForSection. */
	MOVIESCENEANIMMIXER_API bool HasApplicableAccumulatedEntry(FFrameTime Time, UMovieSceneSection* Section, bool bAllowCrossSection) const;

private:
	FTransform FindAccumulatedOffsetForSection(FFrameTime Time, UMovieSceneSection* Section, bool bAnimSpace, bool bAllowCrossSection) const;

public:
	/** Rebuild the accumulated offset cache using isolated bake evaluations.*/
	MOVIESCENEANIMMIXER_API void RebuildAccumulatedOffsetCache(
		class UMovieSceneEntitySystemLinker* Linker,
		UE::MovieScene::FInstanceHandle InstanceHandle,
		class UMovieSceneAnimationMixerTrack* MixerTrack);

	void InvalidateAccumulatedOffsetCache() { bAccumulatedOffsetCacheDirty = true; }
	bool IsAccumulatedOffsetCacheDirty() const { return bAccumulatedOffsetCacheDirty; }

	/** Resolve a loop boundary returned by ExtractBoundariesWithinRange to the
	 *  integer outer tick at which the runtime's loop index actually transitions.
	 *  Used so cache entries land on exactly the same tick as runtime wraps. */
	static FFrameNumber ResolveLoopTransitionTick(
		const FMovieSceneSequenceTransform& Transform,
		FFrameTime ApproximateBoundary,
		float Duration);

public:

	/** How to handle gaps where no sections with root motion are active */
	UPROPERTY(EditAnywhere, Category="Root Motion")
	EMovieSceneRootMotionGapBehavior GapBehavior = EMovieSceneRootMotionGapBehavior::PersistPreviousTransform;

	/** Cached accumulated offsets from sections with KeepState enabled. Serialized so
	 *  the correct values are available at runtime without needing to rebuild. */
	UPROPERTY()
	TArray<FRootMotionAccumulatedOffset> AccumulatedOffsetCache;

	bool bAccumulatedOffsetCacheDirty = false;

private:

	UPROPERTY()
	FMovieSceneByteChannel RootDestinationChannel;

	FColor MixerTintOverride = FColor(20, 70, 70, 200);
};
