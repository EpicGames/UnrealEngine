// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/IMovieSceneTrackDecoration.h"
#include "EntitySystem/IMovieSceneEntityDecorator.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneRootMotionSection.h"
#include "MovieSceneRootMotionTargetDecoration.generated.h"

class UMovieSceneTrack;

/**
 * Track-level decoration that controls where root motion is applied.
 * Wraps a UMovieSceneRootMotionSection
 */
UCLASS(MinimalAPI, collapseCategories, meta=(DisplayName="Root Motion Target"))
class UMovieSceneRootMotionTargetDecoration
	: public UMovieSceneSignedObject
	, public IMovieSceneTrackDecoration
	, public IMovieSceneSectionProviderDecoration
{
	GENERATED_BODY()

public:

	UMovieSceneRootMotionTargetDecoration(const FObjectInitializer& Init);

	// IMovieSceneTrackDecoration interface
	virtual void OnDecorationAdded(UMovieSceneTrack* Track) override;
	virtual void OnDecorationRemoved() override;

	// IMovieSceneSectionProviderDecoration interface
	virtual TArrayView<TObjectPtr<UMovieSceneSection>> GetSections() override;
	MOVIESCENEANIMMIXER_API virtual TSubclassOf<UMovieSceneSection> GetHostedSectionClass() const override;

	/** Get the internal root motion section */
	UMovieSceneRootMotionSection* GetRootMotionSection() const { return Cast<UMovieSceneRootMotionSection>(RootMotionSection); }

protected:

	/** Internal root motion section that provides the destination channel and entity logic */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> RootMotionSection;
};
