// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AnimNode_SequencerMixerTarget.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Animation/AnimInstance.h"

#include "MovieSceneAnimBlueprintTargetSystem.generated.h"

/**
 * Declaring a unique target for the "Sequencer Mixer Target" node in an Anim BP. Will match the node's name with the name specified here.
 */
USTRUCT(meta=(DisplayName="Anim Blueprint Target"))
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimBlueprintTarget : public FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY() 

	// Node name to use for injection.
	UPROPERTY(EditAnywhere, Category="Animation")
	FName BlueprintNodeName;

	inline friend uint32 GetTypeHash(const FMovieSceneAnimBlueprintTarget& Target)
	{
		return HashCombine(GetTypeHash(FMovieSceneAnimBlueprintTarget::StaticStruct()), GetTypeHash(Target.BlueprintNodeName));
	}

	FMovieSceneAnimBlueprintTarget(FName InBlueprintNodeName)
		: BlueprintNodeName(InBlueprintNodeName)
	{
		
	}

	FMovieSceneAnimBlueprintTarget()
		: BlueprintNodeName(FAnimNode_SequencerMixerTarget::DefaultTargetName)
	{

	}

#if WITH_EDITOR
	virtual FText GetShortDisplayName() const override
	{
		return FText::FromName(BlueprintNodeName);
	}
#endif
};

UCLASS(MinimalAPI)
class UMovieSceneAnimBlueprintTargetSystem : public UMovieSceneEntitySystem
{
public:
	
	GENERATED_BODY()

	UMovieSceneAnimBlueprintTargetSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	
};
