// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectKey.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EvaluationVM/EvaluationTask.h"
#include "MovieSceneAnimBusSystem.generated.h"

class UMovieSceneAnimMixerSystem;

// Evaluation task that reads a stored pose from a named bus and pushes it
// onto the VM's keyframe stack. Used by UMovieSceneAnimBusSection to inject
// poses written by bus-target mixers into a consuming mixer's program.
USTRUCT()
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimBusReadTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FMovieSceneAnimBusReadTask)

	static FMovieSceneAnimBusReadTask Make(FName InBusName, FObjectKey InBoundObjectKey, UMovieSceneAnimMixerSystem* InMixerSystem);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UPROPERTY()
	FName BusName;

	FObjectKey BoundObjectKey;

	TWeakObjectPtr<UMovieSceneAnimMixerSystem> MixerSystem;
};

// System that creates/updates FMovieSceneAnimBusReadTask instances on entities
// with BusName components. Runs before UMovieSceneAnimMixerSystem so bus read
// tasks are in place before the mixer builds its evaluation program.
UCLASS(MinimalAPI)
class UMovieSceneAnimBusSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneAnimBusSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
};
