// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "MovieSceneAnimMixerNotifyTypes.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "SequencerAnimationSupport.h"
#include "Animation/AnimNodeBase.h"
#include "MovieSceneAnimInstanceTargetSystem.generated.h"


struct FAnimNextEvaluationTask;

/**
 * Declaring a unique target for targeting a custom anim instance on the skeletal mesh component. Doesn't need additional metadata.
 */
USTRUCT(meta=(DisplayName="Custom Anim Instance"))
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimInstanceTarget : public FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY() 

	inline friend uint32 GetTypeHash(const FMovieSceneAnimInstanceTarget& Target)
	{
		return GetTypeHash(FMovieSceneAnimInstanceTarget::StaticStruct());
	}

#if WITH_EDITOR
	virtual FText GetShortDisplayName() const override
	{
		return NSLOCTEXT("MovieSceneAnimInstanceTarget", "CustomAnimInstance", "Custom Anim Instance");
	}
#endif
};


// Custom anim instance and proxy that take an anim evaluation task, evaluate it, and push the resulting pose to the skeleton.
UCLASS(transient, NotBlueprintable, MinimalAPI)
class USequencerMixedAnimInstance : public UAnimInstance, public ISequencerAnimationSupport
{
	GENERATED_BODY()

	USequencerMixedAnimInstance(const FObjectInitializer& ObjectInitializer);

public:
	MOVIESCENEANIMMIXER_API void SetMixerTask(TSharedPtr<FAnimNextEvaluationTask> InEvalTask);

	MOVIESCENEANIMMIXER_API void StagePendingNotifyBatches(TArray<FSequencerMixerPendingNotifyBatch>&& InBatches);

protected:
	// UAnimInstance interface
	MOVIESCENEANIMMIXER_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;

	// ISequencerAnimationSupport

	// Empty ISequencerAnimationSupport things we don't use- would be better to have 2 interfaces, but deprecation would be a pain.
	MOVIESCENEANIMMIXER_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies) override {}
	MOVIESCENEANIMMIXER_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies) override {}
	MOVIESCENEANIMMIXER_API virtual void ConstructNodes() override {}
	MOVIESCENEANIMMIXER_API virtual void ResetNodes() override {}
	MOVIESCENEANIMMIXER_API virtual void ResetPose() override {}
	MOVIESCENEANIMMIXER_API virtual void SavePose() override {}

	virtual UAnimInstance* GetSourceAnimInstance() override;
	virtual void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance) override;
	virtual bool DoesSupportDifferentSourceAnimInstance() const override { return true; }
};


/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FSequencerMixedAnimInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

public:
	FSequencerMixedAnimInstanceProxy()
	{
	}

	FSequencerMixedAnimInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	MOVIESCENEANIMMIXER_API virtual ~FSequencerMixedAnimInstanceProxy() {}

	// FAnimInstanceProxy interface
	MOVIESCENEANIMMIXER_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	MOVIESCENEANIMMIXER_API virtual bool Evaluate(FPoseContext& Output) override;
	MOVIESCENEANIMMIXER_API virtual void CacheBones() override;
	MOVIESCENEANIMMIXER_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	MOVIESCENEANIMMIXER_API virtual void PreEvaluateAnimation(UAnimInstance* InAnimInstance) override;

	/** Anim Instance Source info - created externally and used here */
	void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance, FAnimInstanceProxy* SourceAnimInputProxy);
	UAnimInstance* GetSourceAnimInstance() const { return CurrentSourceAnimInstance; }

	void SetMixerTask(TSharedPtr<FAnimNextEvaluationTask> InEvalTask);
	void LinkSourcePose(UAnimInstance* InInputInstance, FAnimInstanceProxy* InInputProxy);
	void UnlinkSourcePose();

	void StagePendingNotifyBatches(TArray<FSequencerMixerPendingNotifyBatch>&& InBatches);

protected:

	/** Source Anim Instance */
	TObjectPtr<UAnimInstance> CurrentSourceAnimInstance;
	FAnimInstanceProxy* CurrentSourceProxy = nullptr;

	/* Optional link to root node if the source anim instance exists and has one */
	FPoseLink SourcePose;

	// Pointer to the task passed from the mixer.
	TSharedPtr<FAnimNextEvaluationTask> MixerTask;

	// One batch per active section's PendingNotifies, with that section's effective blend weight.
	TArray<FSequencerMixerPendingNotifyBatch> PendingNotifyBatches;
};




// System that handles applying animation mixer evaluation tasks to a custom anim instance on a skeletal mesh component.
UCLASS(MinimalAPI)
class UMovieSceneAnimInstanceTargetSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneAnimInstanceTargetSystem(const FObjectInitializer& ObjInit);

#if WITH_EDITOR
	virtual ~UMovieSceneAnimInstanceTargetSystem() override;

	TArray<FDelegateHandle> PreCompileHandles;
#endif
	
private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;

};