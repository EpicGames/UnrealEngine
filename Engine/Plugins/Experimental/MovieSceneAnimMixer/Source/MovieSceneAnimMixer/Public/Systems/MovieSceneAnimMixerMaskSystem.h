// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneAnimMixerBlendProvider.h"
#include "HierarchyTableBlendProfile.h"
#include "Animation/AnimCurveTypes.h"
#include "MovieSceneAnimMixerMaskSystem.generated.h"

class UMovieSceneAnimationMixerLayer;
class UUAFBlendMask;
class USkeleton;
struct FAnimNextEvaluationTask;

// Cached data from a single mask asset. Invalidated when the asset changes.
struct FCachedMaskAssetData
{
	TWeakObjectPtr<UUAFBlendMask> BlendMask;
	TArray<float> DampingValues;
	TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight> AttributeDampingValues;
	// Curve damping cached as name→damping pairs
	TMap<FName, float> CurveDampingValues;

	void CacheFromAsset(UUAFBlendMask* InBlendMask);
	bool IsStale(const UUAFBlendMask* InBlendMask) const;
};

// Per-{BoundObject, MixerLayer} accumulation state for mask blending.
struct FMaskLayerAccumulation
{
	UE::MovieScene::FMovieSceneEntityID OutputEntityID;
	TWeakObjectPtr<USkeleton> MaskSkeleton;

	// Damping values accumulated from all contributing mask sections, weighted by easing
	TArray<float> AccumulationBoneWeights;
	TMap<FName, float> CurveWeightsMap;
	TMap<UE::Anim::FAttributeId, float> AttributeWeightsMap;

	// Final task data arrays (reused across frames to avoid allocation).
	// These must persist past BuildBlendTask since the task holds pointers to them.
	TArray<float> TaskBoneWeights;
	TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight> TaskAttributeWeights;
	UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement> TaskCurveWeights;

	// Cached per-asset data for contributing mask sections
	TMap<TObjectKey<UUAFBlendMask>, FCachedMaskAssetData> CachedAssets;

	// Number of contributing mask sections this frame
	int32 NumContributingSections = 0;
	// For the single-section fast path: the mask that contributed this frame
	TWeakObjectPtr<UUAFBlendMask> SingleContributingMask;
	// True if the single section fast path can be used
	bool bSingleSectionFullWeight = false;

	void ResetAccumulation();
	void AccumulateMask(UUAFBlendMask* BlendMask, double EasingWeight);
	TSharedPtr<FAnimNextEvaluationTask> BuildBlendTask(float LayerWeight);
};

// Key for the accumulation map
struct FMaskLayerKey
{
	FObjectKey BoundObjectKey;
	TWeakObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer;

	friend bool operator==(const FMaskLayerKey& A, const FMaskLayerKey& B)
	{
		return A.BoundObjectKey == B.BoundObjectKey && A.MixerLayer == B.MixerLayer;
	}

	friend uint32 GetTypeHash(const FMaskLayerKey& Key)
	{
		return HashCombineFast(GetTypeHash(Key.BoundObjectKey), GetTypeHash(Key.MixerLayer));
	}
};

// Gathers mask section entities and produces layer blend task entities for the mixer.
// Runs before the mixer system. Creates one output entity per {BoundObject, MixerLayer}
// that has active mask sections, with a LayerBlendTask component.
UCLASS(MinimalAPI)
class UMovieSceneAnimMixerMaskSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	UMovieSceneAnimMixerMaskSystem(const FObjectInitializer& ObjInit);

private:
	TMap<FMaskLayerKey, FMaskLayerAccumulation> Accumulations;

	void UpdateAccumulationLifetimes();

	virtual void OnLink() override;
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnCleanTaggedGarbage() override;
};
