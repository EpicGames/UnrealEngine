// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "StructUtils/InstancedStruct.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "AnimSequencerInstanceProxy.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "EvaluationVM/Tasks/BlendKeyframes.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "Animation/AnimNotifyQueue.h"
#include "MovieSceneAnimMixerNotifyTypes.h"
#include "MovieSceneAnimMixerSystem.generated.h"

class UMovieSceneAnimationMixerTrack;
class UMovieSceneAnimationMixerLayer;
struct FMovieSceneSkeletalAnimationComponentData;
struct FAnimNextEvaluationTask;

namespace UE::UAF
{
	struct FEvaluationProgram;
}

// Target type that writes the mixer's evaluated pose to a named bus.
// Other mixers can then read from the bus via UMovieSceneAnimBusSection.
USTRUCT(meta=(DisplayName="Bus Target"))
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimBusTarget : public FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Animation")
	FName BusName;

	inline friend uint32 GetTypeHash(const FMovieSceneAnimBusTarget& Target)
	{
		return HashCombine(GetTypeHash(FMovieSceneAnimBusTarget::StaticStruct()), GetTypeHash(Target.BusName));
	}

	FMovieSceneAnimBusTarget(FName InBusName)
		: BusName(InBusName)
	{
	}

	FMovieSceneAnimBusTarget()
		: BusName(NAME_None)
	{
	}

#if WITH_EDITOR
	virtual FText GetShortDisplayName() const override
	{
		if (!BusName.IsNone())
		{
			return FText::Format(NSLOCTEXT("MovieSceneAnimBusTarget", "BusDisplayFmt", "Bus: {0}"), FText::FromName(BusName));
		}
		return NSLOCTEXT("MovieSceneAnimBusTarget", "BusTarget", "Bus Target");
	}
#endif
};

struct FMovieSceneAnimMixer;

struct FMovieSceneAnimMixerEntry
{

	TWeakPtr<FMovieSceneAnimMixer> WeakMixer;
	TSharedPtr<FAnimNextEvaluationTask> EvalTask;
	/** Shared pointer to the root motion for this entry if it came from a FMovieSceneRootMotionSettings on a mixer
	 * @note: Only to be used for lifetime management to keep FMovieSceneAnimMixer::WeakRootMotion alive!
	 *        This may be null even if the result of the mix still has root motion */
	TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionLifetimeReference;
	TOptional<FMovieSceneRootMotionSettings> RootMotionSettings;
	/** Combined weight and easing (WeightResult * EasingResult). Used for backwards-compat weighted average. */
	double CombinedWeight = 1.0;
	/** Easing-only weight from section overlap. Controls intra-layer blending when using mixer track. */
	double EasingWeight = 1.0;
	/** Section weight (from weight channel). Contributes to inter-layer blend weight. */
	double SectionWeight = 1.0;
	int32 Priority = 0;
	bool bAdditive = false;
	bool bSkipPoseWeight = false;
	// True when this entry's eval task reads bone[0] from the previous pose on the stack
	// (layered Control Rigs). The mixer brackets such tasks with a bake-into-bone[0] before
	// and an extract or reset after to maintain the bone[0]=identity / attribute-populated
	// invariant downstream layers rely on.
	bool bPreBakeRootMotion = false;
	TObjectPtr<UMirrorDataTable> MirrorTable;
	UE::MovieScene::FMovieSceneEntityID EntityID;
	UE::MovieScene::FInstanceHandle InstanceHandle;
	/** The section or track that owns this entity - used for transition section lookups */
	FObjectKey EntityOwner;
	/** The mixer layer this entry belongs to (null for backwards-compat standalone tracks) */
	TWeakObjectPtr<UMovieSceneAnimationMixerLayer> MixerLayer;
	/** True if this entry is from a transition section */
	bool bIsTransition = false;

	/** Bone match transform for this entry (set only when a valid match exists) */
	TOptional<FTransform> BoneMatchTransform;

	/** Per-section anim-notify references for this frame's evaluation window. Filled by
	 *  FUpdateTaskEntities; moved out by BuildProgramFromEntries onto FMovieSceneAnimMixer::PendingNotifyBatches
	 *  with a resolved blend weight. Not read directly by target consumer systems. */
	TArray<FAnimNotifyEventReference> PendingNotifies;

	inline bool operator<(const FMovieSceneAnimMixerEntry& RHS) const
	{
		if (Priority == RHS.Priority)
		{
			// Sort additives after absolutes so they are applied on top
			if (bAdditive != RHS.bAdditive)
			{
				return RHS.bAdditive;
			}
			return false;
		}
		return Priority < RHS.Priority;
	}
};

// Key into the mixer map- one mixer per bound object per animation target

struct FMovieSceneAnimMixerKey
{
	FObjectKey BoundObjectKey;
	TInstancedStruct<FMovieSceneMixedAnimationTarget> Target;

	/** Equality operator */
	friend bool operator==(const FMovieSceneAnimMixerKey& A, const FMovieSceneAnimMixerKey& B)
	{
		return A.BoundObjectKey == B.BoundObjectKey && A.Target == B.Target;
	}

	/** Generate a type hash from this component */
	friend uint32 GetTypeHash(const FMovieSceneAnimMixerKey& AnimMixerKey)
	{
		return HashCombineFast(GetTypeHash(AnimMixerKey.BoundObjectKey), AnimMixerKey.Target.IsValid() ? AnimMixerKey.Target.GetScriptStruct()->GetStructTypeHash(&AnimMixerKey.Target.Get()) : INDEX_NONE);
	}
};

// A single anim mixer, containing an array of mixer entries sorted by priority.

struct FMovieSceneAnimMixer
{
	UE::MovieScene::FMovieSceneEntityID MixerEntityID;
	TArray<TWeakPtr<FMovieSceneAnimMixerEntry>> WeakEntries;
	TSharedPtr<UE::UAF::FEvaluationProgram> EvaluationProgram;
	TWeakPtr<FMovieSceneMixerRootMotionComponentData> WeakRootMotion;

	// Per-section anim-notify dispatch list, refreshed each frame alongside EvaluationProgram.
	// Each batch carries one section's notifies with its final blend weight (intra-layer
	// normalize * layer weight * exposure factor). Consumers move-and-clear this list.
	TArray<FSequencerMixerPendingNotifyBatch> PendingNotifyBatches;

	TWeakObjectPtr<UMovieSceneAnimationMixerTrack> OwnerTrack;
	bool bNeedsResort = false;

	/**
	 * Find a mixer entry by its EntityOwner (the section or track that created it).
	 * Used by transition sections to find from/to entries for building transition programs.
	 */
	TSharedPtr<FMovieSceneAnimMixerEntry> FindEntryByOwner(const FObjectKey& InOwner) const
	{
		for (const TWeakPtr<FMovieSceneAnimMixerEntry>& WeakEntry : WeakEntries)
		{
			if (TSharedPtr<FMovieSceneAnimMixerEntry> Entry = WeakEntry.Pin())
			{
				if (Entry->EntityOwner == InOwner)
				{
					return Entry;
				}
			}
		}
		return nullptr;
	}
};


/*
 * Sequencer weighted average addition blend task
 *
 * This happens to be identical to FAnimNextBlendAddKeyframeWithScaleTask except the operatnds are reversed,
 *     so the resulting stack state is:

 * Top = (Top-1) + (Top * ScaleFactor)
 * 
 * Note that rotations will not be normalized after this task.
 */
USTRUCT()
struct FMovieSceneAccumulateAbsoluteBlendTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FMovieSceneAccumulateAbsoluteBlendTask)

	static FMovieSceneAccumulateAbsoluteBlendTask Make(float ScaleFactor);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	// The scale factor to apply to the keyframe
	UPROPERTY()
	float ScaleFactor = 0.0f;
};

USTRUCT()
struct FAnimNextBlendTwoKeyframesPreserveRootMotionTask : public FAnimNextBlendTwoKeyframesTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBlendTwoKeyframesPreserveRootMotionTask)

	static FAnimNextBlendTwoKeyframesPreserveRootMotionTask Make(float InterpolationAlpha);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};



// Key for per-object, per-bus-name storage.
struct FMovieSceneAnimBusStorageKey
{
	FObjectKey BoundObjectKey;
	FName BusName;

	friend bool operator==(const FMovieSceneAnimBusStorageKey& A, const FMovieSceneAnimBusStorageKey& B)
	{
		return A.BoundObjectKey == B.BoundObjectKey && A.BusName == B.BusName;
	}

	friend uint32 GetTypeHash(const FMovieSceneAnimBusStorageKey& Key)
	{
		return HashCombineFast(GetTypeHash(Key.BoundObjectKey), GetTypeHash(Key.BusName));
	}
};

// Stored evaluation program for a bus-target mixer. Executed inline in the
// consuming mixer's VM so the pose is evaluated in the correct context.
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimBusData
{
	TSharedPtr<UE::UAF::FEvaluationProgram> Program;

	// True if the bus-target mixer has entries with root motion settings.
	bool bHasRootMotion = false;

	bool IsValid() const { return Program.IsValid(); }
};

// Takes in evaluation tasks with optional pose weight, masks, priority and a given animation target.
// Constructs a hierarchical 'mixer' per bound object per target. 
// Similar to blender systems, in a 'many to one' operation, each mixer will create an entity with a single evaluation task
// wrapping the full blend operation, with the target component. 
// This entity is then consumed by the appropriate target animation system in order to produce the result on the mesh.
UCLASS(MinimalAPI)
class UMovieSceneAnimMixerSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneAnimMixerSystem(const FObjectInitializer& ObjInit);

	static TInstancedStruct<FMovieSceneMixedAnimationTarget> ResolveAnimationTarget(FObjectKey ObjectKey, const TInstancedStruct<FMovieSceneMixedAnimationTarget>& InTarget, const FMovieSceneSkeletalAnimationComponentData* SkelAnimData = nullptr);

	MOVIESCENEANIMMIXER_API TSharedPtr<FMovieSceneMixerRootMotionComponentData> FindRootMotion(FObjectKey InObject) const;
	void AssignRootMotion(FObjectKey InObjectKey, TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotion);

	// Look up a mixer by (bound object, target).
	TSharedPtr<FMovieSceneAnimMixer> FindMixer(const FMovieSceneAnimMixerKey& InKey) const;

	// Move-and-erase the per-frame notify dispatch list from the mixer keyed by (BoundObject, Target).
	// Empty when no mixer exists or no batches were produced this frame.
	MOVIESCENEANIMMIXER_API static TArray<FSequencerMixerPendingNotifyBatch> ConsumeMixerNotifyBatches(
		UMovieSceneEntitySystemLinker* Linker,
		UObject* BoundObject,
		const TInstancedStruct<FMovieSceneMixedAnimationTarget>& Target);

	// Iterate all mixer layers known to this system and return the first one matching the predicate.
	MOVIESCENEANIMMIXER_API UMovieSceneAnimationMixerLayer* FindMixerLayer(
		TFunctionRef<bool(UMovieSceneAnimationMixerLayer*)> Predicate) const;

	void PreInitializeAllRootMotion();
	void InitializeAllRootMotion();

	// Bus storage API

	// Store a pose result into the named bus for the given bound object
	MOVIESCENEANIMMIXER_API void StoreBusData(FObjectKey BoundObject, FName BusName, TSharedPtr<FMovieSceneAnimBusData> Data);

	// Read a stored bus pose for the given bound object and bus name (returns null if not written this frame)
	MOVIESCENEANIMMIXER_API TSharedPtr<FMovieSceneAnimBusData> ReadBusData(FObjectKey BoundObject, FName BusName) const;

	// Clear all bus storage (called at the start of each evaluation frame)
	MOVIESCENEANIMMIXER_API void ClearBusStorage();

	// Cached bus evaluation order per bound object
	struct FBusTopologyCache
	{
		TArray<FName> EvalOrder;
		bool bDirty = true;
	};

	FBusTopologyCache& GetOrCreateBusTopology(FObjectKey ObjectKey);
	void RemoveStaleBusTopologies(const TSet<FObjectKey>& ActiveObjects);

	// Force-root-bone scope: while the counter is non-zero, normal (non-bus, non-bake-target)
	// mixer programs flip bForceRootBoneDestination on their FAnimNextStoreRootTransformTask.
	// Used by the binding-level Bake to Control Rig flow so the recorder captures root motion
	// on the root bone rather than having it applied to the actor.
	MOVIESCENEANIMMIXER_API static void PushForceRootBoneDestinationScope();
	MOVIESCENEANIMMIXER_API static void PopForceRootBoneDestinationScope();
	MOVIESCENEANIMMIXER_API static bool IsForceRootBoneDestinationScopeActive();

private:

	// Map of animation mixers
	TMap<FMovieSceneAnimMixerKey, TSharedPtr<FMovieSceneAnimMixer>> Mixers;

	// Bus storage: per-object, per-bus-name pose data written by bus-target mixers
	// and read by bus sections. Cleared at the start of each evaluation frame.
	TMap<FMovieSceneAnimBusStorageKey, TSharedPtr<FMovieSceneAnimBusData>> BusStorage;

	TMap<FObjectKey, FBusTopologyCache> BusTopologyByObject;

	// Map of Root Motion data for each object
	TMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>> RootMotionByObject;

	// Map from actor to all the root motions that may contribute to the actor's transform
	TMultiMap<FObjectKey, TWeakPtr<FMovieSceneMixerRootMotionComponentData>> ActorToRootMotion;

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnCleanTaggedGarbage() override;

	void HandlePreFlush(UMovieSceneEntitySystemLinker* InLinker);

	UPROPERTY()
	TObjectPtr<UMovieSceneRootMotionSystem> RootMotionSystem;

	FDelegateHandle PreFlushHandle;
	bool bInsidePreFlush = false;
};