// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "Injection/IEvaluationModifier.h"
#include "Injection/InjectionRequest.h"
#include "Component/AnimNextComponent.h"
#include "MovieSceneAnimNextTargetSystem.generated.h"


struct FAnimNextEvaluationTask;

struct FMovieSceneAnimMixerEvaluationModifier : public UE::UAF::IEvaluationModifier
{
	FMovieSceneAnimMixerEvaluationModifier(TSharedPtr<FAnimNextEvaluationTask> InTaskToInject)
		: TaskToInject(InTaskToInject)
	{
	}

	// IEvaluationModifier interface
	virtual void PreEvaluate(UE::UAF::FEvaluateTraversalContext& Context) const override {}

	virtual void PostEvaluate(UE::UAF::FEvaluateTraversalContext& Context) const override;

	TSharedPtr<FAnimNextEvaluationTask> TaskToInject = nullptr;

};

/**
 * Declaring a unique target for targeting an injection site on an Unreal Animation Framework module. Will find the default injection site, or one specified by name.
 */
USTRUCT(meta=(DisplayName="UAF Module Injection"))
struct MOVIESCENEANIMMIXER_API FMovieSceneAnimNextInjectionTarget : public FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY()

	FMovieSceneAnimNextInjectionTarget(const FAnimNextSoftVariableReference& InVariableReference)
		: InjectionSite(InVariableReference)
	{

	}

	FMovieSceneAnimNextInjectionTarget()
		: InjectionSite()
	{

	}


	UPROPERTY()
	FName InjectionSiteName_DEPRECATED;

	// Site to use for injection. If empty will use the default one found for the actor.
	UPROPERTY(EditAnywhere, Category="Animation", meta = (AllowedType = "FAnimNextAnimGraph"))
	FAnimNextVariableReference InjectionSite;

	inline friend uint32 GetTypeHash(const FMovieSceneAnimNextInjectionTarget& Target)
	{
		return HashCombine(GetTypeHash(FMovieSceneAnimNextInjectionTarget::StaticStruct()), GetTypeHash(Target.InjectionSite));
	}

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif

#if WITH_EDITOR
	virtual FText GetShortDisplayName() const override
	{
		if (!InjectionSite.IsNone())
		{
			return FText::FromName(InjectionSite.GetName());
		}
		return NSLOCTEXT("MovieSceneAnimNextInjectionTarget", "UAFModuleInjection", "UAF Module Injection");
	}
#endif
};

struct FMovieSceneAnimNextTargetData
{
	TWeakObjectPtr<UUAFComponent> AnimNextComponent = nullptr;
	UE::UAF::FInjectionSite InjectionSite;
	UE::UAF::FInjectionRequestPtr InjectionRequestHandle = nullptr;
	TSharedPtr<FMovieSceneAnimMixerEvaluationModifier> Modifier = nullptr;
};

// System that handles applying animation mixer evaluation tasks to an injection site in an Unreal Animation Framework module
UCLASS(MinimalAPI)
class UMovieSceneAnimNextTargetSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneAnimNextTargetSystem(const FObjectInitializer& ObjInit);

	MOVIESCENEANIMMIXER_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	void OnUnlink() override;

	TArray<FMovieSceneAnimNextTargetData> CurrentTargets;

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
};