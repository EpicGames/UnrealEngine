// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieScenePlayback.h"
#include "UObject/ObjectKey.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EvaluationVM/EvaluationTask.h"
#include "AnimSequencerInstanceProxy.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "Animation/AnimNotifyQueue.h"
#include "MovieSceneMixedSkeletalAnimationSystem.generated.h"

struct FAnimSequencerData;
class UAnimSequence;


USTRUCT()
struct FMixedAnimSkeletalAnimationData
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<UAnimSequence> AnimSequence;

	UPROPERTY()
	TOptional<FSkeletalAnimationRootMotionOverride> RootMotionOverride;

	UPROPERTY()
	double FromPosition = -1.0f;

	UPROPERTY()
	double ToPosition = -1.0f;

	UPROPERTY()
	bool bFireNotifies = true;

	UPROPERTY()
	bool bAdditive = false;

	UPROPERTY()
	bool bSuppressForceRootLock = false;

	// Notifies harvested from the source UAnimSequence for this frame's evaluation window.
	// Not a UPROPERTY: FAnimNotifyEventReference holds raw pointers, not GC references.
	TArray<FAnimNotifyEventReference> PendingNotifies;
};

USTRUCT()
struct FMovieSceneSkeletalAnimationEvaluationTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FMovieSceneSkeletalAnimationEvaluationTask)

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UPROPERTY()
	FMixedAnimSkeletalAnimationData AnimationData;
};

// System to handle creating evaluation tasks from skeletal animation track sections for the anim mixer.
UCLASS(MinimalAPI)
class UMovieSceneMixedSkeletalAnimationSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneMixedSkeletalAnimationSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	

};