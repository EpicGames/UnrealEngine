// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AnimMixerBakeEvaluation.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AttributesRuntime.h"
#include "LODPose.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "MovieSceneMixerBakeTargetSystem.generated.h"

struct FAnimNextEvaluationTask;

namespace UE::UAF
{
	struct FEvaluationVM;
}

// Hidden target struct used during bake evaluation to tell the target system
// to cache results instead of applying them. Never user-facing.
USTRUCT(meta=(Hidden))
struct FMovieSceneMixerBakeTarget : public FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY()

	// Identifies the owning FBakeEvaluationScope. UPROPERTY so that
	// FMovieSceneAnimMixerKey (which hashes the target via reflection over
	// UPROPERTY fields) keys each scope to a distinct mixer entity.
	UPROPERTY()
	int32 ScopeID = 0;

	UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter Filter;
};

// Entity system that captures mixer evaluation results during bake evaluation.
// When bBakeActive is true, this system executes mixer programs in a temporary
// VM and stores results (root motion transform + keyframe) instead of applying
// them to any skeletal mesh.
UCLASS(MinimalAPI)
class UMovieSceneMixerBakeTargetSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:

	UMovieSceneMixerBakeTargetSystem(const FObjectInitializer& Init);

	// Result from a single bake evaluation frame. One per mixer entity that
	// matches FMovieSceneMixerBakeTarget during a flush. 
	struct FBakedResult
	{
		UE::UAF::FLODPoseHeap Pose;
		FBlendedHeapCurve Curves;
		UE::Anim::FHeapAttributeContainer Attributes;
		FTransform RootMotionTransform = FTransform::Identity;

		bool IsValid() const { return Pose.IsValid(); }
	};

	// Accumulated baked results, populated during bake evaluation
	TArray<FBakedResult> BakedResults;

	// When true, the system captures results instead of being a no-op
	bool bBakeActive = false;

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
};
