// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "AnimSequencerInstanceProxy.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Tickable.h"
#include "MovieSceneRootMotionSystem.generated.h"

enum class EMovieSceneRootMotionDestination : uint8;
enum class EMovieSceneRootMotionGapBehavior : uint8;

class UMovieSceneAnimMixerSystem;
struct FMovieSceneDoubleChannel;
class UMovieSceneRootMotionSection;
class UMovieSceneRootMotionSystem;

// Which bone to extract root motion from in the pose
UENUM()
enum class EMovieSceneRootMotionSource : uint8
{
	// Extract root motion from skeleton bone 0 (standard root motion)
	RootBone,
	// Auto-detect the first child of root that has animation data.
	// Use when the root bone is static and locomotion lives on a child bone.
	FirstAnimatedChild,
};

// Enum describing what space that root motion should be applied in.
UENUM()
enum class EMovieSceneRootMotionSpace : uint8
{
	// Root motion should be applied in animation space, meaning that it will be applied on top of the blended transform track and transform origin.
	AnimationSpace,
	// Root motion should be applied in world space, meaning that it will override any transform track or transform origin.
	WorldSpace
};


// Enum describing what space that root motion should be applied in.
UENUM()
enum class EMovieSceneRootMotionTransformMode : uint8
{
	// Uses root motion exclusively from the asset as is
	Asset,
	// Adds on the offset field onto the root motion from the asset
	Offset,
	// Uses the offset field as the entire root motion, ignoring root motion from the asset
	Override,
	// Uses the accumulated offset from previous KeepState sections as a starting position.
	// The section's own root motion plays on top of this accumulated transform.
	AccumulatedOffset,
	// Uses a bone-match-derived offset to position the section so a specific bone
	// aligns with the underlying pose at the match time. Set automatically by the
	// bone match dialog. Manual offsets are additive on top.
	BoneMatch,
};

UENUM()
enum class EMovieSceneAnimLoopMode : uint8
{
	// Preserves the section's root bone offset at the loop boundary so the
	// animation continues spatially from where the previous loop ended.
	Accumulated,
	// Resets the section's root bone back to its original start position at
	// each loop boundary.
	Reset
};

USTRUCT()
struct FSkeletalAnimationRootMotionOverride
{
	GENERATED_BODY()
	
	UPROPERTY()
	FTransform RootMotion = FTransform::Identity;

	UPROPERTY()
	int32 ChildBoneIndex = INDEX_NONE;

	/** If true we use the ChildBoneIndex otherwise we use the root*/
	UPROPERTY()
	bool bBlendFirstChildOfRoot = false;
};

// Structure used for animation tracks to communicate to the mixer how they would like root motion handled if at all.
struct FMovieSceneRootMotionSettings
{
	FVector RootLocation = FVector(EForceInit::ForceInit);
	FVector RootOriginLocation = FVector(EForceInit::ForceInit);
	FVector RootOverrideLocation = FVector(EForceInit::ForceInit);

	FQuat   RootRotation = FQuat(EForceInit::ForceInit);
	FQuat   RootOverrideRotation = FQuat(EForceInit::ForceInit);

	int32 ChildBoneIndex = INDEX_NONE;

	// What space to apply root motion in. Defaults to animation space.
	EMovieSceneRootMotionSpace RootMotionSpace = EMovieSceneRootMotionSpace::AnimationSpace;
	EMovieSceneRootMotionTransformMode TransformMode = EMovieSceneRootMotionTransformMode::Offset;
	EMovieSceneRootMotionSource RootMotionSource = EMovieSceneRootMotionSource::RootBone;
	ESwapRootBone LegacySwapRootBone = ESwapRootBone::SwapRootBone_None;
	EMovieSceneAnimLoopMode LoopMode = EMovieSceneAnimLoopMode::Accumulated;

	uint8 bHasRootMotionOverride : 1 = 0;
	/** If true we use the ChildBoneIndex otherwise we use the root*/
	uint8 bBlendFirstChildOfRoot : 1 = 0;
	/** If true, the animation has Force Root Lock + Enable Root Motion. The extraction task
	 *  handles root locking, and the eval task suppresses the lock during pose evaluation. */
	uint8 bForceRootLock : 1 = 0;

	// Offset channels from the root motion settings decoration.
	// Evaluated per-frame by the consumer to populate RootLocation/RootRotation.
	// [0..2] = Location X/Y/Z, [3..5] = Rotation Roll/Pitch/Yaw.
	const FMovieSceneDoubleChannel* OffsetChannels[6] = {};
};

// Structure that is shared between entities for handling the mixer's root motion result. 
// As this can get read/written from multiple threads, access is threadsafe.
struct FMovieSceneMixerRootMotionComponentData
{
public:

	TOptional<FQuat> GetInverseMeshToActorRotation() const;

	void Initialize();

public:

	TWeakObjectPtr<USceneComponent> OriginalBoundObject;

	// Where to apply the root motion
	TWeakObjectPtr<USceneComponent> Target;

	// EntityID for the anim mixer
	UE::MovieScene::FMovieSceneEntityID MixerEntityID;

	EMovieSceneRootMotionDestination RootDestination;

	FTransform ActorTransform;
	FTransform ComponentToActorTransform;

private:

	// Optional inverse mesh component to actor rotation used to offset any mesh component rotation where applicable.
	TOptional<FQuat> InverseMeshToActorRotation;

	mutable FTransactionallySafeRWLock RootMotionLock;

public:

	// The root motion offset applied to the component, relative to the actor heading.
	// Stored each evaluation so the transform track editor can subtract it during auto-key.
	FTransform AppliedRootMotionOffset = FTransform::Identity;

	bool bComponentSpaceRoot = false;
	bool bActorTransformSet = false;

	// Gap behavior fields
	EMovieSceneRootMotionGapBehavior GapBehavior{};
	FTransform CachedGapTransform = FTransform::Identity;
	TWeakObjectPtr<UMovieSceneRootMotionSection> WeakRootMotionSection;
};

// Task that converts the root motion attribute on the top pose of the pose stack to world space by adding on the actor transformation, root base transform, and/or transform origin.
USTRUCT()
struct FAnimNextConvertRootMotionToWorldSpaceTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextConvertRootMotionToWorldSpaceTask)

	/** Enum specifying which conversions to perform */
	enum class ESpaceConversions : uint8
	{
		None = 0,
		/** Convert the root motion from animation space to world space */
		AnimationToWorld = 1 << 0,
		/** Convert the root motion from transform origin space to world space (used when there is no transform track in Sequencer) */
		TransformOriginToWorld = 1 << 1,
		/** Convert the root motion from component -> actor space using the inverse component rotation only */
		ComponentToActorRotation = 1 << 2,
		/** Compensate for component rotation and translation offsets when applying root motion in world space */
		WorldSpaceComponentTransformCompensation = 1 << 3,
		/** Apply RootBaseTransform as an offset around RootOffsetOrigin where Root = Root * RootBaseTransform */
		RootTransformOffset = 1 << 4,
		/** Completely override the root transform with RootTransform */
		RootTransformOverride = 1 << 5,
		/** Apply bone-match-derived offset to align a specific bone with the underlying pose */
		BoneMatchOffset = 1 << 7,
	};

	FAnimNextConvertRootMotionToWorldSpaceTask() = default;
	FAnimNextConvertRootMotionToWorldSpaceTask(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, const FTransform& InTransformOrigin, const FTransform& InRootTransform, const FVector& InRootOffsetOrigin, ESpaceConversions InConversions);

	static FAnimNextConvertRootMotionToWorldSpaceTask Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, const FTransform& InTransformOrigin, const FTransform& InRootTransform, const FVector& InRootOffsetOrigin, ESpaceConversions InConversions);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	/** Weak pointer to the root motion data for all mixed animations. May be null if only a transform origin transformation is required. */
	TWeakPtr<FMovieSceneMixerRootMotionComponentData> WeakRootMotionData;
	/** Base transformation to apply to the root in Actor space, before transform origins. Can be used in place of a transform track. */
	FTransform RootTransform;
	/** Transform origin to apply to the root, if Conversions & ESpaceConversions::TransformOriginToWorld */
	FTransform TransformOrigin;
	/** Origin around which to apply RootTransform when space conversion is RootTransformOffset. */
	FVector RootOffsetOrigin;
	/** Anchor the rotated delta is multiplied by after offset application: the
	 *  accumulated chain when the section is accumulating, InitialAnimRootTransform otherwise. */
	FTransform AnimSpaceAccumulatedTransform;
	/** Initial root transform of this section evaluated in isolation at its start time.
	 *  Used to compute the section-local delta: Delta = N_t * Inv(InitialAnimRootTransform). */
	FTransform InitialAnimRootTransform;
	/** Bone-match-derived root offset. Applied as post-multiply in animation space
	 *  so the matched bone aligns with the underlying pose at the match time. */
	FTransform BoneMatchTransform;
	/** Bitmask specifying the conversions to perform */
	ESpaceConversions Conversions;
	/** When set (bake only), the pre-conversion animation-space root is written here */
	TSharedPtr<FTransform> CaptureAnimSpaceRoot;
};
ENUM_CLASS_FLAGS(FAnimNextConvertRootMotionToWorldSpaceTask::ESpaceConversions);

// Task that gets the final mixed root transform and stores it in the root motion data for later application.
// TODO: it's not ideal that we're writing things outside of the animation system during an evaluation task. 
// Consider refactoring this once we have a way to hook into anim next post execution
USTRUCT()
struct FAnimNextStoreRootTransformTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextStoreRootTransformTask)

	FAnimNextStoreRootTransformTask() = default;
	FAnimNextStoreRootTransformTask(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, bool bInComponentHasKeyedTransform, bool bInRootComponentHasKeyedTransform);

	static FAnimNextStoreRootTransformTask Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData, bool bInComponentHasKeyedTransform, bool bInRootComponentHasKeyedTransform);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	TWeakPtr<FMovieSceneMixerRootMotionComponentData> WeakRootMotionData;
	bool bComponentHasKeyedTransform = false;
	bool bRootComponentHasKeyedTransform = false;

	// When true, forces RootBone destination regardless of the Root Motion Target
	// settings. Used by bus-target mixers to put world-space root motion back onto
	// the root bone so that outer mixers can re-extract and apply it.
	bool bForceRootBoneDestination = false;
};


// Task that sets the RootTransformAttribute on the top keyframe, creating it
// if it doesn't exist. Used for gap behavior (PersistPreviousTransform) where
// we need to inject a cached world-space transform into an empty or
// non-root-motion pose. Unlike FAnimNextConvertRootMotionToWorldSpaceTask,
// this doesn't require an existing attribute and performs no space conversion
// (the gap transform is already in the correct post-conversion space from the
// trajectory cache).
USTRUCT()
struct FAnimNextSetRootTransformAttributeTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextSetRootTransformAttributeTask)

	FAnimNextSetRootTransformAttributeTask() = default;

	static FAnimNextSetRootTransformAttributeTask Make(const FTransform& InTransform);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	FTransform Transform;
};

// Writes the RootTransformAttribute onto the top pose's bone[0] in actor-relative space
// so a non-consuming eval task (e.g. a layered Control Rig) can read it. Paired with
// FAnimNextResetRootBoneTask to restore the input pose afterwards.
USTRUCT()
struct FAnimNextBakeRootAttributeToBoneTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextBakeRootAttributeToBoneTask)

	FAnimNextBakeRootAttributeToBoneTask() = default;

	static FAnimNextBakeRootAttributeToBoneTask Make(const TSharedPtr<FMovieSceneMixerRootMotionComponentData>& InRootMotionData);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	TWeakPtr<FMovieSceneMixerRootMotionComponentData> WeakRootMotionData;
};

// Undoes FAnimNextBakeRootAttributeToBoneTask: targets stack index 1 (the bake's pose,
// pushed down when a non-consuming eval task ran on top) and resets bone[0] to identity.
USTRUCT()
struct FAnimNextResetRootBoneTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextResetRootBoneTask)

	FAnimNextResetRootBoneTask() = default;

	static FAnimNextResetRootBoneTask Make();

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};

// Generic root motion extraction task. Reads the top keyframe's pose,
// extracts root motion from either the root bone or the first animated
// child of root based on the Source setting, and stores the transform as a
// RootTransformAttribute. For the root bone, the bone is zeroed in the
// pose. For a child bone, the delta from the reference pose is extracted
// and the bone is restored to its reference transform.
// Inserted into an entry mixer program after the section's eval task and
// before the space conversion task.
USTRUCT()
struct FAnimNextExtractRootMotionTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextExtractRootMotionTask)

	FAnimNextExtractRootMotionTask() = default;

	static FAnimNextExtractRootMotionTask Make(EMovieSceneRootMotionSource InSource);

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	EMovieSceneRootMotionSource Source = EMovieSceneRootMotionSource::RootBone;
};

// Takes in evaluation tasks on mixers.
// Mixes just the root motion attributes.
// Converts it from animation space to either additive actor or component space (based on which attribute used).
// Writes it out as an additive transform to be mixed alongside other transform track values.
UCLASS(MinimalAPI)
class UMovieSceneRootMotionSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneRootMotionSystem(const FObjectInitializer& ObjInit);

	bool IsTransformKeyed(const FObjectKey& Object) const;

private:

	virtual void OnLink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;

	TSet<FObjectKey> ObjectsWithTransforms;
};
