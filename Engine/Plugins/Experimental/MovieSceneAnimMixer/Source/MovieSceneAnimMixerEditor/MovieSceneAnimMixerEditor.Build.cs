// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieSceneAnimMixerEditor : ModuleRules
	{
		public MovieSceneAnimMixerEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AnimationCore",
					"AnimGraph",
					"AnimGraphRuntime", 
					"Core",
					"CoreUObject",
					"SequencerCore",
					"Sequencer",
					"MovieSceneAnimMixer",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"BlueprintGraph",
					"EditorFramework",
					"Engine",
					"InputCore",
					"LevelEditor",
					"LevelSequence",
					"Persona",
					"MovieScene",
					"MovieSceneAnimMixer",
					"MovieSceneTracks",
					"MovieSceneTools",
					"Sequencer",
					"SequencerAnimTools",
					"SequencerCore",
					"SequencerWidgets",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UAF",
					"UAFEditor",
					"UAFUncookedOnly",
					"UnrealEd",
					"PropertyEditor",
					"EditorInteractiveToolsFramework"
				}
			);
		}
	}
}