// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationMaskDecorationEditor.h"

#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneAnimationMaskDecoration.h"

namespace UE::Sequencer
{
	UClass* FMovieSceneAnimationMaskDecorationEditor::GetDecorationClass() const
	{
		return UMovieSceneAnimationMaskDecoration::StaticClass();
	}

	const FSlateBrush* FMovieSceneAnimationMaskDecorationEditor::GetIconBrush() const
	{
		return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.Masking");
	}

	TSharedPtr<ISequencerSection> FMovieSceneAnimationMaskDecorationEditor::MakeSectionInterface(UMovieSceneSection& Section, UMovieSceneTrack& OwningTrack, FGuid ObjectBinding)
	{
		return MakeShared<FAnimMixerMaskSection>(Section);
	}
}
