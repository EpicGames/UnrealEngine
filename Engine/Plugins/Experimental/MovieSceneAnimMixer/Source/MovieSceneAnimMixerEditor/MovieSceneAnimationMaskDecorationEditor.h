// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ISequencerDecorationEditor.h"
#include "ISequencerSection.h"
#include "MovieSceneAnimMixerMaskSection.h"
#include "MovieSceneTrack.h"

namespace UE::Sequencer
{

class FAnimMixerMaskSection : public FSequencerSection
{
public:

	FAnimMixerMaskSection(UMovieSceneSection& InSection)
		: FSequencerSection(InSection)
	{}
	/** ISequencerSection Interface */

	virtual FText GetSectionTitle() const override
	{
		if (TStrongObjectPtr<UMovieSceneSection> Section = WeakSection.Pin())
		{
			if (UMovieSceneAnimMixerMaskSection* MaskSection = Cast<UMovieSceneAnimMixerMaskSection>(Section.Get()))
			{
				if (const UUAFBlendMask* BlendMask = MaskSection->GetBlendMask().ResolveObjectPtr())
				{
					return FText::FromString(BlendMask->GetName());
				}
			}
		}
		return  FText();
	}
};

class FMovieSceneAnimationMaskDecorationEditor : public ISequencerDecorationEditor
{
public:
	virtual UClass* GetDecorationClass() const override;
	virtual const FSlateBrush* GetIconBrush() const override;

	virtual TSharedPtr<ISequencerSection> MakeSectionInterface(UMovieSceneSection& Section, UMovieSceneTrack& OwningTrack, FGuid ObjectBinding) override;

};

}
