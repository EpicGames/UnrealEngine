// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBusSectionInterface.h"

#include "AnimMixerBusUtils.h"
#include "AnimMixerEditorBusUtils.h"
#include "ISequencer.h"
#include "MovieSceneAnimBusSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneSequence.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "AnimBusSectionInterface"

namespace UE::Sequencer
{

FAnimBusSectionInterface::FAnimBusSectionInterface(UMovieSceneSection& InSection, UMovieSceneTrack& InTrack, FGuid InObjectBinding, TWeakPtr<ISequencer> InSequencer)
	: FSequencerSection(InSection)
	, WeakTrack(&InTrack)
	, WeakSequencer(InSequencer)
	, ObjectBindingId(InObjectBinding)
{
}

const FSlateBrush* FAnimBusSectionInterface::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.Bus");
}

FText FAnimBusSectionInterface::GetSectionTitle() const
{
	UMovieSceneAnimBusSection* BusSection = GetBusSection();
	if (BusSection && !BusSection->BusName.IsNone())
	{
		return FText::Format(LOCTEXT("BusSectionTitleFmt", "Bus: {0}"), FText::FromName(BusSection->BusName));
	}
	return LOCTEXT("BusSectionTitle", "Bus");
}

FText FAnimBusSectionInterface::GetSectionToolTip() const
{
	UMovieSceneAnimBusSection* BusSection = GetBusSection();
	if (BusSection && !BusSection->BusName.IsNone())
	{
		return FText::Format(LOCTEXT("BusSectionTooltipFmt", "Reads pose from bus '{0}'"), FText::FromName(BusSection->BusName));
	}
	return LOCTEXT("BusSectionTooltip", "Bus section (no bus assigned)");
}

UMovieSceneAnimBusSection* FAnimBusSectionInterface::GetBusSection() const
{
	return Cast<UMovieSceneAnimBusSection>(WeakSection.Get());
}

UMovieSceneAnimationMixerTrack* FAnimBusSectionInterface::GetMixerTrack() const
{
	return Cast<UMovieSceneAnimationMixerTrack>(WeakTrack.Get());
}

void FAnimBusSectionInterface::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("ChangeBus", "Change Bus..."),
		LOCTEXT("ChangeBusTooltip", "Change which bus this section reads from."),
		FNewMenuDelegate::CreateSP(this, &FAnimBusSectionInterface::PopulateChangeBusMenu)
	);
}

void FAnimBusSectionInterface::PopulateChangeBusMenu(FMenuBuilder& MenuBuilder)
{
	TWeakPtr<FAnimBusSectionInterface> WeakThis = SharedThis(this);
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneAnimationMixerTrack* MixerTrack = GetMixerTrack();

	TArray<UMovieSceneAnimationMixerTrack*> TracksForObject = Sequencer
		? AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(MixerTrack, *Sequencer)
		: TArray<UMovieSceneAnimationMixerTrack*>();

	// List buses referenced by this character. Buses are scoped per bound object,
	// so only surface buses this character's own tracks reference - buses on other
	// characters are unreachable from here.
	TArray<FName> KnownBuses = AnimMixerEditorBusUtils::GatherBusNamesFromTracks(TracksForObject);

	UMovieSceneAnimBusSection* BusSection = GetBusSection();
	const FName CurrentBusName = BusSection ? BusSection->BusName : NAME_None;

	for (const FName& BusName : KnownBuses)
	{
		if (BusName == CurrentBusName)
		{
			continue;
		}

		const bool bWouldCycle = MixerTrack &&
			FAnimMixerBusUtils::WouldBusSectionCreateCycle(BusName, MixerTrack, TracksForObject);

		MenuBuilder.AddMenuEntry(
			FText::FromName(BusName),
			bWouldCycle
				? FText::Format(LOCTEXT("ChangeBusCycleTooltip", "Cannot read from bus '{0}': would create a dependency cycle."), FText::FromName(BusName))
				: FText::Format(LOCTEXT("ChangeBusEntryTooltip", "Read from bus '{0}'."), FText::FromName(BusName)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakThis, BusName]()
				{
					if (TSharedPtr<FAnimBusSectionInterface> This = WeakThis.Pin())
					{
						This->ChangeBusName(BusName);
					}
					FSlateApplication::Get().DismissAllMenus();
				}),
				FCanExecuteAction::CreateLambda([bWouldCycle]() { return !bWouldCycle; }))
		);
	}

	// "New Bus..." text entry
	TSharedRef<SEditableTextBox> TextBox = SNew(SEditableTextBox)
		.HintText(LOCTEXT("BusNameHint", "Enter bus name..."))
		.OnTextCommitted_Lambda([WeakThis, TracksForObject](const FText& InText, ETextCommit::Type CommitType)
		{
			TSharedPtr<FAnimBusSectionInterface> This = WeakThis.Pin();
			if (!This)
			{
				return;
			}
			if (CommitType == ETextCommit::OnEnter && !InText.IsEmpty())
			{
				FName BusName(*InText.ToString());
				UMovieSceneAnimationMixerTrack* Track = This->GetMixerTrack();
				if (Track && FAnimMixerBusUtils::WouldBusSectionCreateCycle(BusName, Track, TracksForObject))
				{
					FNotificationInfo Info(FText::Format(
						LOCTEXT("ChangeBusCycleBlocked", "Cannot read from bus '{0}': would create a dependency cycle."),
						FText::FromName(BusName)));
					Info.ExpireDuration = 4.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
					return;
				}
				This->ChangeBusName(BusName);
				FSlateApplication::Get().DismissAllMenus();
			}
		});

	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(4.0f)
		.FillWidth(1.0f)
		[
			TextBox
		],
		LOCTEXT("NewBusNameLabel", "Bus Name"),
		/*bNoIndent=*/ true
	);
}

void FAnimBusSectionInterface::ChangeBusName(FName NewBusName)
{
	UMovieSceneAnimBusSection* BusSection = GetBusSection();
	if (!BusSection || NewBusName.IsNone())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ChangeBusName", "Change Bus Name"));
	BusSection->Modify();
	BusSection->BusName = NewBusName;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
