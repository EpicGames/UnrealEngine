// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "Templates/SharedPointer.h"

class UMovieSceneAnimBusSection;
class UMovieSceneAnimationMixerTrack;
class FMenuBuilder;
class ISequencer;

namespace UE::Sequencer
{

// Section interface for bus sections in Animation Mixer tracks.
// Inherits from FSequencerSection for proper background rendering,
// and adds bus-specific title and context menu.
class FAnimBusSectionInterface
	: public FSequencerSection
	, public TSharedFromThis<FAnimBusSectionInterface>
{
public:
	FAnimBusSectionInterface(UMovieSceneSection& InSection, UMovieSceneTrack& InTrack, FGuid InObjectBinding, TWeakPtr<ISequencer> InSequencer);

	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetSectionTitle() const override;
	virtual FText GetSectionToolTip() const override;
	virtual bool SectionIsResizable() const override { return true; }
	virtual bool IsDeletable() const override { return true; }
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;

private:
	void PopulateChangeBusMenu(FMenuBuilder& MenuBuilder);
	void ChangeBusName(FName NewBusName);

	UMovieSceneAnimBusSection* GetBusSection() const;
	UMovieSceneAnimationMixerTrack* GetMixerTrack() const;

	TWeakObjectPtr<UMovieSceneTrack> WeakTrack;
	TWeakPtr<ISequencer> WeakSequencer;
	FGuid ObjectBindingId;
};

} // namespace UE::Sequencer
