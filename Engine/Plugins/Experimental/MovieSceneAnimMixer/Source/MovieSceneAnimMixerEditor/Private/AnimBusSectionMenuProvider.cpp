// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBusSectionMenuProvider.h"

#include "AnimMixerBusUtils.h"
#include "AnimMixerEditorBusUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ISequencer.h"
#include "MovieSceneAnimBusSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneSequence.h"
#include "ScopedTransaction.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Components/SkeletalMeshComponent.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "GameFramework/Actor.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AnimBusSectionMenuProvider"

const UClass* FAnimBusSectionMenuProvider::GetHandledMixerItemClass() const
{
	return UMovieSceneAnimBusSection::StaticClass();
}

void FAnimBusSectionMenuProvider::PopulateAddMixerItemMenu(
	FMenuBuilder& MenuBuilder,
	TArray<FGuid> ObjectBindings,
	UMovieSceneTrack* Track,
	TSharedPtr<ISequencer> Sequencer,
	int32 RowIndex)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
	if (!MixerTrack || !Sequencer)
	{
		return;
	}

	// Gather the mixer tracks bound to this character. Used both to filter the
	// bus list and to run cycle checks in the submenu. Buses are scoped per bound
	// object, so only surface buses this character's own tracks reference.
	TArray<UMovieSceneAnimationMixerTrack*> TracksForObject =
		AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(MixerTrack, *Sequencer);

	TArray<FName> KnownBuses = AnimMixerEditorBusUtils::GatherBusNamesFromTracks(TracksForObject);

	TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakMixerTrack = MixerTrack;
	TWeakPtr<ISequencer> WeakSequencer = Sequencer;

	// Shared creation function used by both the known-bus entries and the new-bus text field
	TFunction<void(FName)> CreateBusSection = [WeakMixerTrack, RowIndex, WeakSequencer](FName InBusName)
	{
		TSharedPtr<ISequencer> Seq = WeakSequencer.Pin();
		UMovieSceneAnimationMixerTrack* MixTrack = WeakMixerTrack.Get();
		if (!Seq || !MixTrack || InBusName.IsNone())
		{
			return;
		}

		// Check for cycles before modifying the track
		TArray<UMovieSceneAnimationMixerTrack*> TracksForObject = AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(MixTrack, *Seq);
		if (FAnimMixerBusUtils::WouldBusSectionCreateCycle(InBusName, MixTrack, TracksForObject))
		{
			FNotificationInfo Info(LOCTEXT("BusCycleDetected", "Cannot add bus section: this would create a dependency cycle."));
			Info.ExpireDuration = 4.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("AddBusSection", "Add Bus Section"));
		MixTrack->Modify();

		UMovieSceneAnimBusSection* NewSection = NewObject<UMovieSceneAnimBusSection>(MixTrack, NAME_None, RF_Transactional);
		NewSection->BusName = InBusName;

		// Determine the row index for the new section
		int32 NewRowIndex = RowIndex;
		if (NewRowIndex == INDEX_NONE)
		{
			int32 MaxRowIndex = -1;
			for (UMovieSceneSection* Section : MixTrack->GetAllSections())
			{
				MaxRowIndex = FMath::Max(MaxRowIndex, Section->GetRowIndex());
			}
			NewRowIndex = FMath::Max(MaxRowIndex + 1, MixTrack->GetNextChildTrackRowIndex());
		}
		NewSection->SetRowIndex(NewRowIndex);

		static_cast<UMovieSceneTrack*>(MixTrack)->AddSection(UMovieSceneTrack::FSectionParameter{*NewSection});

		// Auto-add root motion settings if the bus source has root motion
		if (TSharedPtr<UE::MovieScene::FSharedPlaybackState> PlaybackState = Seq->FindSharedPlaybackState())
		{
			if (UMovieSceneEntitySystemLinker* SeqLinker = PlaybackState->GetLinker())
			{
				if (UMovieSceneAnimMixerSystem* MixerSystem = SeqLinker->FindSystem<UMovieSceneAnimMixerSystem>())
				{
					FGuid BindingGuid = MixTrack->FindObjectBindingGuid();
					TArrayView<TWeakObjectPtr<>> BoundObjects = Seq->FindObjectsInCurrentSequence(BindingGuid);
					for (const TWeakObjectPtr<>& WeakObj : BoundObjects)
					{
						if (UObject* BoundObj = WeakObj.Get())
						{
							// Resolve to the skeletal mesh component, matching
							// how the mixer system keys bus storage.
							UObject* ResolvedObj = BoundObj;
							if (AActor* Actor = Cast<AActor>(BoundObj))
							{
								if (USkeletalMeshComponent* SkelMesh = Actor->FindComponentByClass<USkeletalMeshComponent>())
								{
									ResolvedObj = SkelMesh;
								}
							}
							TSharedPtr<FMovieSceneAnimBusData> BusData = MixerSystem->ReadBusData(FObjectKey(ResolvedObj), InBusName);
							if (BusData && BusData->bHasRootMotion)
							{
								auto* RootMotionSettings = NewObject<UMovieSceneRootMotionSettingsDecoration>(
									NewSection, UMovieSceneRootMotionSettingsDecoration::StaticClass(), NAME_None, RF_Transactional);
								
								// Default to world space since buses bake their root motion back to the root bone in world space
								RootMotionSettings->RootMotionSpace.SetDefault((uint8)EMovieSceneRootMotionSpace::WorldSpace);
								NewSection->AddDecoration(RootMotionSettings);
								break;
							}
						}
					}
				}
			}
		}

		Seq->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("BusSectionMenu", "Bus"),
		LOCTEXT("BusSectionMenuTooltip", "Add a bus section that reads from a named bus."),
		FNewMenuDelegate::CreateLambda([KnownBuses, CreateBusSection, WeakMixerTrack, TracksForObject](FMenuBuilder& SubMenuBuilder)
		{
			UMovieSceneAnimationMixerTrack* MixTrack = WeakMixerTrack.Get();

			for (const FName& BusName : KnownBuses)
			{
				const bool bWouldCycle = MixTrack &&
					FAnimMixerBusUtils::WouldBusSectionCreateCycle(BusName, MixTrack, TracksForObject);

				SubMenuBuilder.AddMenuEntry(
					FText::FromName(BusName),
					bWouldCycle
						? FText::Format(LOCTEXT("BusSectionCycleTooltip", "Cannot read from bus '{0}': would create a dependency cycle."), FText::FromName(BusName))
						: FText::Format(LOCTEXT("BusSectionEntryTooltip", "Read from bus '{0}'."), FText::FromName(BusName)),
					FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Bus"),
					FUIAction(
						FExecuteAction::CreateLambda([CreateBusSection, BusName]() { CreateBusSection(BusName); }),
						FCanExecuteAction::CreateLambda([bWouldCycle]() { return !bWouldCycle; }))
				);
			}

			// "New Bus..." with inline text input
			TSharedRef<SEditableTextBox> TextBox = SNew(SEditableTextBox)
				.HintText(LOCTEXT("NewBusNameHint", "Enter bus name..."))
				.OnTextCommitted_Lambda([CreateBusSection](const FText& InText, ETextCommit::Type CommitType)
				{
					if (CommitType == ETextCommit::OnEnter && !InText.IsEmpty())
					{
						FName BusName(*InText.ToString());
						CreateBusSection(BusName);
						FSlateApplication::Get().DismissAllMenus();
					}
				});

			SubMenuBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(4.0f)
				.FillWidth(1.0f)
				[
					TextBox
				],
				LOCTEXT("NewBusLabel", "New Bus..."),
				/*bNoIndent=*/ true
			);
		}),
		/*bInOpenSubMenuOnClick=*/ false,
		FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Bus")
	);
}

#undef LOCTEXT_NAMESPACE
