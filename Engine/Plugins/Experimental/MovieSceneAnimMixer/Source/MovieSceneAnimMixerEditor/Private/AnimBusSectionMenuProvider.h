// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneAnimMixerItemMenuProvider.h"

// Menu provider for adding bus sections to Animation Mixer layers.
// Shows a submenu with known bus names plus a "New Bus..." entry.
class FAnimBusSectionMenuProvider : public IMovieSceneAnimMixerItemMenuProvider
{
public:
	virtual const UClass* GetHandledMixerItemClass() const override;
	virtual void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) override;
	virtual int32 GetMixerItemMenuPriority() const override { return 50; }
};
