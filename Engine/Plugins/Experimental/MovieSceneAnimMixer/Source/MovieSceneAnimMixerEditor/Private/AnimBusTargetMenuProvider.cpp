// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBusTargetMenuProvider.h"

#include "AnimMixerBusUtils.h"
#include "AnimMixerEditorBusUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "MovieSceneSequence.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "AnimBusTargetMenuProvider"

UScriptStruct* FAnimBusTargetMenuProvider::GetHandledTargetStructType() const
{
	return FMovieSceneAnimBusTarget::StaticStruct();
}

void FAnimBusTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected,
	TSharedPtr<ISequencer> Sequencer,
	UMovieSceneAnimationMixerTrack* Track)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("BusTarget", "Bus Target"),
		LOCTEXT("BusTargetTooltip", "Write the mixer's evaluated pose to a named bus for consumption by other mixers."),
		FNewMenuDelegate::CreateRaw(this, &FAnimBusTargetMenuProvider::PopulateBusTargetSubmenu, BoundObject, OnTargetSelected, Sequencer, Track)
	);
}

void FAnimBusTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	PopulateTargetMenu(MenuBuilder, BoundObject, OnTargetSelected, nullptr, nullptr);
}

void FAnimBusTargetMenuProvider::PopulateBusTargetSubmenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected,
	TSharedPtr<ISequencer> Sequencer,
	UMovieSceneAnimationMixerTrack* Track)
{
	TArray<UMovieSceneAnimationMixerTrack*> TracksForObject = Sequencer
		? AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(Track, *Sequencer)
		: TArray<UMovieSceneAnimationMixerTrack*>();

	// List buses referenced by this character. Buses are scoped per bound object,
	// so only surface buses this character's own tracks reference - buses on other
	// characters are unreachable from here.
	TArray<FName> KnownBuses = AnimMixerEditorBusUtils::GatherBusNamesFromTracks(TracksForObject);

	for (const FName& BusName : KnownBuses)
	{
		const bool bWouldCycle = Track &&
			FAnimMixerBusUtils::WouldBusTargetCreateCycle(BusName, Track, TracksForObject);

		MenuBuilder.AddMenuEntry(
			FText::FromName(BusName),
			bWouldCycle
				? FText::Format(LOCTEXT("BusTargetCycleTooltip", "Cannot target bus '{0}': would create a dependency cycle."), FText::FromName(BusName))
				: FText::Format(LOCTEXT("BusTargetEntryTooltip", "Write to bus '{0}'."), FText::FromName(BusName)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([OnTargetSelected, BusName]()
				{
					OnTargetSelected(TInstancedStruct<FMovieSceneAnimBusTarget>::Make(BusName));
					FSlateApplication::Get().DismissAllMenus();
				}),
				FCanExecuteAction::CreateLambda([bWouldCycle]() { return !bWouldCycle; }))
		);
	}

	// "New Bus..." text entry
	TSharedRef<SEditableTextBox> TextBox = SNew(SEditableTextBox)
		.HintText(LOCTEXT("NewBusNameHint", "Enter bus name..."))
		.OnTextCommitted_Lambda([OnTargetSelected](const FText& InText, ETextCommit::Type CommitType)
		{
			if (CommitType == ETextCommit::OnEnter && !InText.IsEmpty())
			{
				FName BusName(*InText.ToString());
				OnTargetSelected(TInstancedStruct<FMovieSceneAnimBusTarget>::Make(BusName));
				FSlateApplication::Get().DismissAllMenus();
			}
		});

	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(4.0f)
		.FillWidth(1.0f)
		[
			TextBox
		],
		LOCTEXT("NewBusLabel", "New Bus..."),
		/*bNoIndent=*/ true
	);
}

#undef LOCTEXT_NAMESPACE
