// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneAnimMixerTargetMenuProvider.h"

// Target menu provider for the Bus Target type.
// Shows a submenu with known bus names plus a "New Bus..." entry.
class FAnimBusTargetMenuProvider : public IMovieSceneAnimMixerTargetMenuProvider
{
public:
	virtual UScriptStruct* GetHandledTargetStructType() const override;

	// Override the 5-param version to use sequencer/track for bus listing and cycle checks
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected,
		TSharedPtr<ISequencer> Sequencer,
		UMovieSceneAnimationMixerTrack* Track) override;

	// 3-param fallback
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) override;

	virtual int32 GetTargetMenuPriority() const override { return 40; }

private:
	void PopulateBusTargetSubmenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected,
		TSharedPtr<ISequencer> Sequencer,
		UMovieSceneAnimationMixerTrack* Track);
};
