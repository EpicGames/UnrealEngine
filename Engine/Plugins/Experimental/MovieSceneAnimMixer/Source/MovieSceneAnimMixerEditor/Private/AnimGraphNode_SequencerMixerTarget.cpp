// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SequencerMixerTarget.h"

#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_SequencerMixerTarget.h"

#define LOCTEXT_NAMESPACE "SequencerMixerNodes"

FLinearColor UAnimGraphNode_SequencerMixerTarget::GetNodeTitleColor() const
{
	return Super::GetNodeTitleColor();
}

FText UAnimGraphNode_SequencerMixerTarget::GetTooltipText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("TargetName"), FText::FromName(Node.TargetName));
	
	return FText::Format(LOCTEXT("TargetNodeToolTip", "Receives Animation Mixer results from animation sections in a Level Sequence. Target Name: '{TargetName}'"), Args);
}

FText UAnimGraphNode_SequencerMixerTarget::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("TargetName"), FText::FromName(Node.TargetName));
	
	return FText::Format(LOCTEXT("TargetNodeTitle", "Sequencer Target: '{TargetName}'"), Args);
}

FString UAnimGraphNode_SequencerMixerTarget::GetNodeCategory() const
{
	return TEXT("Sequencer|Animation");
}

void UAnimGraphNode_SequencerMixerTarget::BakeDataDuringCompilation(FCompilerResultsLog& MessageLog)
{
	Super::BakeDataDuringCompilation(MessageLog);
}

void UAnimGraphNode_SequencerMixerTarget::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	Super::GetRequiredExtensions(OutExtensions);
	OutExtensions.Add(UAnimBlueprintExtension_SequencerMixerTarget::StaticClass());
}

#undef LOCTEXT_NAMESPACE
