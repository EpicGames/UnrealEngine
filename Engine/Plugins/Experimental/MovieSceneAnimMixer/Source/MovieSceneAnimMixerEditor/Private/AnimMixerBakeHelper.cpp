// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerBakeHelper.h"

#include "AnimMixerBakeEvaluation.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "AnimationEditorUtils.h"
#include "Factories/AnimSequenceFactory.h"
#include "Exporters/AnimSeqExportOption.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Features/IModularFeatures.h"
#include "IMovieSceneAnimMixerBakeProvider.h"

#include "Components/SkeletalMeshComponent.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "AnimMixerBakeHelper"

namespace
{

// Returns true if the name matches any string in the filter list (substring match)
bool MatchesNameFilter(const FName& InName, const TArray<FString>& FilterList)
{
	const FString NameStr = InName.ToString();
	for (const FString& Filter : FilterList)
	{
		if (NameStr.Contains(Filter))
		{
			return true;
		}
	}
	return false;
}

// Returns true if the bone/curve name should be skipped based on Include/Exclude lists
bool ShouldSkipName(const FName& InName, const TArray<FString>& IncludeNames, const TArray<FString>& ExcludeNames)
{
	if (MatchesNameFilter(InName, ExcludeNames))
	{
		return true;
	}
	if (IncludeNames.Num() > 0 && !MatchesNameFilter(InName, IncludeNames))
	{
		return true;
	}
	return false;
}

} // anonymous namespace

namespace UE::Sequencer::AnimMixerBake
{

bool ExportMixerToAnimSequence(
	const TSharedPtr<ISequencer>& InSequencer,
	UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions,
	USkeletalMeshComponent* SkelMeshComp, UMovieSceneAnimationMixerTrack* MixerTrack,
	const UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter& Filter)
{
	using namespace UE::MovieScene::AnimMixerBakeEvaluation;

	USkeleton* Skeleton = AnimSequence ? AnimSequence->GetSkeleton() : nullptr;
	if (!AnimSequence || !MixerTrack || !SkelMeshComp || !Skeleton || !ExportOptions || !InSequencer) { return false; }

	UMovieSceneEntitySystemLinker* Linker = InSequencer->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!Linker) { return false; }
	UE::MovieScene::FRootInstanceHandle RootInstanceHandle = InSequencer->GetEvaluationTemplate().GetRootInstanceHandle();

	// Determine frame range and sample rate from export options
	UMovieSceneSequence* FocusedSequence = InSequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = FocusedSequence ? FocusedSequence->GetMovieScene() : nullptr;
	if (!MovieScene) { return false; }
	const FFrameRate SampleRate = ExportOptions->bUseCustomFrameRate ? ExportOptions->CustomFrameRate : MovieScene->GetDisplayRate();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const bool bTransact = ExportOptions->bTransactRecording;

	TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	FFrameNumber StartTick = PlaybackRange.GetLowerBoundValue();
	FFrameNumber EndTick = PlaybackRange.GetUpperBoundValue();

	if (ExportOptions->bUseCustomTimeRange)
	{
		StartTick = FFrameRate::TransformTime(FFrameTime(ExportOptions->CustomStartFrame), ExportOptions->CustomDisplayRate, TickResolution).FloorToFrame();
		EndTick = FFrameRate::TransformTime(FFrameTime(ExportOptions->CustomEndFrame), ExportOptions->CustomDisplayRate, TickResolution).CeilToFrame();
		if (EndTick < StartTick) { EndTick = StartTick; }
	}

	const double RangeDurationSeconds = TickResolution.AsSeconds(FFrameTime(EndTick - StartTick));
	const int32 NumSamples = FMath::Max(1, FMath::RoundToInt32(RangeDurationSeconds * SampleRate.AsDecimal()) + 1);
	const FFrameTime FrameStep = FFrameRate::TransformTime(FFrameTime(1), SampleRate, TickResolution);

	// Batch-evaluate the mixer track (Filter may be empty for whole-track, or populated for per-layer)
	TArray<FBakeResult> BakeResults = EvaluateRange(Linker, RootInstanceHandle, MixerTrack, FFrameTime(StartTick), FrameStep, NumSamples, Filter);

	if (BakeResults.Num() == 0) { return false; }

	// Find first valid result for skeleton/bone info
	int32 FirstValidIdx = INDEX_NONE;
	for (int32 i = 0; i < BakeResults.Num(); ++i)
	{
		if (BakeResults[i].IsValid()) { FirstValidIdx = i; break; }
	}
	if (FirstValidIdx == INDEX_NONE) { return false; }

	const FBakeResult& FirstResult = BakeResults[FirstValidIdx];
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const int32 NumSkeletonBones = RefSkeleton.GetNum();

	// Build skeleton bone -> LOD bone reverse map
	const TArrayView<const FBoneIndexType> LODToSkeletonMap = FirstResult.Pose.GetLODBoneIndexToSkeletonBoneIndexMap();
	TArray<int32> SkeletonToLODMap;
	SkeletonToLODMap.Init(INDEX_NONE, NumSkeletonBones);
	for (int32 LODIdx = 0; LODIdx < LODToSkeletonMap.Num(); ++LODIdx)
	{
		const int32 SkelIdx = LODToSkeletonMap[LODIdx];
		if (SkelIdx >= 0 && SkelIdx < NumSkeletonBones)
		{
			SkeletonToLODMap[SkelIdx] = LODIdx;
		}
	}

	// Flip bEnableRootMotion when the mixer produced motion. Anim-space ignores any
	// component-level transform on the binding so we don't claim root motion that was
	// really the actor moving.
	bool bHasRootMotion = false;
	if (!Filter.bSkipRootMotionConversion)
	{
		for (const FBakeResult& Result : BakeResults)
		{
			if (Result.IsValid() && !Result.AnimationSpaceRootMotion.Equals(FTransform::Identity))
			{
				bHasRootMotion = true;
				break;
			}
		}
	}

	// Interpolation settings from export options
	const ERichCurveInterpMode CurveInterpMode = ExportOptions->CurveInterpolation;
	const ERichCurveTangentMode CurveTangentMode =
		(CurveInterpMode == ERichCurveInterpMode::RCIM_Constant || CurveInterpMode == ERichCurveInterpMode::RCIM_None)
		? ERichCurveTangentMode::RCTM_None : ERichCurveTangentMode::RCTM_Auto;

	// Set up the animation data model
	IAnimationDataController& Controller = AnimSequence->GetController();
	Controller.OpenBracket(LOCTEXT("BakeMixerToAnimSeq", "Bake Mixer to Animation Sequence"), bTransact);
	Controller.InitializeModel();

	AnimSequence->Interpolation = ExportOptions->Interpolation;
	Controller.SetFrameRate(SampleRate, bTransact);
	Controller.SetNumberOfFrames(FFrameNumber(FMath::Max(0, NumSamples - 1)), bTransact);

	// -- Bone tracks --
	if (ExportOptions->bExportTransforms)
	{
		for (int32 BoneIdx = 0; BoneIdx < NumSkeletonBones; ++BoneIdx)
		{
			Controller.AddBoneCurve(RefSkeleton.GetBoneName(BoneIdx), bTransact);
		}

		for (int32 BoneIdx = 0; BoneIdx < NumSkeletonBones; ++BoneIdx)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIdx);
			const int32 LODIdx = SkeletonToLODMap[BoneIdx];
			const bool bShouldSkip = ShouldSkipName(BoneName, ExportOptions->IncludeAnimationNames, ExportOptions->ExcludeAnimationNames);

			TArray<FVector3f> PosKeys;
			TArray<FQuat4f> RotKeys;
			TArray<FVector3f> ScaleKeys;

			if (bShouldSkip)
			{
				FTransform RefTransform = (BoneIdx < RefSkeleton.GetRawRefBonePose().Num()) ? RefSkeleton.GetRawRefBonePose()[BoneIdx] : FTransform::Identity;
				PosKeys.Add(FVector3f(RefTransform.GetTranslation()));
				RotKeys.Add(FQuat4f(RefTransform.GetRotation()));
				ScaleKeys.Add(FVector3f(RefTransform.GetScale3D()));
			}
			else
			{
				PosKeys.Reserve(NumSamples);
				RotKeys.Reserve(NumSamples);
				ScaleKeys.Reserve(NumSamples);

				for (const FBakeResult& Result : BakeResults)
				{
					FTransform LocalTransform = FTransform::Identity;
					if (Result.IsValid() && LODIdx != INDEX_NONE && LODIdx < Result.Pose.LocalTransformsView.Num())
					{
						LocalTransform = Result.Pose.LocalTransformsView[LODIdx];
					}
					else if (BoneIdx < RefSkeleton.GetRawRefBonePose().Num())
					{
						LocalTransform = RefSkeleton.GetRawRefBonePose()[BoneIdx];
					}

					// Composite anim-space root motion onto bone 0. World-space would fold in
					// the binding's component transform and double-apply on playback.
					if (BoneIdx == 0
						&& !Filter.bSkipRootMotionConversion
						&& Result.IsValid()
						&& !Result.AnimationSpaceRootMotion.Equals(FTransform::Identity))
					{
						LocalTransform = LocalTransform * Result.AnimationSpaceRootMotion;
					}

					PosKeys.Add(FVector3f(LocalTransform.GetTranslation()));
					RotKeys.Add(FQuat4f(LocalTransform.GetRotation()));
					ScaleKeys.Add(FVector3f(LocalTransform.GetScale3D()));
				}
			}

			Controller.SetBoneTrackKeys(BoneName, PosKeys, RotKeys, ScaleKeys, bTransact);
		}
	}

	// -- Curves (morph targets, material curves, attribute curves) --
	{
		const bool bRecordMorphTargets = ExportOptions->bExportMorphTargets;
		const bool bRecordMaterialCurves = ExportOptions->bExportMaterialCurves;
		const bool bRecordAttributeCurves = ExportOptions->bExportAttributeCurves;
		const bool bSkipZeroValue = ExportOptions->bSkipCurvesWithZeroValue;

		TMap<FName, UE::Anim::ECurveElementFlags> FilteredCurves;
		for (const FBakeResult& Result : BakeResults)
		{
			if (!Result.IsValid()) { continue; }
			Result.Curves.ForEachElement([&](const UE::Anim::FCurveElement& Element)
			{
				if (FilteredCurves.Contains(Element.Name)) { return; }

				const bool bIsMorphTarget = EnumHasAnyFlags(Element.Flags, UE::Anim::ECurveElementFlags::MorphTarget);
				const bool bIsMaterial = EnumHasAnyFlags(Element.Flags, UE::Anim::ECurveElementFlags::Material);
				const bool bIsAttribute = !bIsMorphTarget && !bIsMaterial;

				if ((bIsMorphTarget && !bRecordMorphTargets) ||
					(bIsMaterial && !bRecordMaterialCurves) ||
					(bIsAttribute && !bRecordAttributeCurves))
				{
					return;
				}

				if (ShouldSkipName(Element.Name, ExportOptions->IncludeAnimationNames, ExportOptions->ExcludeAnimationNames))
				{
					return;
				}

				FilteredCurves.Add(Element.Name, Element.Flags);
			});
		}

		for (const auto& Pair : FilteredCurves)
		{
			const FName CurveName = Pair.Key;
			const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
			Controller.AddCurve(CurveId, AACF_DefaultCurve, bTransact);

			TArray<FRichCurveKey> Keys;
			Keys.Reserve(BakeResults.Num());
			bool bHasNonZeroValue = false;

			for (int32 FrameIdx = 0; FrameIdx < BakeResults.Num(); ++FrameIdx)
			{
				float Value = 0.0f;
				if (BakeResults[FrameIdx].IsValid())
				{
					BakeResults[FrameIdx].Curves.ForEachElement([&CurveName, &Value](const UE::Anim::FCurveElement& Element)
					{
						if (Element.Name == CurveName) { Value = Element.Value; }
					});
				}

				if (!FMath::IsNearlyZero(Value, UE_KINDA_SMALL_NUMBER))
				{
					bHasNonZeroValue = true;
				}

				FRichCurveKey Key(SampleRate.AsSeconds(FFrameTime(FrameIdx)), Value);
				Key.InterpMode = CurveInterpMode;
				Key.TangentMode = CurveTangentMode;
				Keys.Add(Key);
			}

			if (bSkipZeroValue && !bHasNonZeroValue)
			{
				Controller.RemoveCurve(CurveId, bTransact);
			}
			else
			{
				Controller.SetCurveKeys(CurveId, Keys, bTransact);
			}
		}
	}

	// Finalize
	Controller.NotifyPopulated();
	Controller.CloseBracket(bTransact);

	// If the mixer produced root motion and we composited it onto the root bone,
	// enable root motion extraction on the asset. Otherwise tracks that reference
	// this anim sequence won't extract the baked motion by default.
	if (bHasRootMotion)
	{
		AnimSequence->bEnableRootMotion = true;
	}

	if (ExportOptions->bSetRetargetSourceAsset && SkelMeshComp->GetSkeletalMeshAsset())
	{
		AnimSequence->RetargetSource = Skeleton->GetRetargetSourceForMesh(SkelMeshComp->GetSkeletalMeshAsset());
		if (AnimSequence->RetargetSource == NAME_None)
		{
			AnimSequence->SetRetargetSourceAsset(SkelMeshComp->GetSkeletalMeshAsset());
			AnimSequence->UpdateRetargetSourceAssetData();
		}
	}

	AnimSequence->PostEditChange();
	AnimSequence->MarkPackageDirty();

	return AnimSequence->GetDataModel()->HasBeenPopulated();
}

void BuildBakeMenuSection(
	FMenuBuilder& MenuBuilder,
	const TSharedPtr<ISequencer>& Sequencer,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	USkeletalMeshComponent* SkelMeshComp,
	USkeleton* Skeleton,
	FGuid ObjectBinding,
	UObject* BoundObject,
	bool& bFilterAssetBySkeleton,
	const UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter& Filter,
	TFunction<UMovieSceneTrack*(UMovieScene*, TSubclassOf<UMovieSceneTrack>)> ChildTrackFactory,
	TFunction<void(UMovieSceneTrack*, UObject*, bool)> OnComplete,
	const FText& EntryLabelVerb,
	FCanExecuteAction CanExecuteOverride,
	const FText& DisabledTooltipOverride)
{
	if (!Sequencer || !MixerTrack || !SkelMeshComp || !Skeleton) { return; }

	TSharedPtr<ISequencer> SequencerShared = Sequencer;

	// Resolve the verb used in leaf entry labels.
	const FText ResolvedVerb = EntryLabelVerb.IsEmpty()
		? LOCTEXT("DefaultBakeVerb", "Bake")
		: EntryLabelVerb;

	// Evaluate the override predicate once at build time so we can choose the layout.
	// Contiguity (the main use case for this override) is known at menu-build time
	// and doesn't change while the menu is open.
	const bool bEnabled = !CanExecuteOverride.IsBound() || CanExecuteOverride.Execute();

	// Tooltips — base, plus optional override when disabled.
	const FText AnimSeqEnabledTooltip = FText::Format(
		LOCTEXT("BakeAnimSequenceTooltipFmt",
			"{0} to an Animation Sequence based upon the Sequencer Display Range and Display Frame Rate."),
		ResolvedVerb);
	const FText AnimSeqTooltip = (!bEnabled && !DisabledTooltipOverride.IsEmpty())
		? DisabledTooltipOverride
		: AnimSeqEnabledTooltip;

	// -- {Verb} to Anim Sequence --
	const TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakMixerTrack(MixerTrack);
	const TWeakObjectPtr<USkeletalMeshComponent> WeakSkelMeshComp(SkelMeshComp);
	const TWeakObjectPtr<USkeleton> WeakSkeleton(Skeleton);
	FUIAction AnimSeqAction(
		FExecuteAction::CreateLambda(
			[SequencerShared, WeakMixerTrack, WeakSkelMeshComp, WeakSkeleton, Filter]()
			{
				USkeletalMeshComponent* SkelMeshComp = WeakSkelMeshComp.Get();
				USkeleton* Skeleton = WeakSkeleton.Get();
				UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
				if (!SkelMeshComp || !MixerTrack) { return; }

				TArray<TSoftObjectPtr<UObject>> Skels;
				Skels.Add(SkelMeshComp->GetSkeletalMeshAsset()
					? TSoftObjectPtr<UObject>(SkelMeshComp->GetSkeletalMeshAsset())
					: TSoftObjectPtr<UObject>(Skeleton));

				AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>(
					Skels, FString("_Sequence"),
					FAnimAssetCreated::CreateLambda(
						[SequencerShared, WeakMixerTrack, WeakSkelMeshComp, Filter](const TArray<UObject*> NewAssets) -> bool
						{
							if (NewAssets.Num() == 0) { return false; }
							UAnimSequence* AnimSequence = Cast<UAnimSequence>(NewAssets[0]);
							if (!AnimSequence || !AnimSequence->GetSkeleton()) { return false; }

							UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
							USkeletalMeshComponent* SkelMeshComp = WeakSkelMeshComp.Get();
							if (!MixerTrack || !SkelMeshComp) { return false; }

							UAnimSeqExportOption* ExportOptions = NewObject<UAnimSeqExportOption>(GetTransientPackage(), NAME_None);
							ExportOptions->CustomDisplayRate = SequencerShared->GetFocusedDisplayRate();

							bool bResult = ExportMixerToAnimSequence(SequencerShared, AnimSequence, ExportOptions, SkelMeshComp, MixerTrack, Filter);

							if (bResult)
							{
								FNotificationInfo Info(FText::Format(
									LOCTEXT("BakeSuccess", "Successfully baked mixer to {0}"),
									FText::FromString(AnimSequence->GetName())));
								Info.ExpireDuration = 8.0f;
								Info.bUseLargeFont = false;
								Info.Hyperlink = FSimpleDelegate::CreateLambda([AnimSequence]()
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimSequence);
								});
								Info.HyperlinkText = FText::Format(
									LOCTEXT("OpenNewAnimSeqHyperlink", "Open {0}"),
									FText::FromString(AnimSequence->GetName()));
								FSlateNotificationManager::Get().AddNotification(Info);
								SequencerShared->RequestEvaluate();
							}

							ExportOptions->MarkAsGarbage();
							return bResult;
						}),
					false, true);
			}),
		CanExecuteOverride.IsBound() ? CanExecuteOverride : FCanExecuteAction());

	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("BakeToAnimSequenceFmt", "{0} to Anim Sequence"), ResolvedVerb),
		AnimSeqTooltip,
		FSlateIcon(),
		AnimSeqAction);

	// -- {Verb} to Control Rig (via modular feature provider, if available) --
	// Skip the CR submenu when the entry is disabled — a disabled submenu that opens an empty picker is confusing.
	if (bEnabled)
	{
		TArray<IMovieSceneAnimMixerBakeProvider*> BakeProviders =
			IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimMixerBakeProvider>(
				IMovieSceneAnimMixerBakeProvider::GetModularFeatureName());

		if (BakeProviders.Num() > 0)
		{
			FAnimMixerBakeMenuParams BakeParams;
			BakeParams.Sequencer = SequencerShared;
			BakeParams.SkelMeshComp = SkelMeshComp;
			BakeParams.Skeleton = Skeleton;
			BakeParams.ObjectBinding = ObjectBinding;
			BakeParams.BoundObject = BoundObject;
			BakeParams.bFilterAssetBySkeleton = &bFilterAssetBySkeleton;
			BakeParams.EntryLabelVerb = ResolvedVerb;
			BakeParams.PopulateAnimSequence.BindLambda(
				[SequencerShared, WeakMixerTrack, Filter](UAnimSequence* AnimSeq, UAnimSeqExportOption* Opts, USkeletalMeshComponent* Mesh) -> bool
				{
					UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
					if (!MixerTrack) { return false; }
					return ExportMixerToAnimSequence(SequencerShared, AnimSeq, Opts, Mesh, MixerTrack, Filter);
				});
			BakeParams.ChildTrackFactory = ChildTrackFactory;
			BakeParams.OnComplete = OnComplete;

			BakeProviders[0]->BuildBakeToControlRigMenuSection(MenuBuilder, BakeParams);
		}
	}
}

} // namespace UE::Sequencer::AnimMixerBake

#undef LOCTEXT_NAMESPACE
