// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerDecorationMenuProviders.h"

#include "AnimatedRange.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MovieSceneAnimationMaskDecoration.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneAnimMixerMaskSection.h"
#include "ScopedTransaction.h"
#include "SequencerSettings.h"
#include "Components/SkeletalMeshComponent.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Factories/Factory.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#define LOCTEXT_NAMESPACE "AnimmixerDecorationMenuProviders"

UClass* FMaskDecorationMenuProvider::GetHandledDecorationClass() const
{
	return UMovieSceneAnimationMaskDecoration::StaticClass();
}

void FMaskDecorationMenuProvider::PopulateAddDecorationMenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject,
	UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("MaskDecoration", "Mask"),
		LOCTEXT("MaskDecorationTooltip", "Add a mask from the selected asset."),
		FNewMenuDelegate::CreateRaw(this, &FMaskDecorationMenuProvider::PopulateMaskSubmenu, BoundObject, DecorationContainer, InSequencer),
		/*bInOpenSubMenuOnClick=*/ false,
		FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Masking")
	);
}

const USkeleton* FMaskDecorationMenuProvider::FindSkeletonForObject(const TObjectPtr<UObject>& Object)
{
	const USkeletalMeshComponent* SkeletalMesh = nullptr;

	// If the bound object is an actor, look for a mesh component.
	if (const AActor* Actor = Cast<AActor>(Object))
	{
		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		Actor->GetComponents(SkeletalMeshComponents);

		// If there is only one mesh component on the actor, use that. If there are multiple, it's ambiguous which one to use, and the user should bind to components instead.
		// This will result in no filtering of masks, which can allow the user to bind an incompatible mask, and may result in unwanted behavior.
		// Choosing to filter based on the first mesh, regardless of if there are others, could result in the same behavior.
		// Not filtering at least makes the correct masks visible to all components.
		if (SkeletalMeshComponents.Num() == 1)
		{
			SkeletalMesh = SkeletalMeshComponents[0];
		}
	}
	// If the bound object is a component, use it. 
	else
	{
		SkeletalMesh = Cast<USkeletalMeshComponent>(Object);
	}

	return SkeletalMesh && SkeletalMesh->GetSkeletalMeshAsset() ? SkeletalMesh->GetSkeletalMeshAsset()->GetSkeleton() : nullptr;
}

void FMaskDecorationMenuProvider::OnCreateNewBlendMask(UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> WeakSequencer)
{
	UFactory* BlendMaskFactory = nullptr;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* CurrentClass = *It;
		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !CurrentClass->HasAnyClassFlags(CLASS_Abstract))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->CanCreateNew() && Factory->SupportedClass == UUAFBlendMask::StaticClass())
			{
				BlendMaskFactory = Factory;
				break;
			}
		}
	}

	if (!BlendMaskFactory)
	{
		return;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	UObject* NewAsset = AssetTools.CreateAssetWithDialog(UUAFBlendMask::StaticClass(), BlendMaskFactory);
	if (!NewAsset)
	{
		return;
	}

	// Save the new asset so it persists and is available in the asset registry for future picking.
	constexpr bool bOnlyDirty = false;
	UEditorLoadingAndSavingUtils::SavePackages({ NewAsset->GetPackage() }, bOnlyDirty);

	// Open the blend mask editor so the user can configure it immediately.
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);

	FAssetData AssetData(NewAsset);
	OnMaskAssetSelected(AssetData, DecorationContainer, WeakSequencer);
}

void FMaskDecorationMenuProvider::PopulateMaskSubmenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	UMovieSceneDecorationContainerObject* DecorationContainer,
	TWeakPtr<ISequencer> InSequencer
	)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CreateNewBlendMask", "Create New Blend Mask"),
		LOCTEXT("CreateNewBlendMaskTooltip", "Create a new blend mask asset."),
		FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Masking"),
		FUIAction(FExecuteAction::CreateRaw(this, &FMaskDecorationMenuProvider::OnCreateNewBlendMask, DecorationContainer, InSequencer))
	);

	MenuBuilder.AddSeparator();

	const USkeleton* Skeleton = FindSkeletonForObject(BoundObject);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FMaskDecorationMenuProvider::OnMaskAssetSelected, DecorationContainer, InSequencer);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.Filter.ClassPaths.Add(UUAFBlendMask::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnShouldFilterAsset.BindRaw(this, &FMaskDecorationMenuProvider::FilterBlendMasks, Skeleton);
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData());
	}

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = InSequencer.IsValid() ? InSequencer.Pin()->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = InSequencer.IsValid() ? InSequencer.Pin()->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	const TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}

void FMaskDecorationMenuProvider::OnMaskAssetSelected(const FAssetData& AssetData, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> WeakSequencer)
{
	if (!DecorationContainer)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("AddDecorationSection", "Add {0}"), UMovieSceneAnimationMaskDecoration::StaticClass()->GetDisplayNameText()));

	UMovieSceneAnimationMaskDecoration* MaskDecoration = DecorationContainer->GetOrCreateDecoration<UMovieSceneAnimationMaskDecoration>();

	// The newly added section will start at the current sequence time, and by default end at the playback end.
	const FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();
	const FFrameNumber PlaybackEnd = UE::MovieScene::DiscreteExclusiveUpper(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());

	// Check to see if the playback end is a valid end time, and if not, use the view range of the sequence instead.
	FFrameNumber NewSectionRangeEnd = PlaybackEnd;
	if (PlaybackEnd <= CurrentTime.Time.FrameNumber)
	{
		const FAnimatedRange ViewRange = Sequencer->GetViewRange();
		const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		NewSectionRangeEnd = (ViewRange.GetUpperBoundValue() * TickResolution).FloorToFrame();
	}

	// Create the section, and modify the container object.
	DecorationContainer->Modify();
	UMovieSceneSection* NewSection = MaskDecoration->CreateNewSection();

	// Set the range. Masks don't support infinite ranges, so always set it's range.
	NewSection->SetRange(TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, NewSectionRangeEnd));

	// Add the section, this will fix up existing sections to avoid overlaps, and may change the range originally set above.
	MaskDecoration->AddSection(NewSection);

	// Set the blend mask to the selected asset
	UObject* SelectedMask = AssetData.GetAsset();
	if (UMovieSceneAnimMixerMaskSection* MaskSection = Cast<UMovieSceneAnimMixerMaskSection>(NewSection))
	{
		MaskSection->SetBlendMask(SelectedMask);
	}
	DecorationContainer->MarkAsChanged();
}

bool FMaskDecorationMenuProvider::FilterBlendMasks(const FAssetData& AssetData, const USkeleton* Skeleton)
{
	// Unlike animation sequences this function does not filter using USkeleton::IsCompatibleForEditor since that checks for compatible skeletons,
	// which is not relevant to blend masks.
	// Instead, this function will just check that their export text names match.

	// If there is no skeleton, don't filter blend masks. Default to showing all masks.
	if (!Skeleton)
	{
		return false;
	}
	
	// Get the value of the blend mask's skeleton tag
	const FString AssetSkeleton = AssetData.GetTagValueRef<FString>(TEXT("Skeleton"));

	// Get the export text of the skeleton associated with the owning bound object.
	TStringBuilder<128> SkeletonStringBuilder;
	FAssetData(Skeleton, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering).GetExportTextName(SkeletonStringBuilder);
	const TCHAR* SkeletonString = SkeletonStringBuilder.ToString();
	
	// Filter (do not show) the blend mask asset if the skeletons do not match.
	return  AssetSkeleton != SkeletonString;
}

#undef LOCTEXT_NAMESPACE