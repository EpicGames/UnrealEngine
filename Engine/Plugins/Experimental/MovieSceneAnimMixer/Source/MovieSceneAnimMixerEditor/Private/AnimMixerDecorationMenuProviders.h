// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "Decorations/IMovieSceneDecorationMenuProvider.h"

/**
 * Decoration menu provider for masks (UMovieSceneAnimationMaskDecoration).
 * Adds a "Mask" menu entry that allows the user to select a blend mask asset, which will create a mask section with that asset as its blend mask.
 */
class FMaskDecorationMenuProvider : public IMovieSceneDecorationMenuProvider
{
public:

	/** IMovieSceneDecorationMenuProvider **/
	virtual UClass* GetHandledDecorationClass() const override;
	virtual void PopulateAddDecorationMenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer) override;
	// Masks are high priority.
	virtual int32 GetDecorationMenuPriority() const override { return  0; };

private:

	const USkeleton* FindSkeletonForObject(const TObjectPtr<UObject>& Object);

	void PopulateMaskSubmenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer);

	void OnCreateNewBlendMask(UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> WeakSequencer);

	void OnMaskAssetSelected(const FAssetData& AssetData, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> Sequencer);

	bool FilterBlendMasks(const FAssetData& AssetData, const USkeleton* Skeleton);

};