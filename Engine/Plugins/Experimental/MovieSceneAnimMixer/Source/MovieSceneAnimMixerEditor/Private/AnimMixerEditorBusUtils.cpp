// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerEditorBusUtils.h"

#include "ISequencer.h"
#include "MovieSceneAnimBusSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"

namespace AnimMixerEditorBusUtils
{

// Collect mixer tracks from a movie scene whose binding resolves to TargetObject.
static void GatherFromMovieScene(
	UMovieScene* MovieScene,
	FMovieSceneSequenceID SequenceID,
	UObject* TargetObject,
	ISequencer& Sequencer,
	TArray<UMovieSceneAnimationMixerTrack*>& OutTracks)
{
	if (!MovieScene)
	{
		return;
	}

	const UMovieScene* ConstMovieScene = MovieScene;
	for (const FMovieSceneBinding& Binding : ConstMovieScene->GetBindings())
	{
		bool bMatchesTarget = false;
		TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer.FindBoundObjects(Binding.GetObjectGuid(), SequenceID);
		for (TWeakObjectPtr<> WeakObj : BoundObjects)
		{
			if (WeakObj.Get() == TargetObject)
			{
				bMatchesTarget = true;
				break;
			}
		}

		if (!bMatchesTarget)
		{
			continue;
		}

		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
			{
				OutTracks.AddUnique(MixerTrack);
			}
		}
	}
}

// Find the SequenceID for a movie scene in the compiled hierarchy.
// Returns MovieSceneSequenceID::Invalid if not found.
static FMovieSceneSequenceID FindSequenceIDForMovieScene(
	UMovieScene* TargetMovieScene,
	ISequencer& Sequencer)
{
	UMovieSceneSequence* RootSequence = Sequencer.GetRootMovieSceneSequence();
	if (RootSequence && RootSequence->GetMovieScene() == TargetMovieScene)
	{
		return MovieSceneSequenceID::Root;
	}

	FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer.GetEvaluationTemplate();
	UMovieSceneCompiledDataManager* CompiledDataManager = RootInstance.GetCompiledDataManager();
	if (!CompiledDataManager)
	{
		return MovieSceneSequenceID::Invalid;
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootInstance.GetCompiledDataID());
	if (!Hierarchy)
	{
		return MovieSceneSequenceID::Invalid;
	}

	for (const auto& Pair : Hierarchy->AllSubSequenceData())
	{
		UMovieSceneSequence* SubSequence = Pair.Value.GetSequence();
		if (SubSequence && SubSequence->GetMovieScene() == TargetMovieScene)
		{
			return Pair.Key;
		}
	}

	return MovieSceneSequenceID::Invalid;
}

TArray<UMovieSceneAnimationMixerTrack*> GatherMixerTracksForSameObject(
	UMovieSceneAnimationMixerTrack* Track, ISequencer& Sequencer)
{
	if (!Track)
	{
		return {};
	}

	UMovieScene* TrackMovieScene = Track->GetTypedOuter<UMovieScene>();
	if (!TrackMovieScene)
	{
		return {};
	}

	// Find the track's binding GUID
	FGuid BindingGuid;
	const UMovieScene* ConstMovieScene = TrackMovieScene;
	for (const FMovieSceneBinding& Binding : ConstMovieScene->GetBindings())
	{
		bool bFound = false;
		for (UMovieSceneTrack* BoundTrack : Binding.GetTracks())
		{
			if (BoundTrack == Track)
			{
				BindingGuid = Binding.GetObjectGuid();
				bFound = true;
				break;
			}
		}
		if (bFound)
		{
			break;
		}
	}

	if (!BindingGuid.IsValid())
	{
		return {};
	}

	// Find the SequenceID for the track's movie scene so we resolve
	// the binding in the correct sub-sequence context
	FMovieSceneSequenceID TrackSequenceID = FindSequenceIDForMovieScene(TrackMovieScene, Sequencer);
	if (TrackSequenceID == MovieSceneSequenceID::Invalid)
	{
		return {};
	}

	// Resolve the binding to the actual bound object
	TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer.FindBoundObjects(BindingGuid, TrackSequenceID);
	UObject* BoundObject = nullptr;
	for (TWeakObjectPtr<> WeakObj : BoundObjects)
	{
		BoundObject = WeakObj.Get();
		if (BoundObject)
		{
			break;
		}
	}

	if (!BoundObject)
	{
		return {};
	}

	// Now gather all mixer tracks across the full hierarchy that resolve to
	// the same bound object
	TArray<UMovieSceneAnimationMixerTrack*> Result;

	UMovieSceneSequence* RootSequence = Sequencer.GetRootMovieSceneSequence();
	if (RootSequence)
	{
		GatherFromMovieScene(RootSequence->GetMovieScene(), MovieSceneSequenceID::Root, BoundObject, Sequencer, Result);
	}

	FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer.GetEvaluationTemplate();
	UMovieSceneCompiledDataManager* CompiledDataManager = RootInstance.GetCompiledDataManager();
	if (CompiledDataManager)
	{
		const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootInstance.GetCompiledDataID());
		if (Hierarchy)
		{
			for (const auto& Pair : Hierarchy->AllSubSequenceData())
			{
				UMovieSceneSequence* SubSequence = Pair.Value.GetSequence();
				if (SubSequence)
				{
					GatherFromMovieScene(SubSequence->GetMovieScene(), Pair.Key, BoundObject, Sequencer, Result);
				}
			}
		}
	}

	return Result;
}

TArray<FName> GatherBusNamesFromTracks(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks)
{
	TSet<FName> BusNames;

	for (UMovieSceneAnimationMixerTrack* Track : MixerTracks)
	{
		if (!Track)
		{
			continue;
		}

		if (const FMovieSceneAnimBusTarget* BusTarget = Track->MixedAnimationTarget.GetPtr<FMovieSceneAnimBusTarget>())
		{
			if (!BusTarget->BusName.IsNone())
			{
				BusNames.Add(BusTarget->BusName);
			}
		}

		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (UMovieSceneAnimBusSection* BusSection = Cast<UMovieSceneAnimBusSection>(Section))
			{
				if (!BusSection->BusName.IsNone())
				{
					BusNames.Add(BusSection->BusName);
				}
			}
		}
	}

	TArray<FName> Result = BusNames.Array();
	Result.Sort(FNameLexicalLess());
	return Result;
}

} // namespace AnimMixerEditorBusUtils
