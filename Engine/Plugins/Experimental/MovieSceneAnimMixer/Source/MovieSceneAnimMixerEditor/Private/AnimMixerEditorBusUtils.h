// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISequencer;
class UMovieSceneAnimationMixerTrack;

// Editor-side bus utilities that require ISequencer for binding resolution.
namespace AnimMixerEditorBusUtils
{
	// Gather all mixer tracks across the sequence hierarchy that are bound to
	// the same object as the given track. Resolves the track's binding through
	// the compiled hierarchy (not just the focused sequence) so this works
	// correctly for tracks in any sub-sequence.
	TArray<UMovieSceneAnimationMixerTrack*> GatherMixerTracksForSameObject(
		UMovieSceneAnimationMixerTrack* Track, ISequencer& Sequencer);

	// Gather the bus names (targets and sections) referenced by the given mixer
	// tracks, sorted. Buses are scoped per bound object, so pass the tracks for a
	// single character (see GatherMixerTracksForSameObject) to get the buses
	// relevant to that character rather than every bus in the sequence.
	TArray<FName> GatherBusNamesFromTracks(TArrayView<UMovieSceneAnimationMixerTrack* const> MixerTracks);
}
