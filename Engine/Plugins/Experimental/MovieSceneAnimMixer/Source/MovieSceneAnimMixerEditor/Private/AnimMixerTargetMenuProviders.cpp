// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerTargetMenuProviders.h"

#include "AnimGraphNode_SequencerMixerTarget.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamType.h"
#include "Param/ParamUtils.h"
#include "Variables/SVariablePickerCombo.h"
#include "Systems/MovieSceneAnimBlueprintTargetSystem.h"
#include "Systems/MovieSceneAnimInstanceTargetSystem.h"
#include "Systems/MovieSceneAnimNextTargetSystem.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/VariablePickerArgs.h"

#define LOCTEXT_NAMESPACE "AnimMixerTargetMenuProviders"

//------------------------------------------------------------------------------
// FAutomaticTargetMenuProvider
//------------------------------------------------------------------------------

UScriptStruct* FAutomaticTargetMenuProvider::GetHandledTargetStructType() const
{
	return FMovieSceneMixedAnimationTarget::StaticStruct();
}

void FAutomaticTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutomaticTarget", "Automatic"),
		LOCTEXT("AutomaticTargetTooltip", "Automatically determine the animation target based on the object's configuration."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([OnTargetSelected]()
			{
				OnTargetSelected(TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make());
			})
		)
	);
}

//------------------------------------------------------------------------------
// FAnimInstanceTargetMenuProvider
//------------------------------------------------------------------------------

UScriptStruct* FAnimInstanceTargetMenuProvider::GetHandledTargetStructType() const
{
	return FMovieSceneAnimInstanceTarget::StaticStruct();
}

void FAnimInstanceTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomAnimInstanceTarget", "Custom Anim Instance"),
		LOCTEXT("CustomAnimInstanceTargetTooltip", "Use a custom anim instance on the skeletal mesh component."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([OnTargetSelected]()
			{
				OnTargetSelected(TInstancedStruct<FMovieSceneAnimInstanceTarget>::Make());
			})
		)
	);
}

//------------------------------------------------------------------------------
// FAnimBlueprintTargetMenuProvider
//------------------------------------------------------------------------------

UScriptStruct* FAnimBlueprintTargetMenuProvider::GetHandledTargetStructType() const
{
	return FMovieSceneAnimBlueprintTarget::StaticStruct();
}

void FAnimBlueprintTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("AnimBlueprintTarget", "Anim Blueprint Target"),
		LOCTEXT("AnimBlueprintTargetTooltip", "Target a specific Sequencer Mixer Target node in an Animation Blueprint."),
		FNewMenuDelegate::CreateRaw(this, &FAnimBlueprintTargetMenuProvider::PopulateAnimBlueprintTargetSubmenu, BoundObject, OnTargetSelected)
	);
}

void FAnimBlueprintTargetMenuProvider::PopulateAnimBlueprintTargetSubmenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	// Try to find SequencerMixerTarget nodes from the bound object's anim blueprint
	TArray<FName> TargetNames;

	// Get the skeletal mesh component from the bound object
	USkeletalMeshComponent* SkeletalMeshComponent = nullptr;
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>();
	}
	else if (USkeletalMeshComponent* SMC = Cast<USkeletalMeshComponent>(BoundObject))
	{
		SkeletalMeshComponent = SMC;
	}

	if (SkeletalMeshComponent)
	{
		// Get the configured anim blueprint class
		if (UClass* AnimClass = SkeletalMeshComponent->GetAnimClass())
		{
			if (UAnimBlueprintGeneratedClass* AnimBPClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass))
			{
				// Find all SequencerMixerTarget nodes in the anim blueprint
				// Use CDO to get default property values since we may not have a runtime instance
				UAnimInstance* AnimCDO = AnimBPClass->GetDefaultObject<UAnimInstance>();
				for (const FStructProperty* NodeProperty : AnimBPClass->AnimNodeProperties)
				{
					if (NodeProperty && NodeProperty->Struct == FAnimNode_SequencerMixerTarget::StaticStruct())
					{
						// Get the node and its target name from the CDO
						const FAnimNode_SequencerMixerTarget* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_SequencerMixerTarget>(AnimCDO);
						if (Node && !Node->TargetName.IsNone())
						{
							TargetNames.AddUnique(Node->TargetName);
						}
					}
				}
			}
		}
	}

	// Always add the default target name
	TargetNames.AddUnique(FAnimNode_SequencerMixerTarget::DefaultTargetName);

	// Add menu entries for each discovered target
	for (const FName& TargetName : TargetNames)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromName(TargetName),
			FText::Format(LOCTEXT("AnimBlueprintTargetEntryTooltip", "Target the '{0}' Sequencer Mixer Target node."), FText::FromName(TargetName)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([OnTargetSelected, TargetName]()
				{
					OnTargetSelected(TInstancedStruct<FMovieSceneAnimBlueprintTarget>::Make(TargetName));
				})
			)
		);
	}
}

//------------------------------------------------------------------------------
// FAnimNextInjectionTargetMenuProvider
//------------------------------------------------------------------------------

UScriptStruct* FAnimNextInjectionTargetMenuProvider::GetHandledTargetStructType() const
{
	return FMovieSceneAnimNextInjectionTarget::StaticStruct();
}

void FAnimNextInjectionTargetMenuProvider::PopulateTargetMenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("UAFModuleInjectionTarget", "UAF Module Injection"),
		LOCTEXT("UAFModuleInjectionTargetTooltip", "Target an injection site on an Unreal Animation Framework module."),
		FNewMenuDelegate::CreateRaw(this, &FAnimNextInjectionTargetMenuProvider::PopulateAnimNextTargetSubmenu, BoundObject, OnTargetSelected)
	);
}

void FAnimNextInjectionTargetMenuProvider::PopulateAnimNextTargetSubmenu(
	FMenuBuilder& MenuBuilder,
	TObjectPtr<UObject> BoundObject,
	TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected)
{
	using namespace UE::UAF::Editor;

	// Always add a default entry that uses automatic injection site discovery
	MenuBuilder.AddMenuEntry(
		LOCTEXT("UAFDefaultInjectionSite", "Default"),
		LOCTEXT("UAFDefaultInjectionSiteTooltip", "Use the default injection site on the actor's module."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([OnTargetSelected]()
			{
				OnTargetSelected(TInstancedStruct<FMovieSceneAnimNextInjectionTarget>::Make());
			})
		)
	);

	// Add an embedded variable picker for selecting a specific injection site
	FVariablePickerArgs PickerArgs;
	PickerArgs.OnVariablePicked = FOnVariablePicked::CreateLambda([OnTargetSelected](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
	{
		OnTargetSelected(TInstancedStruct<FMovieSceneAnimNextInjectionTarget>::Make(InVariableReference));
	});

	// Filter to only show FAnimNextAnimGraph types
	FAnimNextParamType FilterType = FAnimNextParamType::FromString(TEXT("FAnimNextAnimGraph"));
	PickerArgs.OnFilterVariableType = FOnFilterVariableType::CreateLambda([FilterType](const FAnimNextParamType& InParamType) -> EFilterVariableResult
	{
		if (FilterType.IsValid())
		{
			if (!UE::UAF::FParamUtils::GetCompatibility(FilterType, InParamType).IsCompatibleWithDataLoss())
			{
				return EFilterVariableResult::Exclude;
			}
		}
		return EFilterVariableResult::Include;
	});

	MenuBuilder.AddWidget(
		SNew(SVariablePickerCombo)
		.PickerArgs(PickerArgs)
		.VariableName(LOCTEXT("SelectInjectionSite", "Select Injection Site...")),
		FText(),
		/*bNoIndent=*/ true
	);
}

#undef LOCTEXT_NAMESPACE
