// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneAnimMixerTargetMenuProvider.h"

/**
 * Target menu provider for the Automatic target type (base FMovieSceneMixedAnimationTarget).
 * Adds a simple "Automatic" menu entry.
 */
class FAutomaticTargetMenuProvider : public IMovieSceneAnimMixerTargetMenuProvider
{
public:
	virtual UScriptStruct* GetHandledTargetStructType() const override;
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) override;
	virtual int32 GetTargetMenuPriority() const override { return 0; }
};

/**
 * Target menu provider for the Custom Anim Instance target type.
 * Adds a simple "Custom Anim Instance" menu entry.
 */
class FAnimInstanceTargetMenuProvider : public IMovieSceneAnimMixerTargetMenuProvider
{
public:
	virtual UScriptStruct* GetHandledTargetStructType() const override;
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) override;
	virtual int32 GetTargetMenuPriority() const override { return 10; }
};

/**
 * Target menu provider for the Anim Blueprint Target type.
 * Queries the bound object's anim instance for SequencerMixerTarget nodes and
 * creates a submenu with discovered target names, plus the default name.
 */
class FAnimBlueprintTargetMenuProvider : public IMovieSceneAnimMixerTargetMenuProvider
{
public:
	virtual UScriptStruct* GetHandledTargetStructType() const override;
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) override;
	virtual int32 GetTargetMenuPriority() const override { return 20; }

private:
	void PopulateAnimBlueprintTargetSubmenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected);
};

/**
 * Target menu provider for the UAF Module Injection target type.
 * Queries the bound object's AnimNextComponent for injection sites and
 * creates a submenu with discovered injection sites, plus the default one.
 */
class FAnimNextInjectionTargetMenuProvider : public IMovieSceneAnimMixerTargetMenuProvider
{
public:
	virtual UScriptStruct* GetHandledTargetStructType() const override;
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) override;
	virtual int32 GetTargetMenuPriority() const override { return 30; }

private:
	void PopulateAnimNextTargetSubmenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected);
};
