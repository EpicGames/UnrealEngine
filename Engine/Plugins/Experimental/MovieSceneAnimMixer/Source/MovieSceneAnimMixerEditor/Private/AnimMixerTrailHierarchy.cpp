// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMixerTrailHierarchy.h"

#include "AnimMixerBakeEvaluation.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MixerRootMotionTrail.h"
#include "MixerTrailKeyTool.h"
#include "MovieScene.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolHelpers.h"
#include "Tools/MotionTrailOptions.h"

namespace UE::Sequencer
{

FAnimMixerTrailHierarchy::FAnimMixerTrailHierarchy(TWeakPtr<ISequencer> InWeakSequencer)
	: FTrailHierarchy()
	, WeakSequencer(InWeakSequencer)
	, HierarchyRenderer(MakeUnique<UE::SequencerAnimTools::FTrailHierarchyRenderer>(this, UMotionTrailToolOptions::GetTrailOptions()))
{
	TrailCategory = ETrailCategory::AnimMixer;
}

void FAnimMixerTrailHierarchy::Initialize()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
	{
		OnViewOptionsChangedHandle = Settings->OnDisplayPropertyChanged.AddLambda([this](FName PropertyName)
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, EvalsPerFrame) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, MotionTrailRange) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, FramesBefore) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, FramesAfter))
			{
				InvalidateAllTrails();
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TrailStyle) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DefaultColor) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TimePreColor) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TimePostColor) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DashPreColor) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DashPostColor))
			{
				for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
				{
					GuidTrailPair.Value->ClearCachedData();
				}
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowSelectedTrails))
			{
				UMotionTrailToolOptions* TrailSettings = UMotionTrailToolOptions::GetTrailOptions();
				if (TrailSettings && !TrailSettings->bShowSelectedTrails)
				{
					// Switching off: make all trails always visible
					for (const TPair<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>, FGuid>& Pair : TrackedMixerTrails)
					{
						VisibilityManager.AlwaysVisible.Add(Pair.Value);
					}
				}
				else
				{
					// Switching on: clear always-visible and use selection-driven visibility
					for (const TPair<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>, FGuid>& Pair : TrackedMixerTrails)
					{
						VisibilityManager.AlwaysVisible.Remove(Pair.Value);
					}
					OnSelectionChanged();
				}
			}
		});
	}

	OnSelectionChangedObjectGuidsHandle = Sequencer->GetSelectionChangedObjectGuids().AddLambda([this](TArray<FGuid>)
	{
		OnSelectionChanged();
	});

	OnSelectionChangedSectionsHandle = Sequencer->GetSelectionChangedSections().AddLambda([this](TArray<UMovieSceneSection*>)
	{
		OnSelectionChanged();
	});

	OnSelectionChangedTracksHandle = Sequencer->GetSelectionChangedTracks().AddLambda([this](TArray<UMovieSceneTrack*>)
	{
		OnSelectionChanged();
	});

	RegisterPinDelegates();

	RebuildTrailsFromSequencer();
}

void FAnimMixerTrailHierarchy::Destroy()
{
	if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
	{
		Settings->OnDisplayPropertyChanged.Remove(OnViewOptionsChangedHandle);
	}

	UnregisterPinDelegates();

	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->GetSelectionChangedObjectGuids().Remove(OnSelectionChangedObjectGuidsHandle);
		Sequencer->GetSelectionChangedSections().Remove(OnSelectionChangedSectionsHandle);
		Sequencer->GetSelectionChangedTracks().Remove(OnSelectionChangedTracksHandle);
	}

	TrackedMixerTrails.Reset();
	EvaluatingTrails.Reset();
	AllTrails.Reset();
	VisibilityManager.Reset();
}

UE::SequencerAnimTools::ITrailHierarchyRenderer* FAnimMixerTrailHierarchy::GetRenderer() const
{
	return HierarchyRenderer.Get();
}

FFrameNumber FAnimMixerTrailHierarchy::GetFramesPerFrame() const
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return FFrameRate::TransformTime(FFrameNumber(1), Sequencer->GetFocusedDisplayRate(),
			Sequencer->GetFocusedTickResolution()).RoundToFrame();
	}
	return FFrameNumber(0);
}

FFrameNumber FAnimMixerTrailHierarchy::GetFramesPerSegment() const
{
	return GetFramesPerFrame();
}

const UE::SequencerAnimTools::FCurrentFramesInfo* FAnimMixerTrailHierarchy::GetCurrentFramesInfo() const
{
	return &CurrentFramesInfo;
}

FFrameNumber FAnimMixerTrailHierarchy::GetLocalTime() const
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetLocalTime().Time.GetFrame();
	}
	return FFrameNumber(0);
}

bool FAnimMixerTrailHierarchy::CheckForChanges()
{
	bool bHasChange = false;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (FocusedSequence && FocusedSequence->GetMovieScene())
		{
			FGuid CurrentSignature = FocusedSequence->GetMovieScene()->GetSignature();
			bHasChange = LastMovieSceneGuid != CurrentSignature;
			LastMovieSceneGuid = CurrentSignature;

			if (bHasChange && !bSuppressChangeDetection)
			{
				// Only do a full rebuild if tracks were added/removed.
				// For data changes (key edits, section moves), just invalidate trails.
				SyncTrailsWithSequencer();
			}
		}
	}
	return bHasChange;
}

void FAnimMixerTrailHierarchy::InvalidateAllTrails()
{
	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		GuidTrailPair.Value->ForceEvaluateNextTick();
	}
}

void FAnimMixerTrailHierarchy::InvalidateTrailForTrack(UMovieSceneAnimationMixerTrack* MixerTrack)
{
	if (const FGuid* TrailGuid = TrackedMixerTrails.Find(MixerTrack))
	{
		if (const TUniquePtr<UE::SequencerAnimTools::FTrail>* TrailPtr = GetAllTrails().Find(*TrailGuid))
		{
			(*TrailPtr)->ForceEvaluateNextTick();
		}
	}
}

void FAnimMixerTrailHierarchy::SyncTrailsWithSequencer()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence || !FocusedSequence->GetMovieScene())
	{
		return;
	}

	const UMovieScene* MovieScene = FocusedSequence->GetMovieScene();

	// Count current mixer tracks with trail enabled
	TSet<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>> CurrentMixerTracks;
	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
			if (MixerTrack)
			{
				CurrentMixerTracks.Add(MixerTrack);
			}
		}
	}

	// Check if the set of tracked tracks changed (structural change)
	bool bStructuralChange = CurrentMixerTracks.Num() != TrackedMixerTrails.Num();
	if (!bStructuralChange)
	{
		for (const auto& Pair : TrackedMixerTrails)
		{
			if (!CurrentMixerTracks.Contains(Pair.Key))
			{
				bStructuralChange = true;
				break;
			}
		}
	}

	if (bStructuralChange)
	{
		RebuildTrailsFromSequencer();
	}
	else
	{
		// Data-only change: just invalidate existing trails for re-evaluation
		InvalidateAllTrails();
	}
}

bool FAnimMixerTrailHierarchy::IsTrailEvaluating(const FGuid& InTrailGuid, bool bIndirectlyOnly) const
{
	if (!EvaluatingTrails.Contains(InTrailGuid))
	{
		return false;
	}
	// When bIndirectlyOnly is true, the renderer asks whether to skip normal
	// rendering. Mixer trails are always directly evaluated (never indirect
	// dependencies), so return false — the renderer should keep drawing them
	// with partial data rather than skipping entirely.
	if (bIndirectlyOnly)
	{
		return false;
	}
	return true;
}

void FAnimMixerTrailHierarchy::CalculateEvalRangeArray()
{
	TicksPerSegment = GetFramesPerFrame();
	CurrentFramesInfo.SetViewRange(TickViewRange, TickEvalRange == TickViewRange);
	if (LastTicksPerSegment != TicksPerSegment || TickEvalRange != LastTickEvalRange)
	{
		LastTicksPerSegment = TicksPerSegment;
		LastTickEvalRange = TickEvalRange;
		for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
		{
			GuidTrailPair.Value->ForceEvaluateNextTick();
		}
		CurrentFramesInfo.SetUpFrameTimes(TickEvalRange, TicksPerSegment);
	}
}

void FAnimMixerTrailHierarchy::Update()
{
	// Check if any trail has a pending synchronous rebake from a key drag.
	// If so, suppress change detection for the entire Update so the normal
	// evaluation path doesn't invalidate trails (which would reset
	// CurrentFramesInfo and prevent drawing).
	bool bHasPendingRebake = false;
	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(GuidTrailPair.Value.Get());
		if (MixerTrail && MixerTrail->NeedsSynchronousRebake())
		{
			bHasPendingRebake = true;
			break;
		}
	}

	if (bHasPendingRebake)
	{
		bSuppressChangeDetection = true;
	}

	UpdateViewAndEvalRange();
	CalculateEvalRangeArray();

	// Base Update handles trail status (dead/stale/up-to-date) and triggers stale trails to evaluate
	FTrailHierarchy::Update();

	EvaluateAndSetTransforms();

	if (bHasPendingRebake)
	{
		bSuppressChangeDetection = false;
		SynchronousRebakeIfNeeded();
	}
}

void FAnimMixerTrailHierarchy::UpdateViewAndEvalRange()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer || !Sequencer->GetFocusedMovieSceneSequence() || !Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return;
	}

	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
	if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
	{
		if (Settings->MotionTrailRange == EMotionTrailRange::SelectionRange && Sequencer->GetSelectionRange().IsEmpty() == false)
		{
			TickEvalRange = Sequencer->GetSelectionRange();
			TickViewRange = TickEvalRange;
		}
		else
		{
			TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer->GetSubSequenceRange();
			TickEvalRange = OptionalRange.IsSet() ? OptionalRange.GetValue() : Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
			if (Settings->MotionTrailRange == EMotionTrailRange::SpecifiedRange)
			{
				FFrameTime SequenceTime = Sequencer->GetLocalTime().Time;
				const FFrameNumber TicksBefore = FFrameRate::TransformTime(FFrameNumber(Settings->FramesBefore), DisplayRate, TickResolution).FloorToFrame();
				const FFrameNumber TicksAfter = FFrameRate::TransformTime(FFrameNumber(Settings->FramesAfter), DisplayRate, TickResolution).FloorToFrame();
				TickViewRange = TRange<FFrameNumber>(SequenceTime.GetFrame() - TicksBefore, SequenceTime.GetFrame() + TicksAfter);
				if (TickViewRange.GetLowerBoundValue() < TickEvalRange.GetLowerBoundValue())
				{
					TickViewRange.SetLowerBoundValue(TickEvalRange.GetLowerBoundValue());
				}
				if (TickViewRange.GetUpperBoundValue() > TickEvalRange.GetUpperBoundValue())
				{
					TickViewRange.SetUpperBoundValue(TickEvalRange.GetUpperBoundValue());
				}
			}
			else
			{
				TickViewRange = TickEvalRange;
			}
		}
	}
}

void FAnimMixerTrailHierarchy::RebuildTrailsFromSequencer()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence || !FocusedSequence->GetMovieScene())
	{
		return;
	}

	const UMovieScene* MovieScene = FocusedSequence->GetMovieScene();

	TrackedMixerTrails.Reset();
	EvaluatingTrails.Reset();
	AllTrails.Reset();
	VisibilityManager.Reset();

	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
			if (!MixerTrack)
			{
				continue;
			}

			FGuid TrailGuid = FGuid::NewGuid();
			TUniquePtr<FMixerRootMotionTrail> NewTrail = MakeUnique<FMixerRootMotionTrail>(MixerTrack, WeakSequencer);
			NewTrail->SetHierarchy(this);
			NewTrail->ForceEvaluateNextTick();

			// Collect decorations from all sections on this mixer track
			TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> TrailDecorations;
			for (UMovieSceneSection* Section : MixerTrack->GetAllSections())
			{
				if (UMovieSceneRootMotionSettingsDecoration* Decoration = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
				{
					TrailDecorations.Add(Decoration);
				}
			}
			NewTrail->SetDecorations(MoveTemp(TrailDecorations));

			// Find the bound skeletal mesh component for world-to-local delta conversion
			TArrayView<TWeakObjectPtr<UObject>> BoundObjects = Sequencer->FindObjectsInCurrentSequence(Binding.GetObjectGuid());
			for (TWeakObjectPtr<UObject>& WeakObj : BoundObjects)
			{
				if (UObject* Obj = WeakObj.Get())
				{
					USkeletalMeshComponent* SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(Obj);
					if (SkelMeshComp)
					{
						NewTrail->SetBoundComponent(SkelMeshComp);
						break;
					}
				}
			}

			TrackedMixerTrails.Add(MixerTrack, TrailGuid);

			UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
			if (!Settings || !Settings->bShowSelectedTrails)
			{
				VisibilityManager.AlwaysVisible.Add(TrailGuid);
			}

			AddTrail(TrailGuid, MoveTemp(NewTrail));
		}
	}

	UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
	if (Settings && Settings->bShowSelectedTrails)
	{
		OnSelectionChanged();
	}
}

void FAnimMixerTrailHierarchy::EvaluateAndSetTransforms()
{
	if (EvaluatingTrails.Num() == 0)
	{
		// Check if any trails need to start evaluating (they were marked stale by base Update)
		for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
		{
			if (GuidTrailPair.Value->GetCacheState() == UE::SequencerAnimTools::ETrailCacheState::Stale)
			{
				EvaluatingTrails.Add(GuidTrailPair.Key);
				GuidTrailPair.Value->HasStartedEvaluating();

				// Resize the transform arrays for this trail
				FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(GuidTrailPair.Value.Get());
				if (MixerTrail)
				{
					TSharedPtr<UE::AIE::FArrayOfTransforms>& CalcTransforms = MixerTrail->GetCalculatedTransforms();
					if (CalcTransforms.IsValid())
					{
						CalcTransforms->Transforms.SetNum(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
					}
				}

				// Reset frame calculation state for fresh evaluation
				CurrentFramesInfo.Reset();
			}
		}

		if (EvaluatingTrails.Num() == 0)
		{
			return;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!SequencerPtr)
	{
		return;
	}

	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!Linker)
	{
		return;
	}
	UE::MovieScene::FRootInstanceHandle RootInstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();

	// Add must-have times: range endpoints and current scrub position
	TSet<FFrameNumber> MustHaveTimes;
	MustHaveTimes.Add(TickEvalRange.GetLowerBoundValue());
	MustHaveTimes.Add(TickEvalRange.GetUpperBoundValue());
	FFrameNumber CurrentFrame = SequencerPtr->GetLocalTime().Time.GetFrame();
	CurrentFramesInfo.AddMustHaveTimes(MustHaveTimes, CurrentFrame);

	bool bKeepCalculating = false;
	if (CurrentFramesInfo.CurrentFrameTimes.NumFrames > 0)
	{
		bKeepCalculating = CurrentFramesInfo.KeepCalculating();
	}

	const TArray<int32>& IndicesToCalculate = CurrentFramesInfo.IndicesToCalculate();

	// Bake-evaluate at each sample index for each evaluating trail
	if (IndicesToCalculate.Num() > 0)
	{
		for (const FGuid& TrailGuid : EvaluatingTrails)
		{
			const TUniquePtr<UE::SequencerAnimTools::FTrail>* TrailPtr = GetAllTrails().Find(TrailGuid);
			if (!TrailPtr || !TrailPtr->IsValid())
			{
				continue;
			}

			FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(TrailPtr->Get());
			UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrail->GetMixerTrack();
			if (!MixerTrack)
			{
				continue;
			}

			TSharedPtr<UE::AIE::FArrayOfTransforms>& CalcTransforms = MixerTrail->GetCalculatedTransforms();
			if (!CalcTransforms.IsValid())
			{
				continue;
			}

			// Ensure the transform array is big enough
			if (CalcTransforms->Num() < CurrentFramesInfo.CurrentFrameTimes.NumFrames)
			{
				CalcTransforms->Transforms.SetNum(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
			}

			for (int32 Index : IndicesToCalculate)
			{
				if (Index < 0 || Index >= CalcTransforms->Num())
				{
					continue;
				}

				FFrameNumber Frame = CurrentFramesInfo.CurrentFrameTimes.CalculateFrame(Index);

				using namespace UE::MovieScene::AnimMixerBakeEvaluation;
				FBakeResult Result = EvaluateAtTime(Linker, RootInstanceHandle, MixerTrack, FFrameTime(Frame));
				CalcTransforms->Transforms[Index] = Result.RootMotionTransform;
			}
		}
	}

	// Notify trails that evaluation finished for this bucket
	const TRange<FFrameNumber> Range = TickEvalRange;
	if (CurrentFramesInfo.TransformIndices.Num() > 0 || IndicesToCalculate.Num() > 0)
	{
		for (const FGuid& TrailGuid : EvaluatingTrails)
		{
			if (const TUniquePtr<UE::SequencerAnimTools::FTrail>* TrailPtr = GetAllTrails().Find(TrailGuid))
			{
				(*TrailPtr)->UpdateFinished(Range, IndicesToCalculate, !bKeepCalculating);
			}
		}
	}

	if (!bKeepCalculating)
	{
		EvaluatingTrails.Reset();
	}
}

void FAnimMixerTrailHierarchy::RequestRebakeAll()
{
	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(GuidTrailPair.Value.Get());
		if (MixerTrail)
		{
			MixerTrail->RequestSynchronousRebake();
		}
	}
}

void FAnimMixerTrailHierarchy::SynchronousRebakeIfNeeded()
{
	// Check if any trail requested a synchronous rebake (from ApplyDelta)
	bool bAnyNeedRebake = false;
	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(GuidTrailPair.Value.Get());
		if (MixerTrail && MixerTrail->NeedsSynchronousRebake())
		{
			bAnyNeedRebake = true;
			break;
		}
	}

	if (!bAnyNeedRebake)
	{
		return;
	}

	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!SequencerPtr)
	{
		return;
	}

	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!Linker)
	{
		return;
	}
	UE::MovieScene::FRootInstanceHandle RootInstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();

	// Suppress change detection during the bake so the entity system flushes
	// inside EvaluateAtTime don't trigger SyncTrailsWithSequencer → InvalidateAllTrails
	bSuppressChangeDetection = true;

	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		FMixerRootMotionTrail* MixerTrail = static_cast<FMixerRootMotionTrail*>(GuidTrailPair.Value.Get());
		if (!MixerTrail || !MixerTrail->NeedsSynchronousRebake())
		{
			continue;
		}

		MixerTrail->ClearSynchronousRebakeFlag();

		UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrail->GetMixerTrack();
		if (!MixerTrack)
		{
			continue;
		}

		TSharedPtr<UE::AIE::FArrayOfTransforms>& CalcTransforms = MixerTrail->GetCalculatedTransforms();
		if (!CalcTransforms.IsValid())
		{
			continue;
		}

		const UE::AIE::FFrameTimeByIndex& FrameTimes = CurrentFramesInfo.CurrentFrameTimes;
		if (FrameTimes.NumFrames <= 0)
		{
			continue;
		}

		// Force sequencer to evaluate so the entity system imports the
		// updated channel values before the bake reads them.
		SequencerPtr->ForceEvaluate();

		// Batch bake: one scope for all samples (N+1 flushes instead of 2N)
		using namespace UE::MovieScene::AnimMixerBakeEvaluation;
		TArray<FBakeResult> Results = EvaluateRange(
			Linker, RootInstanceHandle, MixerTrack,
			FFrameTime(FrameTimes.StartFrame), FFrameTime(FrameTimes.FrameStep),
			FrameTimes.NumFrames);

		CalcTransforms->Transforms.SetNum(FrameTimes.NumFrames);
		for (int32 i = 0; i < Results.Num(); ++i)
		{
			CalcTransforms->Transforms[i] = Results[i].RootMotionTransform;
		}

		// Copy to drawing array and refresh visual state
		TArray<int32> AllIndices;
		AllIndices.SetNum(FrameTimes.NumFrames);
		for (int32 i = 0; i < FrameTimes.NumFrames; ++i)
		{
			AllIndices[i] = i;
		}
		MixerTrail->UpdateFinished(TickEvalRange, AllIndices, true);
	}

	// Rebuild CurrentFrames and TransformIndices to cover all baked frames.
	// These are normally populated by the incremental evaluation path which
	// we suppressed. ReadyToDrawTrail uses them to map transforms to draw points.
	{
		const UE::AIE::FFrameTimeByIndex& FrameTimes = CurrentFramesInfo.CurrentFrameTimes;
		CurrentFramesInfo.TransformIndices.SetNum(FrameTimes.NumFrames);
		CurrentFramesInfo.CurrentFrames.SetNum(FrameTimes.NumFrames);
		for (int32 i = 0; i < FrameTimes.NumFrames; ++i)
		{
			CurrentFramesInfo.TransformIndices[i] = i;
			CurrentFramesInfo.CurrentFrames[i] = FrameTimes.CalculateFrame(i);
		}
		CurrentFramesInfo.SetViewRange(TickViewRange, TickEvalRange == TickViewRange);
	}

	bSuppressChangeDetection = false;

	// Remove any trails from the evaluating set since we just completed them
	for (const TPair<FGuid, TUniquePtr<UE::SequencerAnimTools::FTrail>>& GuidTrailPair : GetAllTrails())
	{
		EvaluatingTrails.Remove(GuidTrailPair.Key);
	}
}

bool FAnimMixerTrailHierarchy::IsHitByClick(HHitProxy* InHitProxy)
{
	if (FTrailHierarchy::IsHitByClick(InHitProxy))
	{
		return true;
	}
	if (InHitProxy && HitProxyCast<HMixerTrailKeyProxy>(InHitProxy))
	{
		return true;
	}
	// Claim the click when keys are selected so that clicking empty space
	// routes through HandleClick and deselects them
	if (IsAnythingSelected())
	{
		return true;
	}
	return false;
}

void FAnimMixerTrailHierarchy::OnSelectionChanged()
{
	UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
	if (!Settings || !Settings->bShowSelectedTrails)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	VisibilityManager.Selected.Reset();

	// Map selected sections to their parent mixer track
	TArray<UMovieSceneSection*> SelectedSections;
	Sequencer->GetSelectedSections(SelectedSections);
	for (UMovieSceneSection* Section : SelectedSections)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Section->GetTypedOuter<UMovieSceneAnimationMixerTrack>())
		{
			if (const FGuid* TrailGuid = TrackedMixerTrails.Find(MixerTrack))
			{
				VisibilityManager.Selected.Add(*TrailGuid);
			}
		}
	}

	// Map selected tracks to mixer tracks (direct cast or parent)
	TArray<UMovieSceneTrack*> SelectedTracks;
	Sequencer->GetSelectedTracks(SelectedTracks);
	for (UMovieSceneTrack* Track : SelectedTracks)
	{
		UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
		if (!MixerTrack)
		{
			MixerTrack = Track->GetTypedOuter<UMovieSceneAnimationMixerTrack>();
		}
		if (MixerTrack)
		{
			if (const FGuid* TrailGuid = TrackedMixerTrails.Find(MixerTrack))
			{
				VisibilityManager.Selected.Add(*TrailGuid);
			}
		}
	}

	// Map selected object binding GUIDs to mixer tracks.
	// This catches outliner nodes (decorations, channels, etc.) that don't
	// appear in the track or section selection but do select their binding.
	TArray<FGuid> SelectedBindingGuids;
	Sequencer->GetSelectedObjects(SelectedBindingGuids);
	if (SelectedBindingGuids.Num() > 0)
	{
		UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (FocusedSequence && FocusedSequence->GetMovieScene())
		{
			const UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
			for (const FGuid& BindingGuid : SelectedBindingGuids)
			{
				const FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingGuid);
				if (!Binding)
				{
					continue;
				}
				for (UMovieSceneTrack* Track : Binding->GetTracks())
				{
					if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
					{
						if (const FGuid* TrailGuid = TrackedMixerTrails.Find(MixerTrack))
						{
							VisibilityManager.Selected.Add(*TrailGuid);
						}
					}
				}
			}
		}
	}
}

void FAnimMixerTrailHierarchy::RegisterPinDelegates()
{
	if (UMotionTrailToolOptions* TrailOptions = UMotionTrailToolOptions::GetTrailOptions())
	{
		TrailOptions->OnPinSelection.AddRaw(this, &FAnimMixerTrailHierarchy::OnPinSelection);
		TrailOptions->OnUnPinSelection.AddRaw(this, &FAnimMixerTrailHierarchy::OnUnPinSelection);
		TrailOptions->OnDeleteAllPinned.AddRaw(this, &FAnimMixerTrailHierarchy::OnDeleteAllPinned);
	}
}

void FAnimMixerTrailHierarchy::UnregisterPinDelegates()
{
	if (UMotionTrailToolOptions* TrailOptions = UMotionTrailToolOptions::GetTrailOptions())
	{
		TrailOptions->OnPinSelection.RemoveAll(this);
		TrailOptions->OnUnPinSelection.RemoveAll(this);
		TrailOptions->OnDeleteAllPinned.RemoveAll(this);
	}
}

void FAnimMixerTrailHierarchy::OnPinSelection(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::AnimMixer))
	{
		return;
	}

	UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
	if (!Settings)
	{
		return;
	}

	for (const FGuid& Guid : VisibilityManager.Selected)
	{
		if (VisibilityManager.IsTrailAlwaysVisible(Guid))
		{
			continue;
		}

		if (Settings->GetNumPinned() >= Settings->MaxNumberPinned)
		{
			break;
		}

		VisibilityManager.SetTrailAlwaysVisible(Guid, true);

		if (const TUniquePtr<UE::SequencerAnimTools::FTrail>* TrailPtr = GetAllTrails().Find(Guid))
		{
			UMotionTrailToolOptions::FPinnedTrail PinnedTrail;
			PinnedTrail.TrailGuid = Guid;
			PinnedTrail.TrailName = (*TrailPtr)->GetName();
			PinnedTrail.TrailColor = Settings->DefaultColor;
			PinnedTrail.bHasOffset = false;
			PinnedTrail.Category = ETrailCategory::AnimMixer;
			Settings->AddPinned(PinnedTrail);
		}
	}
}

void FAnimMixerTrailHierarchy::OnUnPinSelection(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::AnimMixer))
	{
		return;
	}

	UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();

	for (const FGuid& Guid : VisibilityManager.Selected)
	{
		if (!VisibilityManager.IsTrailAlwaysVisible(Guid))
		{
			continue;
		}

		if (Settings)
		{
			int32 Index = Settings->GetIndexFromGuid(Guid);
			if (Index != INDEX_NONE)
			{
				Settings->DeletePinned(Index);
			}
		}

		VisibilityManager.SetTrailAlwaysVisible(Guid, false);
	}
}

void FAnimMixerTrailHierarchy::OnDeleteAllPinned(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::AnimMixer))
	{
		return;
	}

	TArray<FGuid> AlwaysVisibleTrails = VisibilityManager.AlwaysVisible.Array();
	for (const FGuid& Guid : AlwaysVisibleTrails)
	{
		// Only remove mixer trail pins (those in our TrackedMixerTrails map)
		bool bIsMixerTrail = false;
		for (const TPair<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>, FGuid>& Pair : TrackedMixerTrails)
		{
			if (Pair.Value == Guid)
			{
				bIsMixerTrail = true;
				break;
			}
		}
		if (bIsMixerTrail)
		{
			VisibilityManager.SetTrailAlwaysVisible(Guid, false);
		}
	}
}

} // namespace UE::Sequencer
