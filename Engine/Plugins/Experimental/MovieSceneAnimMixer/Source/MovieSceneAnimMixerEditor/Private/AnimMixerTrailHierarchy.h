// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrailHierarchy.h"
#include "ISequencer.h"

class UMovieSceneAnimationMixerTrack;

namespace UE::Sequencer
{

// Trail hierarchy for animation mixer root motion trajectory visualization.
// Manages FMixerRootMotionTrail instances and drives incremental bake evaluation
// to populate trajectory transforms one bucket per tick.
class FAnimMixerTrailHierarchy : public UE::SequencerAnimTools::FTrailHierarchy
{
public:
	FAnimMixerTrailHierarchy(TWeakPtr<ISequencer> InWeakSequencer);

	// FTrailHierarchy interface
	virtual void Initialize() override;
	virtual void Destroy() override;
	virtual UE::SequencerAnimTools::ITrailHierarchyRenderer* GetRenderer() const override;
	virtual FFrameNumber GetFramesPerFrame() const override;
	virtual FFrameNumber GetFramesPerSegment() const override;
	virtual const UE::SequencerAnimTools::FCurrentFramesInfo* GetCurrentFramesInfo() const override;
	virtual bool CheckForChanges() override;
	virtual void Update() override;
	virtual void CalculateEvalRangeArray() override;
	virtual bool IsTrailEvaluating(const FGuid& InTrailGuid, bool bIndirectlyOnly) const override;
	virtual bool IsHitByClick(HHitProxy* HitProxy) override;

	virtual FFrameNumber GetLocalTime() const override;
	TWeakPtr<ISequencer> GetSequencer() const { return WeakSequencer; }

	// Set during ApplyDelta to prevent CheckForChanges from invalidating
	// trails while a synchronous re-bake is in progress
	void SetSuppressChangeDetection(bool bSuppress) { bSuppressChangeDetection = bSuppress; }

	// Request a synchronous rebake of all trails on the next Update.
	// Called from the edit mode's InputDelta when root offsets change.
	void RequestRebakeAll();

	// Invalidate the cached trajectory data for all trails without rebuilding them.
	// Call on data changes (key edits, section moves) that don't add/remove tracks.
	void InvalidateAllTrails();

	// Invalidate the cached trajectory data for a specific mixer track's trail.
	void InvalidateTrailForTrack(UMovieSceneAnimationMixerTrack* MixerTrack);

private:
	void UpdateViewAndEvalRange();
	void RebuildTrailsFromSequencer();
	void SyncTrailsWithSequencer();
	void EvaluateAndSetTransforms();
	void SynchronousRebakeIfNeeded();
	void OnSelectionChanged();

	void RegisterPinDelegates();
	void UnregisterPinDelegates();
	void OnPinSelection(ETrailCategory Category);
	void OnUnPinSelection(ETrailCategory Category);
	void OnDeleteAllPinned(ETrailCategory Category);

	TWeakPtr<ISequencer> WeakSequencer;

	// Map from mixer track to the trail's GUID in AllTrails
	TMap<TWeakObjectPtr<UMovieSceneAnimationMixerTrack>, FGuid> TrackedMixerTrails;

	// Trails being incrementally evaluated
	TSet<FGuid> EvaluatingTrails;

	UE::SequencerAnimTools::FCurrentFramesInfo CurrentFramesInfo;
	TUniquePtr<UE::SequencerAnimTools::FTrailHierarchyRenderer> HierarchyRenderer;

	FGuid LastMovieSceneGuid;
	bool bSuppressChangeDetection = false;
	FDelegateHandle OnViewOptionsChangedHandle;
	FDelegateHandle OnSelectionChangedObjectGuidsHandle;
	FDelegateHandle OnSelectionChangedSectionsHandle;
	FDelegateHandle OnSelectionChangedTracksHandle;
};

} // namespace UE::Sequencer
