// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTransitionSectionInterface.h"

#include "MovieSceneAnimTransitionSectionBase.h"
#include "UObject/UObjectIterator.h"
#include "MovieSceneAnimCrossfadeTransitionSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "ISectionLayoutBuilder.h"
#include "SequencerSectionPainter.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "ScopedTransaction.h"
#include "Rendering/SlateRenderer.h"
#include "TimeToPixel.h"

#define LOCTEXT_NAMESPACE "AnimTransitionSectionInterface"

namespace UE::Sequencer
{

FAnimTransitionSectionInterface::FAnimTransitionSectionInterface(UMovieSceneSection& InSection, UMovieSceneTrack& InTrack, FGuid InObjectBinding)
	: SectionPtr(&InSection)
	, TrackPtr(&InTrack)
	, ObjectBindingId(InObjectBinding)
{
}

UMovieSceneSection* FAnimTransitionSectionInterface::GetSectionObject()
{
	return SectionPtr.Get();
}

const FSlateBrush* FAnimTransitionSectionInterface::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.Transition");
}

FText FAnimTransitionSectionInterface::GetSectionTitle() const
{
	// Empty for now- curves and icons show the transition- title just gets in the way
	return FText::GetEmpty();
}

FText FAnimTransitionSectionInterface::GetSectionToolTip() const
{
	if (UMovieSceneAnimTransitionSectionBase* TransitionSection = GetTransitionSection())
	{
		return TransitionSection->GetTransitionDisplayName();
	}
	return LOCTEXT("TransitionTooltip", "Transition");
}

UMovieSceneAnimTransitionSectionBase* FAnimTransitionSectionInterface::GetTransitionSection() const
{
	return Cast<UMovieSceneAnimTransitionSectionBase>(SectionPtr.Get());
}

int32 FAnimTransitionSectionInterface::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.LayerId;

	UMovieSceneAnimTransitionSectionBase* TransitionSection = GetTransitionSection();
	if (!TransitionSection)
	{
		return LayerId;
	}

	const TRange<FFrameNumber> SectionRange = TransitionSection->GetRange();
	if (SectionRange.IsEmpty() || !SectionRange.HasLowerBound() || !SectionRange.HasUpperBound())
	{
		return LayerId;
	}

	// Check if we're in "section outliner mode" (layer expanded, each section on its own row)
	// vs "overlay mode" (layer collapsed, sections overlapping on the same row).
	// In outliner mode, we paint a solid background since sections don't overlap.
	// In overlay mode, we use transparent fill to let underlying sections show through.
	const bool bIsInOutlinerMode = Painter.SectionModel && Painter.SectionModel->FindAncestorOfType<FSectionOutlinerModel>().IsValid();

	if (bIsInOutlinerMode)
	{
		// Paint teal background to distinguish transitions from regular animation sections
		// Teal suggests "connection/linking" and feels active (not disabled like gray would)
		const FLinearColor TealBackground(0.08f, 0.22f, 0.25f, 1.0f);
		LayerId = Painter.PaintSectionBackground(TealBackground);
	}

	// Use HeaderGeometry to only draw on the section header row, not expanded key areas
	const FVector2D HeaderSize = Painter.HeaderGeometry.GetLocalSize();
	const float CurveHeight = HeaderSize.Y - 2.0f;  // 1px padding top and bottom

	// Get the blend weight channel to sample the curve
	const FMovieSceneFloatChannel* BlendChannel = TransitionSection->GetBlendWeightChannel();

	// Get the time converter for pixel calculations
	const FTimeToPixel& TimeToPixel = Painter.GetTimeConverter();

	// Get the brush for the filled curve area
	const FSlateBrush* CurveBrush = FAppStyle::Get().GetBrush("Sequencer.Timeline.EaseInOut");
	FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*CurveBrush);
	const FSlateShaderResourceProxy* ResourceProxy = ResourceHandle.GetResourceProxy();
	FVector2f AtlasOffset = ResourceProxy ? ResourceProxy->StartUV : FVector2f(0.f, 0.f);
	FVector2f AtlasUVSize = ResourceProxy ? ResourceProxy->SizeUV : FVector2f(1.f, 1.f);

	// Prepare vertices and indices for the filled curve
	TArray<FSlateVertex> Verts;
	TArray<SlateIndex> Indices;
	TArray<FVector2D> BorderPoints;

	// Sample the blend curve at regular intervals
	const int32 NumSamples = FMath::Max(2, FMath::CeilToInt(HeaderSize.X / 4.0f));  // Sample every ~4 pixels
	Verts.Reserve(NumSamples * 2);
	BorderPoints.Reserve(NumSamples);

	const FFrameNumber StartFrame = SectionRange.GetLowerBoundValue();
	const FFrameNumber EndFrame = SectionRange.GetUpperBoundValue();
	const double FrameRange = (EndFrame - StartFrame).Value;

	const FVector2f HeaderSizeF(HeaderSize);
	const FVector2f Pos(Painter.HeaderGeometry.GetAbsolutePosition());
	const FSlateRenderTransform RenderTransform;

	// In overlay mode, use semi-transparent fill; in outliner mode, use more opaque fill
	const FColor FillColor = bIsInOutlinerMode ? FColor(0, 0, 0, 100) : FColor(0, 0, 0, 51);

	for (int32 i = 0; i < NumSamples; ++i)
	{
		// Calculate the normalized position (0 to 1) across the transition
		const float Alpha = static_cast<float>(i) / static_cast<float>(NumSamples - 1);

		// Calculate the frame time at this sample point
		const FFrameTime SampleTime = FFrameTime(StartFrame) + FFrameTime::FromDecimal(Alpha * FrameRange);

		// Get the blend weight at this time (default to linear if no channel)
		float BlendWeight = Alpha;  // Default linear
		if (BlendChannel)
		{
			float ChannelValue = 0.0f;
			BlendChannel->Evaluate(SampleTime, ChannelValue);
			BlendWeight = FMath::Clamp(ChannelValue, 0.0f, 1.0f);
		}

		// Calculate the X position in local space
		const float X = Alpha * HeaderSize.X;

		// Calculate Y position: BlendWeight of 0 = bottom, BlendWeight of 1 = top
		// This shows the "to" section growing from bottom as blend increases
		const float CurveY = 1.0f + (1.0f - BlendWeight) * CurveHeight;

		// UV coordinates for the texture
		const FVector2f UV_Top(Alpha, 0.0f);
		const FVector2f UV_Curve(Alpha, 0.5f);

		// Add top vertex (at y=1, just below top edge)
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
			RenderTransform,
			Pos + FVector2f(X, 1.0f) * Painter.HeaderGeometry.Scale,
			AtlasOffset + UV_Top * AtlasUVSize,
			FillColor));

		// Add curve vertex
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
			RenderTransform,
			Pos + FVector2f(X, CurveY) * Painter.HeaderGeometry.Scale,
			AtlasOffset + UV_Curve * AtlasUVSize,
			FillColor));

		// Store border point for the curve line
		BorderPoints.Add(FVector2D(X, CurveY));

		// Add indices for triangles (after we have at least 4 vertices)
		if (Verts.Num() >= 4)
		{
			const int32 Index0 = Verts.Num() - 4;  // Previous top
			const int32 Index1 = Verts.Num() - 3;  // Previous curve
			const int32 Index2 = Verts.Num() - 2;  // Current top
			const int32 Index3 = Verts.Num() - 1;  // Current curve

			// Two triangles to form a quad
			Indices.Add(Index0);
			Indices.Add(Index1);
			Indices.Add(Index2);

			Indices.Add(Index1);
			Indices.Add(Index2);
			Indices.Add(Index3);
		}
	}

	// Draw the filled curve area
	if (Indices.Num() > 0)
	{
		FSlateDrawElement::MakeCustomVerts(
			Painter.DrawElements,
			LayerId,
			ResourceHandle,
			Verts,
			Indices,
			nullptr,
			0,
			0,
			ESlateDrawEffect::PreMultipliedAlpha);
	}
	++LayerId;

	// Draw the curve border line
	if (BorderPoints.Num() >= 2)
	{
		const FLinearColor BorderColor(0.4f, 0.7f, 0.75f, 0.9f);  // Light teal/cyan border
		FSlateDrawElement::MakeLines(
			Painter.DrawElements,
			LayerId,
			Painter.HeaderGeometry.ToPaintGeometry(),
			BorderPoints,
			ESlateDrawEffect::None,
			BorderColor,
			true,  // bAntialias
			1.5f);
	}
	++LayerId;

	// Draw the transition type icon in the top-left corner
	const FName IconStyleName = TransitionSection->GetTransitionIconStyleName();
	if (!IconStyleName.IsNone())
	{
		const FSlateBrush* IconBrush = FAppStyle::Get().GetBrush(IconStyleName);
		if (IconBrush && IconBrush->GetResourceName() != NAME_None)
		{
			const float IconSize = FMath::Min(16.0f, HeaderSize.Y - 4.0f);
			const FVector2D IconPosition(2.0f, 2.0f);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.HeaderGeometry.ToPaintGeometry(FVector2D(IconSize, IconSize), FSlateLayoutTransform(IconPosition)),
				IconBrush,
				ESlateDrawEffect::None,
				FLinearColor::White);
		}
	}

	return LayerId + 1;
}

void FAnimTransitionSectionInterface::GenerateSectionLayout(ISectionLayoutBuilder& LayoutBuilder)
{
	// Call the base implementation to generate channel layouts for expansion.
	// The blend curve channel has bCanCollapseToTrack = false, so it won't add keys
	// to the section header - instead it creates an expandable row.
	// We draw our own curve visualization in OnPaintSection.
	ISequencerSection::GenerateSectionLayout(LayoutBuilder);
}

void FAnimTransitionSectionInterface::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TransitionSection", "Transition"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ChangeTransitionType_Label", "Change Transition Type"),
			LOCTEXT("ChangeTransitionType_Tooltip", "Change the type of this transition"),
			FNewMenuDelegate::CreateSP(this, &FAnimTransitionSectionInterface::PopulateChangeTypeMenu),
			/*bInOpenSubMenuOnClick=*/ false,
			FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Transition")
		);
	}
	MenuBuilder.EndSection();
}

void FAnimTransitionSectionInterface::PopulateChangeTypeMenu(FMenuBuilder& MenuBuilder)
{
	UMovieSceneAnimTransitionSectionBase* TransitionSection = GetTransitionSection();
	if (!TransitionSection)
	{
		return;
	}

	// Get all transition section classes
	TArray<UClass*> TransitionClasses;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (Class->IsChildOf(UMovieSceneAnimTransitionSectionBase::StaticClass()) &&
			!Class->HasAnyClassFlags(CLASS_Abstract) &&
			Class != UMovieSceneAnimTransitionSectionBase::StaticClass())
		{
			TransitionClasses.Add(Class);
		}
	}

	// Sort by display name
	TransitionClasses.Sort([](const UClass& A, const UClass& B)
	{
		return A.GetDisplayNameText().CompareTo(B.GetDisplayNameText()) < 0;
	});

	UClass* CurrentClass = TransitionSection->GetClass();

	for (UClass* TransitionClass : TransitionClasses)
	{
		FText DisplayName = TransitionClass->GetDisplayNameText();
		bool bIsCurrentType = (TransitionClass == CurrentClass);

		MenuBuilder.AddMenuEntry(
			DisplayName,
			FText::Format(LOCTEXT("ChangeToType_Tooltip", "Change to {0}"), DisplayName),
			FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Transition"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTransitionSectionInterface::ChangeTransitionType, TransitionClass),
				FCanExecuteAction::CreateLambda([bIsCurrentType] { return !bIsCurrentType; }),
				FIsActionChecked::CreateLambda([bIsCurrentType] { return bIsCurrentType; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
}

void FAnimTransitionSectionInterface::BeginResizeSection()
{
	UMovieSceneAnimTransitionSectionBase* TransitionSection = GetTransitionSection();
	if (!TransitionSection)
	{
		return;
	}

	// Cache initial ranges for constrained resize
	InitialTransitionRange = TransitionSection->GetRange();

	if (TransitionSection->FromSection)
	{
		InitialFromRange = TransitionSection->FromSection->GetRange();
	}
	else
	{
		InitialFromRange = TRange<FFrameNumber>::Empty();
	}

	if (TransitionSection->ToSection)
	{
		InitialToRange = TransitionSection->ToSection->GetRange();
	}
	else
	{
		InitialToRange = TRange<FFrameNumber>::Empty();
	}
}

void FAnimTransitionSectionInterface::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeFrameNumber)
{
	UMovieSceneAnimTransitionSectionBase* TransitionSection = GetTransitionSection();
	if (!TransitionSection)
	{
		return;
	}

	if (!TransitionSection->FromSection || !TransitionSection->ToSection)
	{
		return;
	}

	// Get the old range before any modifications for key scaling
	TRange<FFrameNumber> OldRange = TransitionSection->GetRange();

	// Constrain resize based on from/to section bounds
	// The transition defines the overlap region, which must stay within the valid range
	// where both sections can exist

	if (ResizeMode == SSRM_LeadingEdge)
	{
		// Resizing the left edge - this controls where the TO section starts and transition starts.
		// If the TO section is locked, prevent resizing this edge.
		if (TransitionSection->ToSection->IsLocked())
		{
			return;
		}

		// Cannot go earlier than FROM section's original start
		FFrameNumber MinFrame = InitialFromRange.HasLowerBound() ? InitialFromRange.GetLowerBoundValue() : TNumericLimits<int32>::Min();
		FFrameNumber MaxFrame = TransitionSection->HasEndFrame() ? TransitionSection->GetExclusiveEndFrame() - 1 : TNumericLimits<int32>::Max();

		FFrameNumber ClampedFrame = FMath::Clamp(ResizeFrameNumber, MinFrame, MaxFrame);

		// Modify TO section's start bound
		TransitionSection->ToSection->Modify();
		TransitionSection->ToSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(ClampedFrame));

		// Update transition to match the new overlap
		TransitionSection->Modify();
		TransitionSection->SetStartFrame(TRangeBound<FFrameNumber>::Inclusive(ClampedFrame));
	}
	else // SSRM_TrailingEdge
	{
		// Resizing the right edge - this controls where the FROM section ends and transition ends.
		// If the FROM section is locked, prevent resizing this edge.
		if (TransitionSection->FromSection->IsLocked())
		{
			return;
		}

		// Cannot go later than TO section's original end
		FFrameNumber MinFrame = TransitionSection->HasStartFrame() ? TransitionSection->GetInclusiveStartFrame() + 1 : TNumericLimits<int32>::Min();
		FFrameNumber MaxFrame = InitialToRange.HasUpperBound() ? InitialToRange.GetUpperBoundValue() : TNumericLimits<int32>::Max();

		FFrameNumber ClampedFrame = FMath::Clamp(ResizeFrameNumber, MinFrame, MaxFrame);

		// Modify FROM section's end bound
		TransitionSection->FromSection->Modify();
		TransitionSection->FromSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(ClampedFrame));

		// Update transition to match the new overlap
		TransitionSection->Modify();
		TransitionSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(ClampedFrame));
	}

	// Scale existing keys proportionally to the new range
	TransitionSection->ScaleKeysToRange(OldRange, TransitionSection->GetRange());
}

void FAnimTransitionSectionInterface::ChangeTransitionType(UClass* NewTransitionClass)
{
	UMovieSceneAnimTransitionSectionBase* OldTransition = GetTransitionSection();
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(TrackPtr.Get());

	if (!OldTransition || !MixerTrack || !NewTransitionClass)
	{
		return;
	}

	// Don't change if already the same type
	if (OldTransition->GetClass() == NewTransitionClass)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ChangeTransitionType", "Change Transition Type"));

	// Store info from old transition
	UMovieSceneSection* FromSection = OldTransition->FromSection;
	UMovieSceneSection* ToSection = OldTransition->ToSection;
	TRange<FFrameNumber> Range = OldTransition->GetRange();
	int32 RowIndex = OldTransition->GetRowIndex();

	// Get the old blend weight channel if it exists
	const FMovieSceneFloatChannel* OldBlendChannel = OldTransition->GetBlendWeightChannel();

	// Remove old transition
	MixerTrack->Modify();
	MixerTrack->RemoveSection(*OldTransition);

	UMovieSceneAnimTransitionSectionBase* NewTransition = MixerTrack->CreateTransitionSectionOfType(FromSection, ToSection, NewTransitionClass);
	if (!NewTransition)
	{
		// Failed to create new transition - sections may no longer overlap
		// The old transition has been removed, so just update our pointer to null
		SectionPtr = nullptr;
		return;
	}

	// Copy the blend curve from the old transition if both have blend weight channels
	const FMovieSceneFloatChannel* NewBlendChannel = NewTransition->GetBlendWeightChannel();
	if (OldBlendChannel && NewBlendChannel)
	{
		// Get non-const pointer to copy to
		FMovieSceneFloatChannel* MutableNewChannel = const_cast<FMovieSceneFloatChannel*>(NewBlendChannel);
		*MutableNewChannel = *OldBlendChannel;
	}
	else
	{
		// No existing curve to copy, initialize defaults
		NewTransition->InitializeDefaultCurve();
	}

	// Update the section pointer
	SectionPtr = NewTransition;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
