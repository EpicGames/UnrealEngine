// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneAnimTransitionSectionBase;
class FSequencerSectionPainter;
class FMenuBuilder;
class ISequencer;

namespace UE::Sequencer
{

/**
 * Section interface for animation mixer transition sections.
 * Provides custom rendering and interaction constraints for transitions.
 */
class FAnimTransitionSectionInterface
	: public ISequencerSection
	, public TSharedFromThis<FAnimTransitionSectionInterface>
{
public:
	FAnimTransitionSectionInterface(UMovieSceneSection& InSection, UMovieSceneTrack& InTrack, FGuid InObjectBinding);

	//~ ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetSectionTitle() const override;
	virtual FText GetSectionToolTip() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual void GenerateSectionLayout(class ISectionLayoutBuilder& LayoutBuilder) override;
	virtual bool IsReadOnly() const override { return false; }
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;

	/** Custom resize behavior: resizing the transition adjusts the from/to sections */
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeFrameNumber) override;

	/** Transition sections cannot be resized beyond the from section's start or to section's end */
	virtual bool SectionIsResizable() const override { return true; }

	/** Transition sections cannot be explicitly deleted - they are removed automatically when overlap goes away */
	virtual bool IsDeletable() const override { return false; }

	/** Transition sections should not expose mute/solo/lock/deactivate/condition column options */
	virtual bool SupportsOutlinerColumnToggle(FName ColumnName) const override { return false; }

	/** Transition sections draw their own curve visualization in OnPaintSection, so don't show the key area when collapsed */
	virtual bool ShouldShowKeyAreaWhenCollapsed() const override { return false; }

protected:
	UMovieSceneAnimTransitionSectionBase* GetTransitionSection() const;

private:
	void PopulateChangeTypeMenu(FMenuBuilder& MenuBuilder);
	void ChangeTransitionType(UClass* NewTransitionClass);

	TWeakObjectPtr<UMovieSceneSection> SectionPtr;
	TWeakObjectPtr<UMovieSceneTrack> TrackPtr;
	FGuid ObjectBindingId;

	// Cached state for resize operations
	TRange<FFrameNumber> InitialFromRange;
	TRange<FFrameNumber> InitialToRange;
	TRange<FFrameNumber> InitialTransitionRange;
};

} // namespace UE::Sequencer
