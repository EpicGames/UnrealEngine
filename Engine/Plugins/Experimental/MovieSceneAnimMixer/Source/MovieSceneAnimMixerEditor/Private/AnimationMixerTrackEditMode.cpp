// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationMixerTrackEditMode.h"

#include "AnimMixerBakeEvaluation.h"
#include "AnimMixerTrailHierarchy.h"
#include "AnimMixerBoneMatching.h"
#include "DataRegistry.h"
#include "ReferencePose.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Components/SkeletalMeshComponent.h"
#include "EditorModeManager.h"
#include "ILevelEditor.h"
#include "InteractiveToolManager.h"
#include "LevelEditor.h"
#include "TrailHierarchy.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SequencerToolsEditMode.h"
#include "EditorViewportClient.h"
#include "Engine/SkeletalMesh.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieScene.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolHelpers.h"
#include "SkeletalDebugRendering.h"
#include "Tools/TrailCategory.h"
#include "Tools/MotionTrailOptions.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "Tools/MotionTrailMenuHelpers.h"
#include "UnrealEdGlobals.h"

IMPLEMENT_HIT_PROXY(HMixerEditModeRootHitProxy, HHitProxy);

#define LOCTEXT_NAMESPACE "AnimationMixerTrackEditMode"

namespace UE::Sequencer
{

void FOffsetDragState::SnapshotFromDecoration(UMovieSceneRootMotionSettingsDecoration* Decoration, FFrameTime Time)
{
	Reset();
	if (!Decoration)
	{
		return;
	}
	double Val = 0.0;
	if (Decoration->Location[0].Evaluate(Time, Val)) { DragStartLocationOffset.X = Val; }
	if (Decoration->Location[1].Evaluate(Time, Val)) { DragStartLocationOffset.Y = Val; }
	if (Decoration->Location[2].Evaluate(Time, Val)) { DragStartLocationOffset.Z = Val; }
	if (Decoration->Rotation[0].Evaluate(Time, Val)) { DragStartRotationOffset.Roll = Val; }
	if (Decoration->Rotation[1].Evaluate(Time, Val)) { DragStartRotationOffset.Pitch = Val; }
	if (Decoration->Rotation[2].Evaluate(Time, Val)) { DragStartRotationOffset.Yaw = Val; }
}

} // namespace UE::Sequencer

using namespace UE::Sequencer;

namespace
{
	const FName MixerToolbarOwnerName = "AnimMixerViewportToolbar";
}

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationMixerTrackEditMode)

FEditorModeID UAnimationMixerTrackEditMode::ModeName("EditMode.AnimationMixerTrackEditMode");

UAnimationMixerTrackEditMode::UAnimationMixerTrackEditMode()
{
	Info = FEditorModeInfo(
		ModeName,
		LOCTEXT("ModeName", "Animation Mixer"),
		FSlateIcon(),
		false
	);
}

void UAnimationMixerTrackEditMode::Enter()
{
	UBaseLegacyWidgetEdMode::Enter();

	UE::Sequencer::MotionTrailMenu::RegisterMotionPathsToolbarEntry(
		"LevelEditor.ViewportToolbar.Transform",
		MixerToolbarOwnerName,
		"MixerTrails",
		LOCTEXT("MixerTrailsLabel", "Mixer Trails"),
		LOCTEXT("MixerTrailsTooltip", "Toggle animation mixer root motion trails"),
		ETrailCategory::AnimMixer);

	// Activate the SequencerToolsEditMode so the motion trail tool infrastructure
	// is available. This is the same mode ControlRig activates — it registers the
	// "SequencerMotionTrail" tool type with the interactive tools context and is
	// compatible with all other modes.
	Owner->ActivateMode(USequencerToolsEditMode::ModeName);

	// Subscribe after activating SequencerToolsEditMode so its broadcast doesn't
	// re-enter our handler.
	EditorModeIDChangedHandle = Owner->OnEditorModeIDChanged().AddUObject(
		this, &UAnimationMixerTrackEditMode::OnEditorModeIDChanged);
}

void UAnimationMixerTrackEditMode::SetSequencer(const TSharedPtr<ISequencer>& InSequencer)
{
	// Unsubscribe from previous sequencer if any
	if (TSharedPtr<ISequencer> OldSequencer = WeakSequencer.Pin())
	{
		OldSequencer->GetSelectionChangedSections().Remove(SectionSelectionChangedHandle);
		OldSequencer->OnMovieSceneDataChanged().Remove(MovieSceneDataChangedHandle);
		SectionSelectionChangedHandle.Reset();
		MovieSceneDataChangedHandle.Reset();
	}

	WeakSequencer = InSequencer;

	if (InSequencer)
	{
		SectionSelectionChangedHandle = InSequencer->GetSelectionChangedSections().AddUObject(
			this, &UAnimationMixerTrackEditMode::OnSequencerSectionSelectionChanged);
		MovieSceneDataChangedHandle = InSequencer->OnMovieSceneDataChanged().AddUObject(
			this, &UAnimationMixerTrackEditMode::OnMovieSceneDataChanged);
	}
}

void UAnimationMixerTrackEditMode::Exit()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->GetSelectionChangedSections().Remove(SectionSelectionChangedHandle);
		Sequencer->OnMovieSceneDataChanged().Remove(MovieSceneDataChangedHandle);
		SectionSelectionChangedHandle.Reset();
		MovieSceneDataChangedHandle.Reset();
	}

	Owner->OnEditorModeIDChanged().Remove(EditorModeIDChangedHandle);
	EditorModeIDChangedHandle.Reset();

	if (bIsTransacting)
	{
		if (GEditor)
		{
			GEditor->EndTransaction();
		}
		bIsTransacting = false;
	}

	UE::Sequencer::MotionTrailMenu::UnregisterMotionPathsToolbarEntry(MixerToolbarOwnerName);

	UBaseLegacyWidgetEdMode::Exit();
}

void UAnimationMixerTrackEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	UBaseLegacyWidgetEdMode::Tick(ViewportClient, DeltaTime);

	// Tick the tool manager so the motion trail tool updates even when no
	// other edit mode (e.g. ControlRig) drives the tool manager.
	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
	if (LevelEditorModule)
	{
		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();
		if (LevelEditor.IsValid())
		{
			UInteractiveToolManager* ToolManager = LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager;
			if (ToolManager->GetActiveTool(EToolSide::Left))
			{
				ToolManager->Tick(DeltaTime);
			}
		}
	}
}

void UAnimationMixerTrackEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const FLinearColor Colors[5] = {
		FLinearColor(1.0f, 0.0f, 1.0f, 1.0f),
		FLinearColor(0.0f, 1.0f, 0.0f, 1.0f),
		FLinearColor(0.0f, 0.0f, 1.0f, 1.0f),
		FLinearColor(0.5f, 0.5f, 0.0f, 1.0f),
		FLinearColor(0.0f, 0.5f, 0.5f, 1.0f)
	};

	FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();

	// Iterate bindings to find mixer tracks and draw skeletons / root bone points
	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		TArrayView<TWeakObjectPtr<UObject>> BoundObjects = Sequencer->FindObjectsInCurrentSequence(Binding.GetObjectGuid());

		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if (Track->IsEvalDisabled())
			{
				continue;
			}

			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
			if (!MixerTrack)
			{
				continue;
			}

			for (TWeakObjectPtr<> WeakBinding : BoundObjects)
			{
				UObject* BoundObject = WeakBinding.Get();
				if (!BoundObject)
				{
					continue;
				}

				USkeletalMeshComponent* SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(BoundObject);
				if (!SkelMeshComp)
				{
					continue;
				}

				// Invalidate cached poses if the scrub position changed
				FFrameNumber CurrentFrame = CurrentTime.Time.GetFrame();
				if (CurrentFrame != LastRenderedFrame)
				{
					InvalidateAllCachedPoses();
					LastRenderedFrame = CurrentFrame;
				}

				for (UMovieSceneSection* Section : Track->GetAllSections())
				{
					if (!Section || !Section->IsActive())
					{
						continue;
					}

					UMovieSceneRootMotionSettingsDecoration* Decoration = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
					if (!Decoration)
					{
						continue;
					}

					// Check if this section is in SelectedRootData (for gizmo positioning)
					bool bIsSelected = false;
					for (const FMixerSelectedRootData& SelData : SelectedRootData)
					{
						if (SelData.Section.Get() == Section)
						{
							bIsSelected = true;
							break;
						}
					}

					// Skip sections that aren't showing skeletons and aren't selected
					if (!Decoration->bShowSkeleton && !bIsSelected)
					{
						continue;
					}

					// Clamp to the section's range so we draw the nearest
					// pose even when the playhead is outside the section.
					FFrameNumber EvalFrame = CurrentFrame;
					if (Section->HasStartFrame() && EvalFrame < Section->GetInclusiveStartFrame())
					{
						EvalFrame = Section->GetInclusiveStartFrame();
					}
					else if (Section->HasEndFrame() && EvalFrame > Section->GetExclusiveEndFrame() - 1)
					{
						EvalFrame = Section->GetExclusiveEndFrame() - 1;
					}

					if (PDI)
					{
						FLinearColor SectionColor = FLinearColor(Section->GetColorTint());

						const float Alpha = SectionColor.A;
						SectionColor.A = 1.f;

						FLinearColor BoneColor = Colors[GetTypeHash(Section) % UE_ARRAY_COUNT(Colors)] * (1.f - Alpha) + SectionColor * Alpha;

						FCachedSectionPose& CachedPose = GetOrEvaluateSectionPose(
							Section, MixerTrack, SkelMeshComp, EvalFrame);

						FVector RootLocation = CachedPose.RootBoneLocation;

						// Update cached root transform for gizmo positioning regardless
						// of skeleton visibility
						if (bIsSelected)
						{
							CurrentRootTransform.SetLocation(RootLocation);
						}

						if (Decoration->bShowSkeleton)
						{
							// Draw bones using SkeletalDebugRendering for proper bone shapes
							const USkeletalMesh* SkelMesh = SkelMeshComp->GetSkeletalMeshAsset();
							if (SkelMesh && CachedPose.WorldBoneTransforms.Num() > 0 && CachedPose.ParentBoneIndices.Num() > 0)
							{
								const int32 NumBones = CachedPose.WorldBoneTransforms.Num();

								FSkelDebugDrawConfig DrawConfig;
								DrawConfig.BoneDrawMode = EBoneDrawMode::All;
								DrawConfig.BoneDrawSize = 1.f;
								DrawConfig.bAddHitProxy = false;
								DrawConfig.bForceDraw = true;
								DrawConfig.DefaultBoneColor = BoneColor;
								DrawConfig.AffectedBoneColor = BoneColor;
								DrawConfig.SelectedBoneColor = BoneColor;
								DrawConfig.ParentOfSelectedBoneColor = BoneColor;

								TArray<uint16> RequiredBones;
								RequiredBones.Reserve(NumBones);
								TArray<FLinearColor> BoneColours;
								BoneColours.Reserve(NumBones);
								for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
								{
									RequiredBones.Add(static_cast<uint16>(BoneIdx));
									BoneColours.Add(BoneColor);
								}

								SkeletalDebugRendering::DrawBones(
									PDI,
									SkelMeshComp->GetComponentLocation(),
									RequiredBones,
									SkelMesh->GetRefSkeleton(),
									CachedPose.WorldBoneTransforms,
									TArray<int32>(),
									BoneColours,
									TArray<TRefCountPtr<HHitProxy>>(),
									DrawConfig
								);
							}

							// Draw root bone point with hit proxy for gizmo selection.
							// Use a brighter color and larger size so it's easy to spot and click.
							FLinearColor RootColor = (BoneColor + FLinearColor::White) * 0.5f;
							RootColor.A = 1.0f;

							if (PDI->IsHitTesting())
							{
								PDI->SetHitProxy(new HMixerEditModeRootHitProxy(Section, SkelMeshComp));
							}
							PDI->DrawPoint(RootLocation, RootColor, 18.f, SDPG_Foreground);
							if (PDI->IsHitTesting())
							{
								PDI->SetHitProxy(nullptr);
							}
						}
					}

				}
			}
		}
	}

}

bool UAnimationMixerTrackEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const bool bShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	const bool bCtrlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();

	if (HitProxy && HitProxy->IsA(HMixerEditModeRootHitProxy::StaticGetType()))
	{
		HMixerEditModeRootHitProxy* Proxy = static_cast<HMixerEditModeRootHitProxy*>(HitProxy);

		UMovieSceneSection* Section = Proxy->Section.Get();
		USkeletalMeshComponent* Comp = Proxy->SkelMeshComp.Get();

		if (Section && Comp)
		{
			FMixerSelectedRootData RootData;
			RootData.Section = Section;
			RootData.MeshComp = Comp;
			RootData.Decoration = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();

			if (bCtrlDown)
			{
				// Ctrl-click: toggle the clicked item in the selection
				int32 ExistingIdx = SelectedRootData.IndexOfByPredicate([&RootData](const FMixerSelectedRootData& ExistingData) { return ExistingData == RootData; });
				if (ExistingIdx != INDEX_NONE)
				{
					SelectedRootData.RemoveAt(ExistingIdx);
				}
				else
				{
					SelectedRootData.Add(RootData);
				}
			}
			else if (bShiftDown)
			{
				// Shift-click: add to selection
				if (!SelectedRootData.Contains(RootData))
				{
					SelectedRootData.Add(RootData);
				}
			}
			else
			{
				// Plain click: replace selection
				SelectedRootData.Empty();
				SelectedRootData.Add(RootData);

				if (GEditor)
				{
					GEditor->Exec(GetWorld(), TEXT("SELECT NONE"));
				}
			}
		}
	}
	else
	{
		// Clicked empty space without modifiers: clear selection
		if (!bShiftDown && !bCtrlDown)
		{
			SelectedRootData.Empty();
		}
	}

	return false;
}

void UAnimationMixerTrackEditMode::SelectNone()
{
	SelectedRootData.Empty();
	UBaseLegacyWidgetEdMode::SelectNone();
}

bool UAnimationMixerTrackEditMode::IsSomethingSelected() const
{
	return SelectedRootData.Num() > 0;
}

bool UAnimationMixerTrackEditMode::IsTrailKeySelected() const
{
	TSharedPtr<FAnimMixerTrailHierarchy> Hierarchy = WeakTrailHierarchy.Pin();
	return Hierarchy && Hierarchy->IsAnythingSelected();
}

bool UAnimationMixerTrackEditMode::UsesTransformWidget() const
{
	return IsSomethingSelected() && !IsTrailKeySelected();
}

bool UAnimationMixerTrackEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return IsSomethingSelected() && !IsTrailKeySelected();
}

FVector UAnimationMixerTrackEditMode::GetWidgetLocation() const
{
	if (IsSomethingSelected() && WeakSequencer.IsValid())
	{
		FQualifiedFrameTime CurrentTime = WeakSequencer.Pin()->GetLocalTime();
		for (const FMixerSelectedRootData& Data : SelectedRootData)
		{
			if (Data.Section.IsValid() && Data.Section->GetRange().Contains(CurrentTime.Time.GetFrame()))
			{
				return CurrentRootTransform.GetLocation();
			}
		}
	}
	return UBaseLegacyWidgetEdMode::GetWidgetLocation();
}

bool UAnimationMixerTrackEditMode::ShouldDrawWidget() const
{
	return IsSomethingSelected() && !IsTrailKeySelected();
}

EAxisList::Type UAnimationMixerTrackEditMode::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	// Only draw widget axes when a root control point is selected
	// (not when a trail key is selected or nothing is selected)
	if (IsSomethingSelected() && !IsTrailKeySelected())
	{
		return EAxisList::All;
	}
	return EAxisList::None;
}

bool UAnimationMixerTrackEditMode::GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	if (IsSomethingSelected())
	{
		OutMatrix = CurrentRootTransform.ToMatrixNoScale().RemoveTranslation();
		return true;
	}
	return false;
}

bool UAnimationMixerTrackEditMode::GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(OutMatrix, InData);
}

bool UAnimationMixerTrackEditMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return true;
}

bool UAnimationMixerTrackEditMode::BeginDragInternal()
{
	if (bIsTransacting)
	{
		return false;
	}

	bIsTransacting = IsSomethingSelected();
	bManipulatorMadeChange = false;
	if (bIsTransacting)
	{
		DragState.Reset();
		if (SelectedRootData.Num() > 0 && SelectedRootData[0].Decoration.IsValid())
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid())
			{
				DragState.SnapshotFromDecoration(SelectedRootData[0].Decoration.Get(), Sequencer->GetLocalTime().Time);
			}
		}
	}
	return bIsTransacting;
}

bool UAnimationMixerTrackEditMode::EndDragInternal()
{
	if (bIsTransacting)
	{
		if (bManipulatorMadeChange)
		{
			bManipulatorMadeChange = false;
			GEditor->EndTransaction();

			InvalidateOffsetCacheForSelection();

			if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}
		bIsTransacting = false;
		return true;
	}
	bManipulatorMadeChange = false;
	return false;
}

void UAnimationMixerTrackEditMode::InvalidateOffsetCacheForSelection()
{
	TSet<UMovieSceneAnimationMixerTrack*> InvalidatedTracks;
	for (const FMixerSelectedRootData& Data : SelectedRootData)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Data.Section.IsValid() ? Data.Section->GetTypedOuter<UMovieSceneAnimationMixerTrack>() : nullptr)
		{
			bool bAlreadyInSet = false;
			InvalidatedTracks.Add(MixerTrack, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				MixerTrack->InvalidateAccumulatedOffsetCache();
			}
		}
	}
}

bool UAnimationMixerTrackEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return BeginDragInternal();
}

bool UAnimationMixerTrackEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return EndDragInternal();
}

bool UAnimationMixerTrackEditMode::BeginTransform(const FGizmoState& InState)
{
	return BeginDragInternal();
}

bool UAnimationMixerTrackEditMode::EndTransform(const FGizmoState& InState)
{
	return EndDragInternal();
}

bool UAnimationMixerTrackEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	const bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);
	const bool bAltDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	const bool bMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);

	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (bIsTransacting && Sequencer && bMouseButtonDown && !bCtrlDown && !bShiftDown && !bAltDown && CurrentAxis != EAxisList::None)
	{
		const bool bDoRotation = !InRot.IsZero() && (WidgetMode == UE::Widget::WM_Rotate || WidgetMode == UE::Widget::WM_TranslateRotateZ);
		const bool bDoTranslation = !InDrag.IsZero() && (WidgetMode == UE::Widget::WM_Translate || WidgetMode == UE::Widget::WM_TranslateRotateZ);
		// Manipulator drag always keys regardless of AutoKey; without a key the channel
		// has no way to reflect the drag and the gizmo would visually freeze.
		const bool bShouldAutoKey = true;

		if (bDoRotation || bDoTranslation)
		{
			for (const FMixerSelectedRootData& Data : SelectedRootData)
			{
				if (!Data.Section.IsValid() || !Data.Decoration.IsValid())
				{
					continue;
				}

				FTransform ComponentTransform = Data.MeshComp.IsValid()
					? Data.MeshComp->GetComponentTransform()
					: FTransform::Identity;

				// Accumulate the drag delta in component-local space so the
				// offset channels receive values in animation space
				if (bDoTranslation)
				{
					DragState.AccumulatedLocalDrag += ComponentTransform.InverseTransformVector(InDrag);
				}
				if (bDoRotation)
				{
					DragState.AccumulatedLocalRot += InRot;
				}

				if (!bManipulatorMadeChange)
				{
					bManipulatorMadeChange = true;
					GEditor->BeginTransaction(LOCTEXT("MoveMixerRootTransaction", "Move Mixer Root Control"));
				}

				KeyOffsetChannels(
					Data.Decoration.Get(),
					DragState.GetCurrentLocationOffset(),
					DragState.GetCurrentRotationOffset(),
					Sequencer->GetLocalTime().Time.GetFrame(),
					bShouldAutoKey);
			}

			InvalidateOffsetCacheForSelection();

			// Request trail rebake so the path redraws during the drag
			if (TSharedPtr<FAnimMixerTrailHierarchy> Hierarchy = WeakTrailHierarchy.Pin())
			{
				Hierarchy->RequestRebakeAll();
			}

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			InvalidateAllCachedPoses();
			return true;
		}
	}
	return false;
}

USkeletalMeshComponent* UAnimationMixerTrackEditMode::FindSkelMeshForSection(UMovieSceneSection* Section) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer || !Section)
	{
		return nullptr;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return nullptr;
	}

	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}

	UMovieSceneTrack* OwnerTrack = Section->GetTypedOuter<UMovieSceneTrack>();
	if (!OwnerTrack)
	{
		return nullptr;
	}

	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if (Track == OwnerTrack)
			{
				TArrayView<TWeakObjectPtr<UObject>> BoundObjects = Sequencer->FindObjectsInCurrentSequence(Binding.GetObjectGuid());
				for (TWeakObjectPtr<> WeakObj : BoundObjects)
				{
					if (UObject* Obj = WeakObj.Get())
					{
						if (USkeletalMeshComponent* SkelMesh = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(Obj))
						{
							return SkelMesh;
						}
					}
				}
				return nullptr;
			}
		}
	}

	return nullptr;
}

void UAnimationMixerTrackEditMode::OnSequencerSectionSelectionChanged(TArray<UMovieSceneSection*> SelectedSections)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	// Build the new selection from sections that have a root motion decoration
	TArray<FMixerSelectedRootData> NewSelection;
	for (UMovieSceneSection* Section : SelectedSections)
	{
		if (!Section || !Section->IsActive())
		{
			continue;
		}

		UMovieSceneRootMotionSettingsDecoration* Decoration = Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
		if (!Decoration)
		{
			continue;
		}

		UMovieSceneAnimationMixerTrack* MixerTrack = Section->GetTypedOuter<UMovieSceneAnimationMixerTrack>();
		if (!MixerTrack)
		{
			continue;
		}

		USkeletalMeshComponent* SkelMesh = FindSkelMeshForSection(Section);
		if (!SkelMesh)
		{
			continue;
		}

		FMixerSelectedRootData RootData;
		RootData.Section = Section;
		RootData.MeshComp = SkelMesh;
		RootData.Decoration = Decoration;
		NewSelection.Add(MoveTemp(RootData));
	}

	SelectedRootData = MoveTemp(NewSelection);

	if (SelectedRootData.Num() > 0)
	{
		EnsureWidgetPriority();
	}
}

void UAnimationMixerTrackEditMode::KeyOffsetChannels(
	UMovieSceneRootMotionSettingsDecoration* Decoration,
	const FVector& LocationOffset,
	const FRotator& RotationOffset,
	FFrameNumber KeyTime,
	const bool bShouldAutoKey)
{
	if (!Decoration)
	{
		return;
	}

	Decoration->Modify();

	auto SetKey = [KeyTime, bShouldAutoKey](FMovieSceneDoubleChannel& Channel, double Value)
	{
		TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel.GetData();
		int32 ExistingIndex = ChannelData.FindKey(KeyTime);
		if (ExistingIndex != INDEX_NONE)
		{
			// Always edit an existing key at the playhead - the user is directly adjusting that key's value.
			ChannelData.GetValues()[ExistingIndex].Value = Value;
		}
		else if (bShouldAutoKey && ChannelData.GetTimes().Num() > 0)
		{
			FMovieSceneDoubleValue KeyValue(Value);
			KeyValue.InterpMode = RCIM_Cubic;
			KeyValue.TangentMode = RCTM_Auto;
			ChannelData.AddKey(KeyTime, KeyValue);
			Channel.AutoSetTangents();
		}
		else
		{
			Channel.SetDefault(Value);
		}
	};

	SetKey(Decoration->Location[0], LocationOffset.X);
	SetKey(Decoration->Location[1], LocationOffset.Y);
	SetKey(Decoration->Location[2], LocationOffset.Z);
	SetKey(Decoration->Rotation[0], RotationOffset.Roll);
	SetKey(Decoration->Rotation[1], RotationOffset.Pitch);
	SetKey(Decoration->Rotation[2], RotationOffset.Yaw);

	// If the current mode is Asset (no offsets), switch to Offset so the keyed values take effect.
	// All other modes (Offset, Override, AccumulatedOffset, BoneMatch) already apply offsets additively.
	if (Decoration->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::Asset)
	{
		Decoration->TransformMode.SetDefault((uint8)EMovieSceneRootMotionTransformMode::Offset);
	}
}

UAnimationMixerTrackEditMode::FCachedSectionPose& UAnimationMixerTrackEditMode::GetOrEvaluateSectionPose(
	UMovieSceneSection* Section,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	USkeletalMeshComponent* SkelMeshComp,
	FFrameNumber CurrentFrame)
{
	FObjectKey SectionKey(Section);
	FCachedSectionPose* Existing = CachedSectionPoses.Find(SectionKey);

	// Return the cached pose if it was evaluated at the current frame. Validity is
	// gated on the retained reference-pose handle rather than Pose.IsValid(), which
	// would dereference the pose's raw reference-pose pointer.
	if (Existing && Existing->CachedAtFrame == CurrentFrame && Existing->RefPoseHandle.IsValid())
	{
		return *Existing;
	}

	// Evaluate the section's pose in isolation
	FCachedSectionPose& CachedPose = CachedSectionPoses.FindOrAdd(SectionKey);
	CachedPose.CachedAtFrame = CurrentFrame;
	CachedPose.WorldBoneTransforms.Reset();
	CachedPose.RootBoneLocation = SkelMeshComp->GetComponentLocation();
	CachedPose.Pose = UE::UAF::FLODPoseHeap();
	CachedPose.RefPoseHandle = UE::UAF::FDataHandle();

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return CachedPose;
	}

	UMovieSceneEntitySystemLinker* Linker = Sequencer->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!Linker)
	{
		return CachedPose;
	}

	UE::MovieScene::FRootInstanceHandle RootHandle = Sequencer->GetEvaluationTemplate().GetRootInstanceHandle();

	using namespace UE::MovieScene::AnimMixerBakeEvaluation;

	FBakeFilter SoloFilter;
	SoloFilter.IncludeOnlySections.Add(FObjectKey(Section));

	// Hold the reference pose for this cache entry's lifetime. The baked pose keeps
	// only a raw FReferencePose*, which the data registry frees and regenerates on a
	// required-bones change (e.g. an LOD change). Bone metadata is read through this
	// handle and the loop is sized from the pose's own buffer, so the raw pointer is
	// never dereferenced.
	UE::UAF::FDataHandle RefPoseHandle =
		UE::UAF::FDataRegistry::Get()->GetOrGenerateReferencePose(SkelMeshComp);

	FBakeResult PoseResult = EvaluateAtTime(Linker, RootHandle, MixerTrack, FFrameTime(CurrentFrame), SoloFilter);

	if (!PoseResult.IsValid())
	{
		return CachedPose;
	}

	CachedPose.RootMotionTransform = PoseResult.RootMotionTransform;
	CachedPose.Pose = MoveTemp(PoseResult.Pose);
	CachedPose.RefPoseHandle = MoveTemp(RefPoseHandle);

	// Bound by the pose's own transform buffer rather than GetNumBones(), which
	// re-queries the reference pose. The data registry can release and regenerate
	// that reference pose between evaluations (e.g. an LOD change), leaving its bone
	// count out of step with the buffer the bake produced.
	const int32 NumBones = CachedPose.Pose.LocalTransformsView.Num();
	if (NumBones > 0)
	{
		// Use the bake result's root motion transform as the base for world-space
		// bone computation. This positions the skeleton at the correct root location
		// for the evaluated time, which matters for out-of-range sections that are
		// clamped to their nearest edge rather than drawn at the current playhead.
		const FTransform ComponentTransform = CachedPose.RootMotionTransform;

		const TArrayView<const FBoneIndexType> ParentMap =
			CachedPose.RefPoseHandle.GetRef<UE::UAF::FReferencePose>()
				.GetLODBoneIndexToParentLODBoneIndexMap(CachedPose.Pose.LODLevel);

		CachedPose.ParentBoneIndices.SetNum(NumBones);
		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			if (ParentMap.IsValidIndex(BoneIdx))
			{
				FBoneIndexType Parent = ParentMap[BoneIdx];
				CachedPose.ParentBoneIndices[BoneIdx] = (Parent != INVALID_BONE_INDEX) ? (int32)Parent : INDEX_NONE;
			}
			else
			{
				CachedPose.ParentBoneIndices[BoneIdx] = INDEX_NONE;
			}
		}

		CachedPose.WorldBoneTransforms.SetNum(NumBones);

		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			int32 ParentIdx = CachedPose.ParentBoneIndices[BoneIdx];
			FTransform LocalTransform = (FTransform)CachedPose.Pose.LocalTransformsView[BoneIdx];
			LocalTransform.NormalizeRotation();

			if (ParentIdx == INDEX_NONE)
			{
				CachedPose.WorldBoneTransforms[BoneIdx] = LocalTransform * ComponentTransform;
			}
			else if (ParentIdx < NumBones)
			{
				CachedPose.WorldBoneTransforms[BoneIdx] = LocalTransform * CachedPose.WorldBoneTransforms[ParentIdx];
			}
			else
			{
				CachedPose.WorldBoneTransforms[BoneIdx] = ComponentTransform;
			}
		}

		CachedPose.RootBoneLocation = CachedPose.WorldBoneTransforms[0].GetLocation();
	}

	return CachedPose;
}

void UAnimationMixerTrackEditMode::InvalidateAllCachedPoses()
{
	CachedSectionPoses.Reset();
}

void UAnimationMixerTrackEditMode::OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	InvalidateAllCachedPoses();
}

void UAnimationMixerTrackEditMode::OnEditorModeIDChanged(const FEditorModeID& ModeID, bool bIsEnteringMode)
{
	if (bReorderingSelf || !bIsEnteringMode || ModeID == ModeName || SelectedRootData.Num() == 0)
	{
		return;
	}

	EnsureWidgetPriority();
}

void UAnimationMixerTrackEditMode::EnsureWidgetPriority()
{
	TGuardValue<bool> Guard(bReorderingSelf, true);

	// Deactivate moves us to PendingDeactivateModes; the subsequent Activate finds us
	// there and re-appends without calling Exit/Enter (those only fire from Tick).
	Owner->DeactivateMode(ModeName);
	Owner->ActivateMode(ModeName);
}

#undef LOCTEXT_NAMESPACE
