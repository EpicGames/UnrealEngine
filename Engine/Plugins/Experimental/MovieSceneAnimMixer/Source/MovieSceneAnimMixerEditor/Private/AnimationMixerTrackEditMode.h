// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GizmoEdModeInterface.h"
#include "Components/SkeletalMeshComponent.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "HitProxies.h"
#include "ISequencer.h"
#include "LODPose.h"
#include "DataRegistryTypes.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"

#include "AnimationMixerTrackEditMode.generated.h"

class UMovieSceneAnimationMixerTrack;
class UMovieSceneRootMotionSettingsDecoration;
class UMovieSceneSection;
enum class EMovieSceneDataChangeType;


// Hit proxy for clicking on root bone points rendered by the mixer edit mode
struct HMixerEditModeRootHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	TWeakObjectPtr<UMovieSceneSection> Section;
	TWeakObjectPtr<USkeletalMeshComponent> SkelMeshComp;

	HMixerEditModeRootHitProxy(UMovieSceneSection* InSection, USkeletalMeshComponent* InComp)
		: HHitProxy(HPP_Foreground)
		, Section(InSection)
		, SkelMeshComp(InComp)
	{}
};

namespace UE::Sequencer
{

class FAnimMixerTrailHierarchy;

// Tracks accumulated drag state for offset channel keying.
// Shared between the edit mode (root control gizmo) and trail key tool (trail gizmo).
struct FOffsetDragState
{
	FVector DragStartLocationOffset = FVector::ZeroVector;
	FRotator DragStartRotationOffset = FRotator::ZeroRotator;
	FVector AccumulatedLocalDrag = FVector::ZeroVector;
	FRotator AccumulatedLocalRot = FRotator::ZeroRotator;

	void Reset()
	{
		DragStartLocationOffset = FVector::ZeroVector;
		DragStartRotationOffset = FRotator::ZeroRotator;
		AccumulatedLocalDrag = FVector::ZeroVector;
		AccumulatedLocalRot = FRotator::ZeroRotator;
	}

	// Snapshot the current offset channel values at a given time
	void SnapshotFromDecoration(UMovieSceneRootMotionSettingsDecoration* Decoration, FFrameTime Time);

	FVector GetCurrentLocationOffset() const { return DragStartLocationOffset + AccumulatedLocalDrag; }
	FRotator GetCurrentRotationOffset() const { return DragStartRotationOffset + AccumulatedLocalRot; }
};

// Tracks which section/mesh/decoration is selected for gizmo interaction
struct FMixerSelectedRootData
{
	TWeakObjectPtr<UMovieSceneSection> Section;
	TWeakObjectPtr<USkeletalMeshComponent> MeshComp;
	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> Decoration;

	bool operator==(const FMixerSelectedRootData& Other) const
	{
		return Section == Other.Section && MeshComp == Other.MeshComp;
	}
};

} // namespace UE::Sequencer

// Edit mode for animation mixer tracks providing trajectory drawing,
// per-section skeleton visualization, and gizmo-based root motion offset keying.
UCLASS()
class UAnimationMixerTrackEditMode : public UBaseLegacyWidgetEdMode, public IGizmoEdModeInterface
{
	GENERATED_BODY()
public:
	static FEditorModeID ModeName;

	UAnimationMixerTrackEditMode();

	// UEdMode interface
	virtual void Enter() override;
	virtual void Exit() override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual void SelectNone() override;

	// ILegacyEdModeViewportInterface
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	// ILegacyEdModeWidgetInterface
	virtual bool UsesToolkits() const override { return false; }
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool ShouldDrawWidget() const override;
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData) override;

	void SetSequencer(const TSharedPtr<ISequencer>& InSequencer);
	void SetTrailHierarchy(const TSharedPtr<UE::Sequencer::FAnimMixerTrailHierarchy>& InHierarchy) { WeakTrailHierarchy = InHierarchy; }
	void InvalidateAllCachedPoses();

public:
	// Key the Location/Rotation channels on the decoration at a specific frame time.
	// Static so it can be called from trail key editing as well.
	static void KeyOffsetChannels(
		UMovieSceneRootMotionSettingsDecoration* Decoration,
		const FVector& LocationOffset,
		const FRotator& RotationOffset,
		FFrameNumber KeyTime,
		const bool bShouldAutoKey);

private:
	bool IsSomethingSelected() const;
	bool IsTrailKeySelected() const;
	bool BeginDragInternal();
	bool EndDragInternal();
	void InvalidateOffsetCacheForSelection();

	// Respond to sequencer section selection changes by auto-selecting roots
	// for sections that have a root motion settings decoration
	void OnSequencerSectionSelectionChanged(TArray<UMovieSceneSection*> SelectedSections);

	void OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType);
	void OnEditorModeIDChanged(const FEditorModeID& ModeID, bool bIsEnteringMode);

	// Moves this mode to the end of the active mode list so its widget-location
	// query is selected first by FEditorModeTools::GetWidgetLocation.
	void EnsureWidgetPriority();

	// Find the skeletal mesh component bound to a section's track in the sequencer
	USkeletalMeshComponent* FindSkelMeshForSection(UMovieSceneSection* Section) const;

	// Cached skeleton pose for a single section, invalidated when scrub time or data changes
	struct FCachedSectionPose
	{
		FFrameNumber CachedAtFrame;
		FTransform RootMotionTransform = FTransform::Identity;
		UE::UAF::FLODPoseHeap Pose;
		UE::UAF::FDataHandle RefPoseHandle;
		TArray<FTransform> WorldBoneTransforms;
		TArray<int32> ParentBoneIndices;
		FVector RootBoneLocation = FVector::ZeroVector;
	};

	// Evaluate or retrieve the cached skeleton pose for a section
	FCachedSectionPose& GetOrEvaluateSectionPose(
		UMovieSceneSection* Section,
		UMovieSceneAnimationMixerTrack* MixerTrack,
		USkeletalMeshComponent* SkelMeshComp,
		FFrameNumber CurrentFrame);

	TWeakPtr<ISequencer> WeakSequencer;
	TWeakPtr<UE::Sequencer::FAnimMixerTrailHierarchy> WeakTrailHierarchy;
	TArray<UE::Sequencer::FMixerSelectedRootData> SelectedRootData;
	FTransform CurrentRootTransform;

	UE::Sequencer::FOffsetDragState DragState;

	// Per-section cached skeleton poses, keyed by section object
	TMap<FObjectKey, FCachedSectionPose> CachedSectionPoses;

	// Last frame we rendered at, used to detect scrub position changes
	FFrameNumber LastRenderedFrame = FFrameNumber(TNumericLimits<int32>::Min());

	bool bIsTransacting = false;
	bool bManipulatorMadeChange = false;
	bool bReorderingSelf = false;

	FDelegateHandle SectionSelectionChangedHandle;
	FDelegateHandle MovieSceneDataChangedHandle;
	FDelegateHandle EditorModeIDChangedHandle;
};
