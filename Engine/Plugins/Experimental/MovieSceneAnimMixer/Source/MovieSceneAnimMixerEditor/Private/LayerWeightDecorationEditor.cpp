// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayerWeightDecorationEditor.h"

#include "MovieSceneLayerWeightDecoration.h"

namespace UE::Sequencer
{
	UClass* FLayerWeightDecorationEditor::GetDecorationClass() const
	{
		return UMovieSceneLayerWeightDecoration::StaticClass();
	}

	TSharedPtr<ISequencerSection> FLayerWeightDecorationEditor::MakeSectionInterface(UMovieSceneSection& Section, UMovieSceneTrack& OwningTrack, FGuid ObjectBinding)
	{
		return MakeShared<FSequencerSection>(Section);
	}
}
