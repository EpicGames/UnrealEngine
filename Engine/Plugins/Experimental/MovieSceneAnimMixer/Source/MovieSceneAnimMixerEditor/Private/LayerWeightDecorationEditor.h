// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ISequencerDecorationEditor.h"
#include "ISequencerSection.h"

namespace UE::Sequencer
{

class FLayerWeightDecorationEditor : public ISequencerDecorationEditor
{
public:
	virtual UClass* GetDecorationClass() const override;

	virtual TSharedPtr<ISequencerSection> MakeSectionInterface(UMovieSceneSection& Section, UMovieSceneTrack& OwningTrack, FGuid ObjectBinding) override;
};

}
