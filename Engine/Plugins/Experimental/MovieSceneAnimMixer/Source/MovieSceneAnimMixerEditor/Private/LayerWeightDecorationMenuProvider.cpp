// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayerWeightDecorationMenuProvider.h"

#include "ISequencer.h"
#include "MovieScene.h"
#include "MovieSceneLayerWeightDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "LayerWeightDecorationMenuProvider"

UClass* FLayerWeightDecorationMenuProvider::GetHandledDecorationClass() const
{
	return UMovieSceneLayerWeightDecoration::StaticClass();
}

void FLayerWeightDecorationMenuProvider::PopulateAddDecorationMenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject,
	UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("LayerWeightDecoration", "Layer Weight"),
		LOCTEXT("LayerWeightDecorationTooltip", "Add a keyframeable weight channel to this layer."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Weight"),
		FUIAction(FExecuteAction::CreateRaw(this, &FLayerWeightDecorationMenuProvider::OnAddLayerWeight, DecorationContainer, InSequencer))
	);
}

void FLayerWeightDecorationMenuProvider::OnAddLayerWeight(UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> WeakSequencer)
{
	if (!DecorationContainer)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddLayerWeight", "Add Layer Weight"));

	DecorationContainer->Modify();

	UMovieSceneLayerWeightDecoration* Decoration = DecorationContainer->GetOrCreateDecoration<UMovieSceneLayerWeightDecoration>();
	if (!Decoration)
	{
		return;
	}

	Decoration->Modify();

	UMovieSceneSection* NewSection = Decoration->CreateNewSection();
	if (!NewSection)
	{
		return;
	}

	Decoration->AddSection(NewSection);

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

#undef LOCTEXT_NAMESPACE
