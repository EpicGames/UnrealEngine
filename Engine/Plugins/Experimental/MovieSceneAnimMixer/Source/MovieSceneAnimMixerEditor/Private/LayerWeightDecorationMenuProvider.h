// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decorations/IMovieSceneDecorationMenuProvider.h"

// Decoration menu provider for layer weight (UMovieSceneLayerWeightDecoration).
// Adds a "Layer Weight" menu entry that creates a layer weight section spanning
// the current playback range.
class FLayerWeightDecorationMenuProvider : public IMovieSceneDecorationMenuProvider
{
public:

	virtual UClass* GetHandledDecorationClass() const override;
	virtual void PopulateAddDecorationMenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer) override;
	virtual int32 GetDecorationMenuPriority() const override { return 50; }

private:

	void OnAddLayerWeight(UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> WeakSequencer);
};
