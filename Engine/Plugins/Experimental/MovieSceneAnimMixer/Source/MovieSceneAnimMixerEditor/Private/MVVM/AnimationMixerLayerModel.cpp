// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/AnimationMixerLayerModel.h"
#include "MVVM/AnimationMixerTrackModel.h"
#include "MVVM/AnimationMixerLayoutHelpers.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneTrack.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"

#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "SequencerUtilities.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "SequencerCommonHelpers.h"
#include "Decorations/MovieSceneTrackRowDecoration.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"


// Per-layer bake support
#include "AnimMixerBakeHelper.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "MovieSceneAnimationMixerTrackEditor.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"

#define LOCTEXT_NAMESPACE "AnimationMixerLayerModel"

namespace UE
{
namespace Sequencer
{

UE_SEQUENCER_DEFINE_CASTABLE(FAnimationMixerLayerModel);

/**
 * Sort sections for display: transitions appear between their from/to sections.
 * Regular sections are sorted by start time, and transitions are inserted after their "from" section.
 */
static void SortSectionsForDisplay(TArray<UMovieSceneSection*>& Sections)
{
	// Separate transitions from regular sections
	TArray<UMovieSceneAnimTransitionSectionBase*> Transitions;
	TArray<UMovieSceneSection*> RegularSections;

	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneAnimTransitionSectionBase* Transition = Cast<UMovieSceneAnimTransitionSectionBase>(Section))
		{
			Transitions.Add(Transition);
		}
		else if (Section)
		{
			RegularSections.Add(Section);
		}
	}

	// Sort regular sections by start time
	RegularSections.Sort([](const UMovieSceneSection& A, const UMovieSceneSection& B)
	{
		if (!A.HasStartFrame() && !B.HasStartFrame())
		{
			return false;
		}
		if (!A.HasStartFrame())
		{
			return true;
		}
		if (!B.HasStartFrame())
		{
			return false;
		}
		return A.GetInclusiveStartFrame() < B.GetInclusiveStartFrame();
	});

	// Build result: insert transitions after their "from" section
	Sections.Empty(RegularSections.Num() + Transitions.Num());

	for (UMovieSceneSection* Section : RegularSections)
	{
		Sections.Add(Section);

		// Check if any transitions have this section as their "from" section
		for (int32 i = Transitions.Num() - 1; i >= 0; --i)
		{
			if (Transitions[i]->FromSection == Section)
			{
				Sections.Add(Transitions[i]);
				Transitions.RemoveAt(i);
			}
		}
	}

	// Add any remaining transitions (orphaned or with invalid from/to references)
	for (UMovieSceneAnimTransitionSectionBase* Transition : Transitions)
	{
		Sections.Add(Transition);
	}
}

FAnimationMixerLayerModel::FAnimationMixerLayerModel(UMovieSceneAnimationMixerTrack* InParentTrack, UMovieSceneAnimationMixerLayer* InLayer)
	: WeakParentTrack(InParentTrack)
	, WeakLayer(InLayer)
{
	RegisterChildList(&SectionList);
	RegisterChildList(&TopLevelChannelList);
}

FAnimationMixerLayerModel::FAnimationMixerLayerModel(UMovieSceneAnimationMixerTrack* InParentTrack, UMovieSceneAnimationMixerLayer* InLayer, UMovieSceneTrack* InChildTrack)
	: WeakParentTrack(InParentTrack)
	, WeakLayer(InLayer)
	, WeakChildTrack(InChildTrack)
{
	RegisterChildList(&SectionList);
	RegisterChildList(&TopLevelChannelList);
}

FAnimationMixerLayerModel::~FAnimationMixerLayerModel()
{
	if (ChildTrackChangedHandle.IsValid() && WeakChildTrack.IsValid())
	{
		WeakChildTrack.Get()->OnSignatureChanged().Remove(ChildTrackChangedHandle);
	}

	if (LayerChangedHandle.IsValid() && WeakLayer.IsValid())
	{
		WeakLayer.Get()->OnSignatureChanged().Remove(LayerChangedHandle);
	}
}

void FAnimationMixerLayerModel::OnChildTrackChanged()
{
	UMovieSceneTrack* ChildTrack = WeakChildTrack.Get();
	if (!ChildTrack)
	{
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	if (FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>())
	{
		UMovieSceneAnimationMixerTrack* ParentTrack = WeakParentTrack.Get();
		if (ParentTrack)
		{
			FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
			TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
			DecorationModelStorage->SyncDecorationModels(ChildTrack, ParentTrack, OutlinerChildren, SequencerPtr.Get());
		}
	}

	// The child track's signature changes on undo/redo, property edits, etc.
	// Refresh the layout so section models with stale channel proxies get rebuilt.
	RefreshLayout(false);
}

void FAnimationMixerLayerModel::OnLayerChanged()
{
	// The layer's signature changes when decorations are added/removed/edited on the
	// layer itself (e.g. LayerWeight).Re-run the layout so SyncDecorationModels(Layer)
	// picks up the new state.
	RefreshLayout(false);
}

void FAnimationMixerLayerModel::OnConstruct()
{
	// Get track editor from sequencer
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (SequenceModel)
	{
		TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
		if (Sequencer)
		{
			if (WeakChildTrack.IsValid())
			{
				TrackEditor = Sequencer->GetTrackEditor(WeakChildTrack.Get());

				// Listen for child track changes to sync decoration models
				// (handles decoration add/remove and reconstruction needs like Mode changes).
				UMovieSceneTrack* ChildTrack = WeakChildTrack.Get();
				ChildTrackChangedHandle = ChildTrack->OnSignatureChanged().AddSP(
					this, &FAnimationMixerLayerModel::OnChildTrackChanged);
			}
			else if (WeakParentTrack.IsValid())
			{
				TrackEditor = Sequencer->GetTrackEditor(WeakParentTrack.Get());
			}

			// Listen for layer-level changes (decoration adds/removes on the layer itself,
			// e.g. LayerWeight).
			if (UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get())
			{
				LayerChangedHandle = Layer->OnSignatureChanged().AddSP(
					this, &FAnimationMixerLayerModel::OnLayerChanged);
			}
		}
	}
}

void FAnimationMixerLayerModel::RefreshLayout(bool bChildrenNeedLayout)
{
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	FGuid ObjectBinding;
	if (TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = FindAncestorOfType<IObjectBindingExtension>())
	{
		ObjectBinding = ObjectBindingExtension->GetObjectGuid();
	}

	UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get();
	if (!Layer)
	{
		return;
	}

	FSectionModelStorageExtension* SectionModelStorage = SequenceModel->CastDynamic<FSectionModelStorageExtension>();
	if (!SectionModelStorage)
	{
		return;
	}

	// Layer has a child track - act like FTrackModel
	if (WeakChildTrack.IsValid())
	{
		UMovieSceneTrack* ChildTrack = WeakChildTrack.Get();
		if (!ChildTrack)
		{
			return;
		}

		// Register this layer model as the ITrackExtension for the child track
		FTrackModelStorageExtension* TrackModelStorage = SequenceModel->CastDynamic<FTrackModelStorageExtension>();
		if (TrackModelStorage)
		{
			TViewModelPtr<ITrackExtension> TrackExtension = CastViewModel<ITrackExtension>(AsShared());
			TrackModelStorage->RegisterTrackExtensionForTrack(ChildTrack, TrackExtension);
		}

		// Get populated rows from the child track
		TBitArray<> PopulatedRows;
		// Child track case - act like FTrackModel
		for (UMovieSceneSection* Section : ChildTrack->GetAllSections())
		{
			if (Section)
			{
				const int32 RowIndex = Section->GetRowIndex();
				PopulatedRows.PadToNum(RowIndex + 1, false);
				PopulatedRows[RowIndex] = true;
			}
		}

		const int32 NumRows = PopulatedRows.CountSetBits();

		FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
		FViewModelChildren SectionChildren = GetChildrenForList(&SectionList);
		FViewModelChildren TopLevelChannelChildren = GetTopLevelChannels();

		if (NumRows == 0)
		{
			// Clear everything
			OutlinerChildren.Empty();
			SectionChildren.Empty();
			TopLevelChannelChildren.Empty();
			return;
		}
		else if (NumRows == 1)
		{
			// Single row case - use helper for section layout
			TArray<UMovieSceneSection*> AllSections = ChildTrack->GetAllSections();
			// Query saved expansion state directly - we can't use IsExpanded() here because
			// it checks for children, but we're in the process of building them
			bool bCreateOutliners = GetSavedExpansionState();

			// Only recycle top level channels if children need layout
			if (bChildrenNeedLayout)
			{
				FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
				GetTopLevelChannels().MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
			}

			AnimationMixerHelpers::LayoutSections(
				AllSections,
				ChildTrack,
				AsShared(),
				OutlinerChildren,
				SectionChildren,
				SectionModelStorage,
				TrackEditor,
				ObjectBinding,
				bCreateOutliners,
				bChildrenNeedLayout,
				PreviousLayoutNumSections);

			PreviousLayoutNumSections = AllSections.Num();
		}
		else
		{
			// Multi-row case - need track row models
			FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
			SectionChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
			OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
			TopLevelChannelChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);

			FTrackRowModelStorageExtension* TrackRowModelStorage = SequenceModel->CastDynamic<FTrackRowModelStorageExtension>();
			if (!TrackRowModelStorage)
			{
				return;
			}

			// Create track row models for all populated rows
			TSharedPtr<FTrackRowModel> LastTrackRowModel;
			for (TConstSetBitIterator<> It(PopulatedRows); It; ++It)
			{
				const int32 RowIndex = It.GetIndex();
				bool bRowNeedsLayout = bChildrenNeedLayout;

				TSharedPtr<FTrackRowModel> TrackRowModel = TrackRowModelStorage->FindModelForTrackRow(ChildTrack, RowIndex);
				if (!TrackRowModel)
				{
					TrackRowModel = TrackRowModelStorage->CreateModelForTrackRow(ChildTrack, RowIndex, AsShared());
					bRowNeedsLayout = true;
				}

				if (ensure(TrackRowModel))
				{
					OutlinerChildren.InsertChild(TrackRowModel, LastTrackRowModel);
					LastTrackRowModel = TrackRowModel;
					TrackRowModel->RefreshLayout(bRowNeedsLayout);
				}
			}
		}

		// Sync child track decorations (e.g., root motion settings) AND the layer's own
		// decorations (e.g. LayerWeight) so both show in the outliner. The else branch
		// below covers the no-child-track case; this branch must do both.
		if (FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>())
		{
			UMovieSceneAnimationMixerTrack* ParentTrack = WeakParentTrack.Get();
			if (ParentTrack)
			{
				TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
				DecorationModelStorage->SyncDecorationModels(ChildTrack, ParentTrack, OutlinerChildren, SequencerPtr.Get());
				DecorationModelStorage->SyncDecorationModels(Layer, ParentTrack, OutlinerChildren, SequencerPtr.Get());
			}
		}
	}
	// Layer has regular sections - act like FTrackRowModel
	else
	{
		UMovieSceneAnimationMixerTrack* ParentTrack = WeakParentTrack.Get();
		if (ParentTrack)
		{
			// Get sections from the layer and use helper
			TArray<UMovieSceneSection*> LayerSections = Layer->GetSections();

			// Sort sections so transitions appear between their from/to sections
			SortSectionsForDisplay(LayerSections);

			// Query saved expansion state directly - we can't use IsExpanded() here because
			// it checks for children, but we're in the process of building them
			bool bCreateOutliners = GetSavedExpansionState();

			// Only recycle top level channels if children need layout
			if (bChildrenNeedLayout)
			{
				FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
				GetTopLevelChannels().MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
			}

			FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
			FViewModelChildren SectionChildren = GetChildrenForList(&SectionList);

			AnimationMixerHelpers::LayoutSections(
				LayerSections,
				ParentTrack,
				AsShared(),
				OutlinerChildren,
				SectionChildren,
				SectionModelStorage,
				TrackEditor,
				ObjectBinding,
				bCreateOutliners,
				bChildrenNeedLayout,
				PreviousLayoutNumSections);

			PreviousLayoutNumSections = LayerSections.Num();

			// Handle layer decorations. Section decorations are handled by
			// RefreshDecorations() on each section outliner (called from LayoutSections).
			if (FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>())
			{
				TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
				DecorationModelStorage->SyncDecorationModels(Layer, ParentTrack, OutlinerChildren, SequencerPtr.Get());
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	ITrackExtension
-----------------------------------------------------------------------------*/

UMovieSceneTrack* FAnimationMixerLayerModel::GetTrack() const
{
	// We return the child track if it's valid, otherwise we should return the parent track
	return WeakChildTrack.IsValid() ? WeakChildTrack.Get() : GetParentTrack();
}

TSharedPtr<ISequencerTrackEditor> FAnimationMixerLayerModel::GetTrackEditor() const
{
	return TrackEditor;
}

/*-----------------------------------------------------------------------------
	ITrackRowExtension
-----------------------------------------------------------------------------*/
int32 FAnimationMixerLayerModel::GetRowIndex() const
{
	UMovieSceneAnimationMixerTrack* ParentTrack = WeakParentTrack.Get();
	UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get();

	if (ParentTrack && Layer)
	{
		const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = ParentTrack->GetLayers();
		return Layers.IndexOfByKey(Layer);
	}

	return 0;
}

bool FAnimationMixerLayerModel::SetRowIndex(int32 InRowIndex)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = GetParentMixerTrack();
	UMovieSceneAnimationMixerLayer* Layer = GetLayer();
	if (!MixerTrack || !Layer)
	{
		return false;
	}

	TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = const_cast<TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>&>(MixerTrack->GetLayers());

	// Find current index
	int32 CurrentIndex = Layers.IndexOfByKey(Layer);
	if (CurrentIndex == INDEX_NONE || CurrentIndex == InRowIndex)
	{
		return false;
	}

	// Validate target index
	if (InRowIndex < 0 || InRowIndex >= Layers.Num())
	{
		return false;
	}

	MixerTrack->Modify();

	// Build a mapping of old row indices before reordering so we can remap
	// per-row metadata (mute/solo state, track row metadata, etc.)
	TArray<int32> OldIndices;
	OldIndices.SetNum(Layers.Num());
	for (int32 Index = 0; Index < Layers.Num(); ++Index)
	{
		OldIndices[Index] = Index;
	}

	// Move the layer in the array
	Layers.RemoveAt(CurrentIndex);
	Layers.Insert(Layer, InRowIndex);

	// The same reorder applied to the old indices gives us the old-index-at-new-position
	OldIndices.RemoveAt(CurrentIndex);
	OldIndices.Insert(CurrentIndex, InRowIndex);

	// Build NewToOld map: NewRowIndex -> OldRowIndex
	TMap<int32, int32> NewToOldRowIndices;
	for (int32 NewIndex = 0; NewIndex < OldIndices.Num(); ++NewIndex)
	{
		NewToOldRowIndices.Add(NewIndex, OldIndices[NewIndex]);
	}

	// Update row indices for all layers and their contents
	for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
	{
		UMovieSceneAnimationMixerLayer* LayerToUpdate = Layers[LayerIndex];
		if (!LayerToUpdate)
		{
			continue;
		}

		// Update child track row if present
		if (LayerToUpdate->HasChildTrack())
		{
			UMovieSceneTrack* ChildTrack = LayerToUpdate->GetChildTrack();
			if (ChildTrack)
			{
				MixerTrack->SetChildTrackRow(ChildTrack, LayerIndex);
			}
		}

		// Update section row indices
		for (UMovieSceneSection* Section : LayerToUpdate->GetSections())
		{
			if (Section)
			{
				Section->Modify();
				Section->SetRowIndex(LayerIndex);
			}
		}
	}

	// Remap per-row metadata (mute/solo state, track row decoration, etc.)
	MixerTrack->OnRowIndicesChanged(NewToOldRowIndices);

	return true;
}

UMovieSceneTrack* FAnimationMixerLayerModel::GetParentTrack() const
{
	return WeakParentTrack.Get();
}

/*-----------------------------------------------------------------------------
	ISectionOwnerExtension
-----------------------------------------------------------------------------*/

FViewModelChildren FAnimationMixerLayerModel::GetSectionModels()
{
	return GetChildrenForList(&SectionList);
}

/*-----------------------------------------------------------------------------
	ITopLevelChannelHolderExtension
-----------------------------------------------------------------------------*/

FViewModelChildren FAnimationMixerLayerModel::GetTopLevelChannels()
{
	return GetChildrenForList(&TopLevelChannelList);
}

/*-----------------------------------------------------------------------------
	IOutlinerExtension
-----------------------------------------------------------------------------*/

FOutlinerSizing FAnimationMixerLayerModel::GetOutlinerSizing() const
{
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();
	const float SectionAreaDefaultHeight = 27.0f;
	float Height = Density.UniformHeight.Get(SectionAreaDefaultHeight);
	if (auto It = SectionList.Iterate<FSectionModel>(); It)
	{
		TSharedPtr<FSectionModel> Section = *It;
		Height = Section->GetSectionInterface()->GetSectionHeight(Density);
	}
	return FOutlinerSizing(Height);
}

TSharedPtr<SWidget> FAnimationMixerLayerModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	if (TrackEditor)
	{
		// For child track layers, route the "Add" column through the mixer track editor
		// so it can show both child track items and decoration entries (e.g., root motion settings)
		if (InColumnName == FCommonOutlinerNames::Add && WeakChildTrack.IsValid() && WeakParentTrack.IsValid())
		{
			TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
			TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
			if (Sequencer)
			{
				TSharedPtr<ISequencerTrackEditor> MixerTrackEditor = Sequencer->GetTrackEditor(WeakParentTrack.Get());
				if (MixerTrackEditor)
				{
					FBuildColumnWidgetParams Params(SharedThis(this), InParams);
					return MixerTrackEditor->BuildOutlinerColumnWidget(Params, InColumnName);
				}
			}
		}

		FBuildColumnWidgetParams Params(SharedThis(this), InParams);
		return TrackEditor->BuildOutlinerColumnWidget(Params, InColumnName);
	}
	return nullptr;
}

FSlateFontInfo FAnimationMixerLayerModel::GetLabelFont() const
{
	bool bAllAnimated = false;
	TViewModelPtr<FChannelGroupModel> TopLevelChannel = TopLevelChannelList.GetHead().ImplicitCast();
	if (TopLevelChannel)
	{
		for (const TViewModelPtr<FChannelModel>& ChannelModel : TopLevelChannel->GetDescendantsOfType<FChannelModel>())
		{
			FMovieSceneChannel* Channel = ChannelModel->GetChannel();
			if (!Channel || Channel->GetNumKeys() == 0)
			{
				return FOutlinerItemModel::GetLabelFont();
			}
			else
			{
				bAllAnimated = true;
			}
		}
		if (bAllAnimated == true)
		{
			return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont");
		}
	}
	return FOutlinerItemModel::GetLabelFont();
}

const FSlateBrush* FAnimationMixerLayerModel::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.MixLayer");
}

FText FAnimationMixerLayerModel::GetLabel() const
{
	if (UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get())
	{
		return Layer->GetDisplayName();
	}
	return FText::GetEmpty();
}

FSlateColor FAnimationMixerLayerModel::GetLabelColor() const
{
	UMovieSceneTrack* Track = GetTrack();
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!Track || !SequenceModel)
	{
		return FSlateColor::UseForeground();
	}

	TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
	if (!Sequencer)
	{
		return FSlateColor::UseForeground();
	}
	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	LabelParams.Player = Sequencer.Get();
	LabelParams.SequenceID = SequenceModel->GetSequenceID();
	if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
	{
		LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();
		// If the object binding model has an invalid binding, we want to use its label color, as it may be red or gray depending on situation
		// and we want the children of that to have the same color.
		// Otherwise, we can use the track's label color below
		TArrayView<TWeakObjectPtr<> > BoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
		if (BoundObjects.Num() == 0)
		{
			return ObjectBindingModel->GetLabelColor();
		}
	}
	return Track->GetLabelColor(LabelParams);
}

FText FAnimationMixerLayerModel::GetLabelToolTipText() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return FText();
	}

	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
		{
			LabelParams.SequenceID = SequenceModel->GetSequenceID();
			LabelParams.Player = SequencerModel->GetSequencer().Get();
			if (LabelParams.Player)
			{
				if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
				{
					LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();
				}
				return Track->GetDisplayNameToolTipText(LabelParams);
			}
		}
	}
	return FText();
}

bool FAnimationMixerLayerModel::IsDimmed() const
{
	UMovieSceneTrack* Track = GetTrack();

	if (Track)
	{
		if (Track->IsRowEvalDisabled(GetRowIndex()))
		{
			return true;
		}

		FGuid BindingID;
		FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
		if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
		{
			BindingID = ObjectBindingModel->GetObjectGuid();
		}

		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			SequenceID = SequenceModel->GetSequenceID();

			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, TrackRowMetadata->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
						{
							return true;
						}
					}
				}

				if (Track->ConditionContainer.Condition)
				{
					if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, Track->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
					{
						return true;
					}
				}
			}
		}
	}

	return FOutlinerItemModel::IsDimmed();
}

/*-----------------------------------------------------------------------------
	IResizableExtension
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::IsResizable() const
{
	UMovieSceneTrack* Track = GetTrack();
	return Track && TrackEditor->IsResizable(Track);
}

void FAnimationMixerLayerModel::Resize(float NewSize)
{
	UMovieSceneTrack* Track = GetTrack();

	if (Track && TrackEditor->IsResizable(Track))
	{
		TrackEditor->Resize(NewSize, Track);
	}
}

/*-----------------------------------------------------------------------------
	ILockableExtension
-----------------------------------------------------------------------------*/

ELockableLockState FAnimationMixerLayerModel::GetLockState() const
{
	int32 NumSections = 0;
	int32 NumLockedSections = 0;

	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		++NumSections;

		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject && SectionObject->IsLocked())
		{
			++NumLockedSections;
		}
	}

	if (NumSections == 0 || NumLockedSections == 0)
	{
		return ELockableLockState::None;
	}
	return NumLockedSections == NumSections ? ELockableLockState::Locked : ELockableLockState::PartiallyLocked;
}

void FAnimationMixerLayerModel::SetIsLocked(bool bInIsLocked)
{
	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject)
		{
			SectionObject->Modify();
			SectionObject->SetIsLocked(bInIsLocked);
		}
	}
}

/*-----------------------------------------------------------------------------
	IConditionableExtension
-----------------------------------------------------------------------------*/

const UMovieSceneCondition* FAnimationMixerLayerModel::GetCondition() const
{
	if (UMovieSceneTrack* ChildTrack = GetChildTrack())
	{
		// Use the child track condition
		if (ChildTrack->ConditionContainer.Condition)
		{
			return ChildTrack->ConditionContainer.Condition;
		}

		if (ChildTrack->SupportsMultipleRows() && ChildTrack->GetMaxRowIndex() == 0)
		{
			if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = ChildTrack->FindTrackRowMetadata(0))
			{
				if (TrackRowMetadata->ConditionContainer.Condition)
				{
					return TrackRowMetadata->ConditionContainer.Condition;
				}
			}
		}
	}
	
	if (UMovieSceneTrack* Track = GetTrack())
	{
		if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
		{
			return TrackRowMetadata->ConditionContainer.Condition;
		}
	}

	return nullptr;
}

EConditionableConditionState FAnimationMixerLayerModel::GetConditionState() const
{
	UMovieSceneTrack* ParentTrack = GetParentTrack();
	UMovieSceneTrack* ChildTrack = GetChildTrack();

	if (ParentTrack)
	{
		FGuid BindingID;
		FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
		if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
		{
			BindingID = ObjectBindingModel->GetObjectGuid();
		}

		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			SequenceID = SequenceModel->GetSequenceID();
			TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
			if (Sequencer)
			{
				// We have a child track and need to check additional conditions on the track and its track row metadata
				if (ChildTrack)
				{
					if (ChildTrack->ConditionContainer.Condition)
					{
						if (ChildTrack->ConditionContainer.Condition->bEditorForceTrue)
						{
							return EConditionableConditionState::HasConditionEditorForceTrue;
						}

						if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), ChildTrack->ConditionContainer.Condition, ChildTrack, Sequencer->GetSharedPlaybackState()))
						{
							return EConditionableConditionState::HasConditionEvaluatingTrue;
						}
						else
						{
							return EConditionableConditionState::HasConditionEvaluatingFalse;
						}
					}

					// Special case. If we support multiple rows, and there is only a single row, then we must also check track row metadata for a condition here, as there will be no track row model.
					if (ChildTrack->SupportsMultipleRows() && ChildTrack->GetMaxRowIndex() == 0)
					{
						if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = ChildTrack->FindTrackRowMetadata(0))
						{
							if (TrackRowMetadata->ConditionContainer.Condition)
							{
								if (TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue)
								{
									return EConditionableConditionState::HasConditionEditorForceTrue;
								}
								else if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), TrackRowMetadata->ConditionContainer.Condition, ChildTrack, Sequencer->GetSharedPlaybackState()))
								{
									return EConditionableConditionState::HasConditionEvaluatingTrue;
								}
								else
								{
									return EConditionableConditionState::HasConditionEvaluatingFalse;
								}
							}
						}
					}
				}

				// Now check the track row metadata of the mixer track
				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = ParentTrack->FindTrackRowMetadata(GetRowIndex()))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue)
						{
							return EConditionableConditionState::HasConditionEditorForceTrue;
						}
						else if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), TrackRowMetadata->ConditionContainer.Condition, ParentTrack, Sequencer->GetSharedPlaybackState()))
						{
							return EConditionableConditionState::HasConditionEvaluatingTrue;
						}
						else
						{
							return EConditionableConditionState::HasConditionEvaluatingFalse;
						}
					}
				}
			}
		}
	}
	return EConditionableConditionState::None;
}

void FAnimationMixerLayerModel::SetConditionEditorForceTrue(bool bEditorForceTrue)
{
	if (UMovieSceneTrack* ChildTrack = GetChildTrack())
	{
		// Use the child track condition
		if (ChildTrack->ConditionContainer.Condition)
		{
			ChildTrack->ConditionContainer.Condition->Modify();
			ChildTrack->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
			return;
		}

		if (ChildTrack->SupportsMultipleRows() && ChildTrack->GetMaxRowIndex() == 0)
		{
			if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = ChildTrack->FindTrackRowMetadata(0))
			{
				if (TrackRowMetadata->ConditionContainer.Condition)
				{
					TrackRowMetadata->ConditionContainer.Condition->Modify();
					TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
					return;
				}
			}
		}
	}

	if (UMovieSceneTrack* Track = GetTrack())
	{
		if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
		{
			TrackRowMetadata->ConditionContainer.Condition->Modify();
			TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
			return;
		}
	}
}

/*-----------------------------------------------------------------------------
	ITrackAreaExtension
-----------------------------------------------------------------------------*/

FTrackAreaParameters FAnimationMixerLayerModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Nested;
	Params.TrackLanePadding.Bottom = 1.f;
	return Params;
}

FViewModelVariantIterator FAnimationMixerLayerModel::GetTrackAreaModelList() const
{
	return &SectionList;
}

FViewModelVariantIterator FAnimationMixerLayerModel::GetTopLevelChildTrackAreaModels() const
{
	return &TopLevelChannelList;
}

/*-----------------------------------------------------------------------------
	ICurveEditorTreeItem
-----------------------------------------------------------------------------*/

void FAnimationMixerLayerModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast();
	if (ChannelGroup)
	{
		ChannelGroup->CreateCurveModels(OutCurveModels);
	}
}

/*-----------------------------------------------------------------------------
	IGroupableIdentifier
-----------------------------------------------------------------------------*/

void FAnimationMixerLayerModel::GetIdentifierForGrouping(TStringBuilder<128>& OutString) const
{
	if (UMovieSceneAnimationMixerTrack* ParentTrack = WeakParentTrack.Get())
	{
		OutString.Append(ParentTrack->GetPathName());
		OutString.Appendf(TEXT("_Layer%d"), GetRowIndex());
	}
}

/*-----------------------------------------------------------------------------
	IRenameableExtension
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::CanRename() const
{
	UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(GetTrack());
	return NameableTrack && NameableTrack->CanRename();
}

void FAnimationMixerLayerModel::Rename(const FText& NewName)
{
	if (UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get())
	{
		Layer->Modify();
		Layer->SetDisplayName(NewName);
	}
}

bool FAnimationMixerLayerModel::IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const
{
	UMovieSceneNameableTrack* NameableTrack = ::Cast<UMovieSceneNameableTrack>(GetTrack());
	if (NameableTrack)
	{
		return NameableTrack->ValidateDisplayName(NewName, OutErrorMessage);
	}
	return false;
}

/*-----------------------------------------------------------------------------
	ISortableExtension
-----------------------------------------------------------------------------*/

void FAnimationMixerLayerModel::SortChildren()
{
}

FSortingKey FAnimationMixerLayerModel::GetSortingKey() const
{
	// Use row index for sorting
	FSortingKey SortingKey;
	SortingKey.CustomOrder = GetRowIndex();
	return SortingKey;
}

void FAnimationMixerLayerModel::SetCustomOrder(int32 InCustomOrder)
{
	// Do nothing
}

/*-----------------------------------------------------------------------------
	IDraggableOutlinerExtension
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::CanDrag() const
{
	return true;
}

/*-----------------------------------------------------------------------------
	FOutlinerItemModel
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::HasCurves() const
{
	FViewModelChildren TopLevelChannels = const_cast<FAnimationMixerLayerModel*>(this)->GetTopLevelChannels();
	for (auto It(TopLevelChannels.IterateSubList<FChannelGroupModel>()); It; ++It)
	{
		if (It->HasCurves())
		{
			return true;
		}
	}
	return false;
}

void FAnimationMixerLayerModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> SequencerImpl = EditorViewModel->GetSequencerImpl();
	if (!SequencerImpl.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!IsValid(Track))
	{
		return;
	}

	// If this layer has a child track, delegate to its track editor
	if (WeakChildTrack.IsValid() && TrackEditor.IsValid())
	{
		TrackEditor->BuildTrackContextMenu(MenuBuilder, Track);
	}

	// Build menu for sections in this layer
	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(SequencerImpl, TrackAreaModels, MenuBuilder, true);

	// If this layer has channels, add channel override menu
	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	// Per-layer bake menu - unified across section-layers and child-track layers.
	// Always offers "Bake Layer" for the clicked layer; additionally offers
	// "Bake Selected Layers (N)" when multiple layer rows are selected and the clicked
	// row is part of that selection.
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(WeakParentTrack.Get());
	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();

	if (Sequencer.IsValid() && MixerTrack)
	{
		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence() ? Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
		FGuid ObjectBinding;
		if (MovieScene && MovieScene->FindTrackBinding(*MixerTrack, ObjectBinding))
		{
			USkeletalMeshComponent* SkelMeshComp = FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, Sequencer);
			USkeleton* Skeleton = (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset()) ? SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton() : nullptr;
			UObject* BoundObject = Sequencer->FindSpawnedObjectOrTemplate(ObjectBinding);

			if (SkelMeshComp && Skeleton)
			{
				const int32 ClickedRow = GetRowIndex();
				TSharedPtr<ISequencer> SequencerShared = Sequencer;

				// Resolve the track editor's bFilterAssetBySkeleton toggle. The track editor exists
				// whenever we have a valid layer model, so no fallback is needed. The static_cast
				// is guarded by SupportsType() so third-party editors that don't handle mixer tracks
				// can't be misinterpreted as FAnimationMixerTrackEditor.
				TSharedPtr<ISequencerTrackEditor> MixerTrackEditor = Sequencer->GetTrackEditor(MixerTrack);
				FAnimationMixerTrackEditor* TypedEditor = nullptr;
				if (MixerTrackEditor && MixerTrackEditor->SupportsType(UMovieSceneAnimationMixerTrack::StaticClass()))
				{
					TypedEditor = static_cast<FAnimationMixerTrackEditor*>(MixerTrackEditor.Get());
				}
				if (!TypedEditor) { return; }
				bool& FilterBySkeletonRef = TypedEditor->bFilterAssetBySkeleton;

				// Enumerate selected mixer-layer models from the outliner selection.
				// Defensive: UE does not guarantee the right-clicked row is pre-selected,
				// so we only treat a selection as "valid multi-select" when `this` is in it.
				TArray<TSharedPtr<FAnimationMixerLayerModel>> SelectedLayers;
				if (const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection())
				{
					for (TViewModelPtr<FAnimationMixerLayerModel> TypedItem : Selection->Outliner.Filter<FAnimationMixerLayerModel>())
					{
						if (TypedItem)
						{
							SelectedLayers.AddUnique(TSharedPtr<FAnimationMixerLayerModel>(TypedItem));
						}
					}
				}
				const bool bClickedIsSelected = SelectedLayers.ContainsByPredicate(
					[this](const TSharedPtr<FAnimationMixerLayerModel>& L) { return L.Get() == this; });

				// Helper: mute one source layer. For child-track layers, disable the child track
				// itself (it's its own UMovieSceneTrack with its own rows). For section layers,
				// disable the mixer track's row - that's where the sections live.
				auto MuteSourceLayer = [](UMovieSceneAnimationMixerTrack* InMixer, const TSharedPtr<FAnimationMixerLayerModel>& L)
				{
					UMovieSceneAnimationMixerLayer* LayerPtr = L ? L->GetLayer() : nullptr;
					UMovieSceneTrack* ChildTrack = LayerPtr ? LayerPtr->GetChildTrack() : nullptr;
					if (ChildTrack)
					{
						MovieSceneHelpers::DisableTrack(ChildTrack);
					}
					else if (InMixer && L)
					{
						InMixer->SetRowEvalDisabled(true, L->GetRowIndex());
					}
				};

				// --- "Bake Layer" section (always, scoped to clicked layer only) ---
				{
					UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter LayerFilter;
					LayerFilter.MinPriority = ClickedRow;
					LayerFilter.MaxPriority = ClickedRow;
					LayerFilter.bSkipRootMotionConversion = false;

					TSharedPtr<FAnimationMixerLayerModel> SelfLayer = StaticCastSharedRef<FAnimationMixerLayerModel>(AsShared());

					MenuBuilder.BeginSection("AnimMixerLayerBake_Single", LOCTEXT("BakeLayerSection", "Bake Layer"));
					UE::Sequencer::AnimMixerBake::BuildBakeMenuSection(
						MenuBuilder, SequencerShared, MixerTrack, SkelMeshComp, Skeleton,
						ObjectBinding, BoundObject, FilterBySkeletonRef, LayerFilter,
						[WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack), ObjectBinding, ClickedRow]
						(UMovieScene*, TSubclassOf<UMovieSceneTrack> TrackClass) -> UMovieSceneTrack*
						{
							UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
							if (!MixerTrack) { return nullptr; }
							MixerTrack->Modify();
							MixerTrack->InsertLayer(ClickedRow + 1);
							return MixerTrack->AddChildTrack(ObjectBinding, TrackClass, ClickedRow + 1);
						},
						[WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack), SelfLayer, SequencerShared, MuteSourceLayer]
						(UMovieSceneTrack*, UObject*, bool bSuccess)
						{
							if (!bSuccess) { return; }
							UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
							if (!MixerTrack) { return; }
							MixerTrack->Modify();
							MuteSourceLayer(MixerTrack, SelfLayer);
							SequencerShared->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
						},
						LOCTEXT("BakeLayerVerb", "Bake Layer"));
					MenuBuilder.EndSection();
				}

				// --- "Bake Selected Layers (N)" section (only when N>1 and clicked is in selection) ---
				if (bClickedIsSelected && SelectedLayers.Num() > 1)
				{
					int32 MinRow = MAX_int32;
					int32 MaxRow = MIN_int32;
					TArray<int32> Rows;
					Rows.Reserve(SelectedLayers.Num());
					for (const TSharedPtr<FAnimationMixerLayerModel>& L : SelectedLayers)
					{
						const int32 R = L->GetRowIndex();
						MinRow = FMath::Min(MinRow, R);
						MaxRow = FMath::Max(MaxRow, R);
						Rows.Add(R);
					}
					const bool bContiguous = (MaxRow - MinRow + 1) == SelectedLayers.Num();

					UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter GroupFilter;
					GroupFilter.MinPriority = MinRow;
					GroupFilter.MaxPriority = MaxRow;
					GroupFilter.bSkipRootMotionConversion = false;

					const FText SectionLabel = FText::Format(
						LOCTEXT("BakeSelectedLayersSectionFmt", "Bake Selected Layers ({0})"),
						FText::AsNumber(SelectedLayers.Num()));

					FText DisabledTooltip;
					if (!bContiguous)
					{
						Rows.Sort();
						DisabledTooltip = FText::Format(
							LOCTEXT("BakeSelectedNotContiguousTooltip",
									"Selected layers must be contiguous to bake as a group. Current rows: {0}."),
							FText::FromString(FString::JoinBy(Rows, TEXT(", "), [](int32 R) { return FString::FromInt(R); })));
					}

					// Capture the full source set for muting at completion; sorted by row-index so insertion
					// lands below the visually-lowest source layer.
					TArray<TSharedPtr<FAnimationMixerLayerModel>> SortedSources = SelectedLayers;
					SortedSources.Sort([](const TSharedPtr<FAnimationMixerLayerModel>& A, const TSharedPtr<FAnimationMixerLayerModel>& B)
					{
						return A->GetRowIndex() < B->GetRowIndex();
					});

					MenuBuilder.BeginSection("AnimMixerLayerBake_Selected", SectionLabel);
					UE::Sequencer::AnimMixerBake::BuildBakeMenuSection(
						MenuBuilder, SequencerShared, MixerTrack, SkelMeshComp, Skeleton,
						ObjectBinding, BoundObject, FilterBySkeletonRef, GroupFilter,
						[WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack), ObjectBinding, MaxRow]
						(UMovieScene*, TSubclassOf<UMovieSceneTrack> TrackClass) -> UMovieSceneTrack*
						{
							UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
							if (!MixerTrack) { return nullptr; }
							MixerTrack->Modify();
							MixerTrack->InsertLayer(MaxRow + 1);
							return MixerTrack->AddChildTrack(ObjectBinding, TrackClass, MaxRow + 1);
						},
						[WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack), SortedSources, SequencerShared, MuteSourceLayer]
						(UMovieSceneTrack*, UObject*, bool bSuccess)
						{
							if (!bSuccess) { return; }
							UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
							if (!MixerTrack) { return; }
							MixerTrack->Modify();
							for (const TSharedPtr<FAnimationMixerLayerModel>& L : SortedSources)
							{
								MuteSourceLayer(MixerTrack, L);
							}
							SequencerShared->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
						},
						LOCTEXT("BakeSelectedLayersVerb", "Bake Selected Layers"),
						FCanExecuteAction::CreateLambda([bContiguous]() { return bContiguous; }),
						DisabledTooltip);
					MenuBuilder.EndSection();
				}
			}
		}
	}

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

bool FAnimationMixerLayerModel::GetDefaultExpansionState() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (TrackEditor && Track)
	{
		return TrackEditor->GetDefaultExpansionState(Track);
	}

	return false;
}

void FAnimationMixerLayerModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!IsValid(Track))
	{
		return;
	}

	// If this layer has a child track, delegate to its track editor
	if (WeakChildTrack.IsValid() && TrackEditor.IsValid())
	{
		TrackEditor->BuildTrackSidebarMenu(MenuBuilder, Track);
	}

	// Build menu for sections in this layer
	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, false);

	// If this layer has channels, add channel override menu
	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

void FAnimationMixerLayerModel::SetExpansion(bool bInIsExpanded)
{
	FEvaluableOutlinerItemModel::SetExpansion(bInIsExpanded);

	// Reset layout state to force complete rebuild when expansion changes
	// This ensures we properly switch between collapsed (sections in SectionList)
	// and expanded (sections in section outliners) modes
	PreviousLayoutNumSections.Reset();

	RefreshLayout(true);
}

/*-----------------------------------------------------------------------------
	FEvaluableOutlinerItemModel - Deactivation
	The base class implementation iterates ITrackExtension descendants and
	calls SetEvalDisabled on the track returned by GetTrack(). For mixer
	layers that don't have a child track, GetTrack() returns the parent
	mixer track, which would deactivate the entire mixer instead of just
	this layer. Override to use per-row deactivation on the parent track.
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::IsDeactivated() const
{
	// Child track layer: check if the child track itself is eval-disabled
	if (UMovieSceneTrack* ChildTrack = WeakChildTrack.Get())
	{
		return ChildTrack->IsEvalDisabled(/*bInCheckLocal=*/false);
	}

	// Section layer: check row-level deactivation on the parent mixer track
	if (UMovieSceneTrack* ParentTrack = WeakParentTrack.Get())
	{
		return ParentTrack->IsRowEvalDisabled(GetRowIndex(), /*bInCheckLocal=*/false);
	}
	return false;
}

void FAnimationMixerLayerModel::SetIsDeactivated(bool bInIsDeactivated)
{
	bool bChanged = false;

	// Child track layer: disable the child track itself
	if (UMovieSceneTrack* ChildTrack = WeakChildTrack.Get())
	{
		if (bInIsDeactivated != ChildTrack->IsEvalDisabled(/*bInCheckLocal=*/false))
		{
			ChildTrack->Modify();
			ChildTrack->SetEvalDisabled(bInIsDeactivated);
			bChanged = true;
		}
	}
	else
	{
		// Section layer: use row-level deactivation on the parent mixer track
		UMovieSceneTrack* ParentTrack = WeakParentTrack.Get();
		if (ParentTrack && bInIsDeactivated != ParentTrack->IsRowEvalDisabled(GetRowIndex(), /*bInCheckLocal=*/false))
		{
			ParentTrack->Modify();
			ParentTrack->SetRowEvalDisabled(bInIsDeactivated, GetRowIndex());
			bChanged = true;
		}
	}

	if (bChanged)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		if (EditorViewModel)
		{
			TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
			if (Sequencer)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	IDeletableExtension
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FAnimationMixerLayerModel::Delete()
{
	UMovieSceneAnimationMixerTrack* MixerTrack = WeakParentTrack.Get();
	UMovieSceneAnimationMixerLayer* Layer = WeakLayer.Get();

	if (!MixerTrack || !Layer)
	{
		return;
	}

	// Mark track for modification (for undo/redo)
	MixerTrack->Modify();
	Layer->Modify();

	// Get the layer index before we remove it
	const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& ConstLayers = MixerTrack->GetLayers();
	int32 LayerIndex = ConstLayers.IndexOfByKey(Layer);
	if (LayerIndex == INDEX_NONE)
	{
		return; // Layer not found in track
	}

	// Handle child track removal if present
	if (Layer->HasChildTrack())
	{
		UMovieSceneTrack* ChildTrack = Layer->GetChildTrack();
		if (ChildTrack)
		{
			// Remove the child track from the mixer track
			MixerTrack->RemoveChildTrack(ChildTrack);
		}
		Layer->SetChildTrack(nullptr);
	}

	// Handle section removal
	TArray<UMovieSceneSection*> SectionsToRemove = Layer->GetSections();
	for (UMovieSceneSection* Section : SectionsToRemove)
	{
		if (Section)
		{
			Section->Modify();
			// Remove section from the track entirely
			MixerTrack->RemoveSection(*Section);
		}
	}

	// Remove the layer from the layers array
	TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = const_cast<TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>&>(MixerTrack->GetLayers());
	Layers.RemoveAt(LayerIndex);

	// Update row indices for all remaining layers after the deleted one
	for (int32 UpdateIndex = LayerIndex; UpdateIndex < Layers.Num(); ++UpdateIndex)
	{
		UMovieSceneAnimationMixerLayer* LayerToUpdate = Layers[UpdateIndex];
		if (LayerToUpdate)
		{
			AnimationMixerHelpers::UpdateLayerRowIndices(LayerToUpdate, MixerTrack, UpdateIndex);
		}
	}
}

/*-----------------------------------------------------------------------------
	IMutableExtension / ISoloableExtension
-----------------------------------------------------------------------------*/

bool FAnimationMixerLayerModel::IsMuted() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Track->FindDecoration<UMovieSceneTrackRowDecoration>())
	{
		return TrackRowDecoration->IsMuted(GetRowIndex());
	}

	return false;
}

void FAnimationMixerLayerModel::SetIsMuted(bool bIsMuted)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Cast<UMovieSceneTrackRowDecoration>(Track->GetOrCreateDecoration(UMovieSceneTrackRowDecoration::StaticClass())))
	{
		TrackRowDecoration->SetMuted(GetRowIndex(), bIsMuted);
	}
}

bool FAnimationMixerLayerModel::IsSolo() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Track->FindDecoration<UMovieSceneTrackRowDecoration>())
	{
		return TrackRowDecoration->IsSoloed(GetRowIndex());
	}

	return false;
}

void FAnimationMixerLayerModel::SetIsSoloed(bool bIsSoloed)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Cast<UMovieSceneTrackRowDecoration>(Track->GetOrCreateDecoration(UMovieSceneTrackRowDecoration::StaticClass())))
	{
		TrackRowDecoration->SetSoloed(GetRowIndex(), bIsSoloed);
	}
}

bool FAnimationMixerLayerModel::GetSavedExpansionState() const
{
	// This replicates the initialization logic from FOutlinerItemModelMixin::IsExpanded()
	// but without the children check at the end. We need this during layout because
	// IsExpanded() checks for children, but we're in the process of building them.

	TStringBuilder<256> StringBuilder;
	IOutlinerExtension::GetPathName(*this, StringBuilder);

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	UMovieSceneSequence* Sequence = SequenceModel ? SequenceModel->GetSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (MovieScene)
	{
		FStringView StringView = StringBuilder.ToView();
		FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
		if (const FMovieSceneExpansionState* Expansion = EditorData.ExpansionStates.FindByHash(GetTypeHash(StringView), StringView))
		{
			return Expansion->bExpanded;
		}
	}

	// Fall back to default expansion state if no saved state exists
	return GetDefaultExpansionState();
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE