// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/AnimationMixerLayoutHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "ISequencerTrackEditor.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/SharedViewModelData.h"

namespace UE::Sequencer::AnimationMixerHelpers
{

TArray<TSharedPtr<FSectionModel>> BuildSectionModels(
	const TArray<UMovieSceneSection*>& Sections,
	FSectionModelStorageExtension* SectionStorage,
	TSharedPtr<ISequencerTrackEditor> TrackEditor,
	UMovieSceneTrack* TrackForSections,
	const FGuid& ObjectBinding,
	bool& bOutChildrenNeedLayout)
{
	TArray<TSharedPtr<FSectionModel>> SectionModels;

	if (!SectionStorage || !TrackEditor || !TrackForSections)
	{
		return SectionModels;
	}

	for (UMovieSceneSection* Section : Sections)
	{
		if (!IsValid(Section))
		{
			SectionModels.Add(nullptr);
			continue;
		}

		TSharedPtr<FSectionModel> SectionModel = SectionStorage->FindModelForSection(Section);
		if (!SectionModel)
		{
			TSharedRef<ISequencerSection> SectionInterface = TrackEditor->MakeSectionInterface(*Section, *TrackForSections, ObjectBinding);
			SectionModel = SectionStorage->CreateModelForSection(Section, SectionInterface);
			bOutChildrenNeedLayout = true;
		}
		else
		{
			bOutChildrenNeedLayout |= SectionModel->NeedsLayout();
		}

		SectionModels.Add(SectionModel);
	}

	return SectionModels;
}

TArray<TSharedPtr<FViewModel>> CreateSectionOutliners(
	const TArray<TSharedPtr<FSectionModel>>& SectionModels,
	FViewModelChildren& OutlinerChildren,
	const FViewModelChildren& RecycledChildren,
	TSharedPtr<FViewModel> Owner)
{
	TArray<TSharedPtr<FViewModel>> SectionRoots;
	TSharedPtr<FViewModel> OutlinerTail;

	for (TSharedPtr<FSectionModel> SectionModel : SectionModels)
	{
		if (!SectionModel)
		{
			SectionRoots.Add(Owner);
			continue;
		}

		UMovieSceneSection* Section = SectionModel->GetSection();
		if (!Section)
		{
			SectionRoots.Add(Owner);
			continue;
		}

		// Find existing section outliner model in recycled children first, then current children
		TSharedPtr<FSectionOutlinerModel> SectionOutliner;

		// Search recycled children for existing section outliner
		for (TSharedPtr<FSectionOutlinerModel> ExistingOutliner : RecycledChildren.IterateSubList<FSectionOutlinerModel>())
		{
			if (ExistingOutliner && ExistingOutliner->GetSection() == Section)
			{
				SectionOutliner = ExistingOutliner;
				break;
			}
		}

		// Also check current children (in case of partial rebuild)
		if (!SectionOutliner)
		{
			for (TSharedPtr<FSectionOutlinerModel> ExistingOutliner : OutlinerChildren.IterateSubList<FSectionOutlinerModel>())
			{
				if (ExistingOutliner && ExistingOutliner->GetSection() == Section)
				{
					SectionOutliner = ExistingOutliner;
					break;
				}
			}
		}

		if (!SectionOutliner)
		{
			SectionOutliner = MakeShared<FSectionOutlinerModel>(Section, SectionModel);
		}

		// Add to outliner hierarchy
		OutlinerChildren.InsertChild(SectionOutliner, OutlinerTail);
		OutlinerTail = SectionOutliner;

		SectionOutliner->OnConstruct();
		SectionOutliner->GetTrackAreaChildren().AddChild(SectionModel);
		SectionModel->SetLinkedOutlinerItem(SectionOutliner);

		SectionRoots.Add(SectionOutliner);
	}

	return SectionRoots;
}

TArray<TSharedPtr<FViewModel>> AddSectionsDirectly(
	const TArray<TSharedPtr<FSectionModel>>& SectionModels,
	FViewModelChildren& SectionChildren,
	TSharedPtr<FViewModel> Owner)
{
	TArray<TSharedPtr<FViewModel>> SectionRoots;
	TSharedPtr<FViewModel> SectionsTail;

	// Cast Owner to IOutlinerExtension once
	TViewModelPtr<IOutlinerExtension> OwnerOutliner = CastViewModel<IOutlinerExtension>(Owner.ToSharedRef());

	for (TSharedPtr<FSectionModel> SectionModel : SectionModels)
	{
		if (SectionModel)
		{
			SectionChildren.InsertChild(SectionModel, SectionsTail);
			SectionsTail = SectionModel;

			// Set linked outliner item if Owner supports IOutlinerExtension
			if (OwnerOutliner)
			{
				SectionModel->SetLinkedOutlinerItem(OwnerOutliner);
			}
		}

		SectionRoots.Add(Owner);
	}

	return SectionRoots;
}

void RunLayoutBuilders(
	const TArray<TSharedPtr<FSectionModel>>& SectionModels,
	const TArray<TSharedPtr<FViewModel>>& SectionRoots)
{
	ensure(SectionModels.Num() == SectionRoots.Num());

	for (int32 i = 0; i < SectionModels.Num(); ++i)
	{
		if (SectionModels[i] && SectionRoots[i])
		{
			FTrackModelLayoutBuilder LayoutBuilder(SectionRoots[i]);
			LayoutBuilder.RefreshLayout(SectionModels[i]);
		}
	}
}

void LayoutSections(
	const TArray<UMovieSceneSection*>& Sections,
	UMovieSceneTrack* TrackForSections,
	TSharedPtr<FViewModel> Owner,
	FViewModelChildren& OutlinerChildren,
	FViewModelChildren& SectionChildren,
	FSectionModelStorageExtension* SectionStorage,
	TSharedPtr<ISequencerTrackEditor> TrackEditor,
	const FGuid& ObjectBinding,
	bool bCreateSectionOutliners,
	bool& bOutChildrenNeedLayout,
	TOptional<int32> PreviousLayoutNumSections)
{
	if (!Owner || !SectionStorage || !TrackEditor || !TrackForSections)
	{
		return;
	}

	// Check if number of sections changed
	if (PreviousLayoutNumSections.IsSet())
	{
		bOutChildrenNeedLayout |= (Sections.Num() != PreviousLayoutNumSections.GetValue());
	}

	// Build section models (this checks if models exist and need layout)
	TArray<TSharedPtr<FSectionModel>> SectionModels = BuildSectionModels(
		Sections,
		SectionStorage,
		TrackEditor,
		TrackForSections,
		ObjectBinding,
		bOutChildrenNeedLayout);

	// Only recycle and rebuild if children actually need layout
	if (bOutChildrenNeedLayout)
	{
		// Recycle existing section and outliner children
		FScopedViewModelListHead RecycledModels(Owner, EViewModelListType::Recycled);
		SectionChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
		OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);

		TArray<TSharedPtr<FViewModel>> SectionRoots;

		// When outliner creation is enabled, create section outliner models for each section.
		// Otherwise add sections directly to the track area.
		if (bCreateSectionOutliners)
		{
			SectionRoots = CreateSectionOutliners(SectionModels, OutlinerChildren, RecycledModels.GetChildren(), Owner);
		}
		else
		{
			SectionRoots = AddSectionsDirectly(SectionModels, SectionChildren, Owner);
		}

		// Run layout builder on each section
		RunLayoutBuilders(SectionModels, SectionRoots);
	}

	// Refresh decorations on section outliners. This runs outside the rebuild block
	// so that decoration state changes (e.g. channel proxy dirty) are detected even
	// when the layout itself didn't change.
	for (TSharedPtr<FSectionOutlinerModel> SectionOutliner : OutlinerChildren.IterateSubList<FSectionOutlinerModel>())
	{
		if (SectionOutliner)
		{
			SectionOutliner->RefreshDecorations();
		}
	}
}

void UpdateLayerRowIndices(
	UMovieSceneAnimationMixerLayer* Layer,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	int32 NewRowIndex,
	const TSet<UMovieSceneSection*>* ExcludeSections)
{
	if (!Layer || !MixerTrack)
	{
		return;
	}

	// Update child track row if present
	if (Layer->HasChildTrack())
	{
		if (UMovieSceneTrack* ChildTrack = Layer->GetChildTrack())
		{
			MixerTrack->SetChildTrackRow(ChildTrack, NewRowIndex);
		}
	}

	// Update section row indices
	for (UMovieSceneSection* Section : Layer->GetSections())
	{
		if (Section)
		{
			if (ExcludeSections && ExcludeSections->Contains(Section))
			{
				continue;
			}

			Section->Modify();
			Section->SetRowIndex(NewRowIndex);
		}
	}
}

} // namespace UE::Sequencer::AnimationMixerHelpers
