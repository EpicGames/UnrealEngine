// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelPtr.h"

class UMovieSceneSection;
class UMovieSceneTrack;
class UMovieSceneAnimationMixerLayer;
class UMovieSceneAnimationMixerTrack;
class ISequencerTrackEditor;
struct FGuid;

namespace UE::Sequencer
{
	class FSectionModel;
	class FViewModel;
	class FSectionModelStorageExtension;
	class IOutlinerExtension;
	struct FViewModelChildren;
}

namespace UE::Sequencer::AnimationMixerHelpers
{
	/**
	 * Lays out sections for the given track: handles both single and multiple section cases
	 */
	void LayoutSections(
		const TArray<UMovieSceneSection*>& Sections,
		UMovieSceneTrack* TrackForSections,
		TSharedPtr<FViewModel> Owner,
		FViewModelChildren& OutlinerChildren,
		FViewModelChildren& SectionChildren,
		FSectionModelStorageExtension* SectionStorage,
		TSharedPtr<ISequencerTrackEditor> TrackEditor,
		const FGuid& ObjectBinding,
		bool bCreateSectionOutliners,
		bool& bOutChildrenNeedLayout,
		TOptional<int32> PreviousLayoutNumSections = TOptional<int32>());

	/**
	 * Update row indices for a layer's contents (sections and child tracks)
	 * Common pattern used in drag/cleanup operations
	 */
	void UpdateLayerRowIndices(
		UMovieSceneAnimationMixerLayer* Layer,
		UMovieSceneAnimationMixerTrack* MixerTrack,
		int32 NewRowIndex,
		const TSet<UMovieSceneSection*>* ExcludeSections = nullptr);

} // namespace UE::Sequencer::AnimationMixerHelpers
