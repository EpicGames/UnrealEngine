// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/AnimationMixerTrackModel.h"
#include "MVVM/AnimationMixerLayerModel.h"
#include "MVVM/AnimationMixerLayoutHelpers.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "SequencerSettings.h"
#include "MovieSceneSection.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Rendering/DrawElements.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "Framework/Application/SlateApplication.h"

namespace UE::Sequencer
{

UE_SEQUENCER_DEFINE_CASTABLE(FAnimationMixerTrackModel);

static EViewModelListType GetLayerBarListType()
{
	static EViewModelListType LayerBarListType = RegisterCustomModelListType();
	return LayerBarListType;
}

FAnimationMixerTrackModel::FAnimationMixerTrackModel(UMovieSceneTrack* Track)
	: FTrackModel(Track)
	, LayerBarList(GetLayerBarListType())
{
	RegisterChildList(&LayerBarList);
}

void FAnimationMixerTrackModel::OnConstruct()
{
	FTrackModel::OnConstruct();

	if (!LayerBar)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		TSharedPtr<ISequencer> Sequencer = EditorViewModel ? EditorViewModel->GetSequencer() : nullptr;

		if (Sequencer && Sequencer->GetSequencerSettings()->GetShowLayerBars())
		{
			LayerBar = MakeShared<FLayerBarModel>(AsShared());
			LayerBar->SetLinkedOutlinerItem(SharedThis(this));

			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
			TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakMixerTrack(MixerTrack);
			LayerBar->SetOverlayText(MakeAttributeLambda([WeakMixerTrack]() -> FText
			{
				if (UMovieSceneAnimationMixerTrack* Track = WeakMixerTrack.Get())
				{
					const int32 NumLayers = Track->GetLayers().Num();
					return FText::Format(NSLOCTEXT("AnimationMixerTrackModel", "LayerCount", "{0} {0}|plural(one=Layer,other=Layers)"), NumLayers);
				}
				return FText::GetEmpty();
			}));

			GetChildrenForList(&LayerBarList).AddChild(LayerBar);
		}
	}
}

FViewModelVariantIterator FAnimationMixerTrackModel::GetTopLevelChildTrackAreaModels() const
{
	return &LayerBarList;
}

bool FAnimationMixerTrackModel::IsDeactivated() const
{
	UMovieSceneTrack* Track = GetTrack();
	return Track && Track->IsEvalDisabled(/*bInCheckLocal=*/false);
}

void FAnimationMixerTrackModel::SetIsDeactivated(bool bInIsDeactivated)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track || bInIsDeactivated == Track->IsEvalDisabled(/*bInCheckLocal=*/false))
	{
		return;
	}

	Track->Modify();
	Track->SetEvalDisabled(bInIsDeactivated);

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (EditorViewModel)
	{
		if (TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

void FAnimationMixerTrackModel::ForceUpdate()
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
	ensure(MixerTrack);
	if (!MixerTrack)
	{
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);

	// Recycle existing outliner children
	FScopedViewModelListHead RecycledOutliner(AsShared(), EViewModelListType::Recycled);
	OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledOutliner.GetChildren(), IRecyclableExtension::CallOnRecycle);

	const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = MixerTrack->GetLayers();

	// Always create layer models for all layers
	TSharedPtr<FAnimationMixerLayerModel> LastLayerModel;

	for (UMovieSceneAnimationMixerLayer* Layer : Layers)
	{
		if (!Layer)
		{
			continue;
		}

		// Find existing layer model or create new one
		TSharedPtr<FAnimationMixerLayerModel> LayerModel;

		// Try to find existing model in recycled children
		for (TViewModelPtr<FAnimationMixerLayerModel> ExistingModel : RecycledOutliner.GetChildren().IterateSubList<FAnimationMixerLayerModel>().ToArray())
		{
			if (ExistingModel && ExistingModel->GetLayer() == Layer)
			{
				// If the layer's child track state changed, discard the old model and create fresh
				if (ExistingModel->HasChildTrack() != Layer->HasChildTrack())
				{
					break;
				}
				LayerModel = ExistingModel;
				break;
			}
		}

		// Create new layer model if not found or if layer type changed
		bool bNeedsLayout = false;
		if (!LayerModel)
		{
			if (Layer->HasChildTrack())
			{
				LayerModel = MakeShared<FAnimationMixerLayerModel>(MixerTrack, Layer, Layer->GetChildTrack());
			}
			else
			{
				LayerModel = MakeShared<FAnimationMixerLayerModel>(MixerTrack, Layer);
			}
			bNeedsLayout = true;
		}

		// Add to outliner
		OutlinerChildren.InsertChild(LayerModel, LastLayerModel);
		LastLayerModel = LayerModel;

		// Initialize and refresh layout
		if (bNeedsLayout)
		{
			LayerModel->OnConstruct();
		}
		LayerModel->RefreshLayout(bNeedsLayout);
	}

	// Handle track-level decorations
	FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>();
	if (DecorationModelStorage)
	{
		TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
		DecorationModelStorage->SyncDecorationModels(MixerTrack, MixerTrack, OutlinerChildren, SequencerPtr.Get());
	}
}

bool FAnimationMixerTrackModel::IsRowOccupiedByChildTrack(int32 RowIndex) const
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
	if (!MixerTrack)
	{
		return false;
	}

	// Check if this row (layer index) has a child track that blocks section drops
	const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = MixerTrack->GetLayers();
	if (RowIndex >= 0 && RowIndex < Layers.Num())
	{
		UMovieSceneAnimationMixerLayer* Layer = Layers[RowIndex];
		if (Layer && Layer->HasChildTrack())
		{
			return true;
		}
	}

	return false;
}

void FAnimationMixerTrackModel::ShiftRowsDownFromIndex(int32 StartIndex, const TSet<UMovieSceneSection*>& ExcludeSections)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
	if (!MixerTrack)
	{
		FTrackModel::ShiftRowsDownFromIndex(StartIndex, ExcludeSections);
		return;
	}

	MixerTrack->Modify();
	MixerTrack->InsertLayer(StartIndex);

	// Rebuild layer models with updated indices
	ForceUpdate();
}

void FAnimationMixerTrackModel::OnBeginSectionVerticalDrag()
{
	// Mark that we're now in a drag operation
	bIsInDrag = true;
	CachedPreview.Reset();
}

void FAnimationMixerTrackModel::OnEndSectionVerticalDrag(const FSectionVerticalDragContext& Context)
{
	// Apply the cached preview if one exists
	if (CachedPreview.IsSet())
	{
		ApplyDropPreview(CachedPreview.GetValue(), Context);
	}

	// Clear drag flag and cached preview
	bIsInDrag = false;
	CachedPreview.Reset();
}

bool FAnimationMixerTrackModel::OnSectionVerticalDrag(const FSectionVerticalDragContext& Context)
{
	TOptional<FAnimMixerDropPreview> NewPreview = ComputeDropPreview(Context);

	bool bChanged = (CachedPreview.IsSet() && NewPreview != CachedPreview);
	CachedPreview = NewPreview;
	return bChanged;
}

int32 FAnimationMixerTrackModel::OnPaintSectionDragPreview(
	const FGeometry& TrackAreaGeometry,
	const FVirtualTrackArea& VirtualTrackArea,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled) const
{
	if (!CachedPreview.IsSet())
	{
		return LayerId;
	}

	const FAnimMixerDropPreview& Preview = CachedPreview.GetValue();

	// Convert virtual coordinates to physical coordinates
	FVector2D PhysicalTop = VirtualTrackArea.VirtualToPhysical(FVector2D(0.0f, Preview.VirtualTop));
	FVector2D PhysicalBottom = VirtualTrackArea.VirtualToPhysical(FVector2D(0.0f, Preview.VirtualBottom));
	float PhysicalHeight = PhysicalBottom.Y - PhysicalTop.Y;

	if (Preview.Zone == FAnimMixerDropPreview::EZone::OntoLayer)
	{
		// Draw dashed outline around the section bounds (matching STrackAreaView drop target style)
		FSlateColor DashColor = FStyleColors::AccentBlue;

		const FSlateBrush* HorizontalBrush = FAppStyle::GetBrush("WideDash.Horizontal");
		const FSlateBrush* VerticalBrush = FAppStyle::GetBrush("WideDash.Vertical");

		// Calculate horizontal bounds from section frame range
		float DropMinX = 0.f;
		float DropMaxX = TrackAreaGeometry.GetLocalSize().X;

		if (Preview.SectionFrameRange.HasLowerBound() && Preview.SectionFrameRange.HasUpperBound())
		{
			DropMinX = VirtualTrackArea.FrameToPixel(Preview.SectionFrameRange.GetLowerBoundValue());
			DropMaxX = VirtualTrackArea.FrameToPixel(Preview.SectionFrameRange.GetUpperBoundValue());
		}

		float TrackPosition = PhysicalTop.Y;
		float TrackHeight = PhysicalHeight;

		// Top edge
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			TrackAreaGeometry.ToPaintGeometry(FVector2f(DropMaxX - DropMinX, HorizontalBrush->ImageSize.Y), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition))),
			HorizontalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Bottom edge
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			TrackAreaGeometry.ToPaintGeometry(FVector2f(DropMaxX - DropMinX, HorizontalBrush->ImageSize.Y), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition + (TrackHeight - HorizontalBrush->ImageSize.Y)))),
			HorizontalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Left edge
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			TrackAreaGeometry.ToPaintGeometry(FVector2f(VerticalBrush->ImageSize.X, TrackHeight), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition))),
			VerticalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Right edge
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			TrackAreaGeometry.ToPaintGeometry(FVector2f(VerticalBrush->ImageSize.X, TrackHeight), FSlateLayoutTransform(FVector2f(DropMaxX - VerticalBrush->ImageSize.X, TrackPosition))),
			VerticalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		LayerId++;
	}
	else
	{
		// AboveLayer/BelowLayer - use row-style drop indicators
		const FTableRowStyle* RowStyle = &FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");

		const FSlateBrush* Brush = (Preview.Zone == FAnimMixerDropPreview::EZone::AboveLayer)
			? &RowStyle->DropIndicator_Above
			: &RowStyle->DropIndicator_Below;

		FVector2D LocalSize(TrackAreaGeometry.GetLocalSize().X, PhysicalHeight);
		FVector2D LocalPosition(0.0f, PhysicalTop.Y);

		FGeometry LayerGeometry = TrackAreaGeometry.MakeChild(
			LocalSize,
			FSlateLayoutTransform(LocalPosition)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			LayerGeometry.ToPaintGeometry(),
			Brush,
			ESlateDrawEffect::None,
			Brush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}

	return LayerId;
}

TOptional<FAnimMixerDropPreview> FAnimationMixerTrackModel::ComputeDropPreview(const FSectionVerticalDragContext& Context) const
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
	if (!Context.Section || !Context.DraggedSections || !MixerTrack)
	{
		return TOptional<FAnimMixerDropPreview>();
	}

	// Some sections cannot be moved vertically - they must stay on the same row as from/to sections
	for (UMovieSceneSection* Section : *Context.DraggedSections)
	{
		// Skeletal animation sections do not implement the mixer item interface, but are still draggable
		const UMovieSceneSkeletalAnimationSection* SkeletalAnimationSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);

		// If the section is not a skeletal animation section, check to see if it is a mixer item that supports dragging.
		const IMovieSceneAnimationMixerItemInterface* MixerItem = Cast<IMovieSceneAnimationMixerItemInterface>(Section);
		if (!SkeletalAnimationSection && (!MixerItem || !MixerItem->SupportsVerticalDragging()))
		{
			return TOptional<FAnimMixerDropPreview>();
		}
	}

	// Compute combined frame range of all dragged sections (for outline drawing)
	TRange<FFrameNumber> DraggedSectionRange = TRange<FFrameNumber>::Empty();
	for (UMovieSceneSection* Section : *Context.DraggedSections)
	{
		if (Section)
		{
			TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (!SectionRange.IsEmpty())
			{
				DraggedSectionRange = TRange<FFrameNumber>::Hull(DraggedSectionRange, SectionRange);
			}
		}
	}

	const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = MixerTrack->GetLayers();

	// Check if ctrl key is held - allows insertion between layers
	const bool bCtrlHeld = FSlateApplication::Get().GetModifierKeys().IsControlDown();
	const int32 LastLayerIndex = Layers.Num() - 1;

	// Iterate through layer models to find hover target
	int32 LastRowIndex = -1;
	for (TSharedPtr<FViewModel> ChildNode : const_cast<FAnimationMixerTrackModel*>(this)->GetChildren())
	{
		TViewModelPtr<FAnimationMixerLayerModel> LayerModel = ChildNode->CastThisShared<FAnimationMixerLayerModel>();
		if (!LayerModel)
		{
			continue;
		}

		int32 CurrentRowIndex = LayerModel->GetRowIndex();

		FVirtualGeometry ChildVirtualGeometry;
		if (IGeometryExtension* ChildGeometryExtension = ChildNode->CastThis<IGeometryExtension>())
		{
			ChildVirtualGeometry = ChildGeometryExtension->GetVirtualGeometry();
		}

		float VirtualChildTop = ChildVirtualGeometry.Top;
		float VirtualChildBottom = ChildVirtualGeometry.NestedBottom;
		float VirtualChildHeight = VirtualChildBottom - VirtualChildTop;

		// Check if above all rows - always allowed (new layer at top)
		if (LastRowIndex == -1 && (Context.VirtualMousePos.Y <= VirtualChildTop || Context.LocalMousePos.Y <= 0))
		{
			return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::AboveLayer, 0, VirtualChildTop, VirtualChildBottom };
		}
		else if (Context.VirtualMousePos.Y < VirtualChildBottom)
		{
			// Mouse is within this layer's bounds
			bool bIsChildTrackRow = IsRowOccupiedByChildTrack(CurrentRowIndex);

			const float ZoneBoundary = FMath::Clamp(VirtualChildHeight * 0.25f, 3.0f, 10.0f);
			float LocalMouseY = Context.VirtualMousePos.Y - VirtualChildTop;

			if (LocalMouseY < ZoneBoundary)
			{
				// Top 25% zone- only use insertion drop zone in ctrl insertion case
				if (bCtrlHeld)
				{
					return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::AboveLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom };
				}
				else
				{
					// Without ctrl, drop onto this layer instead
					if (bIsChildTrackRow)
					{
						// Can't drop onto child track - no preview
						return TOptional<FAnimMixerDropPreview>();
					}
					return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::OntoLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom, DraggedSectionRange };
				}
			}
			else if (LocalMouseY > VirtualChildHeight - ZoneBoundary)
			{
				// Bottom 25% zone- only use insertion drop zones in ctrl insertion case
				if (bCtrlHeld)
				{
					return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::BelowLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom };
				}
				else
				{
					// Without ctrl, drop onto this layer instead
					if (bIsChildTrackRow)
					{
						// Can't drop onto child track - no preview
						return TOptional<FAnimMixerDropPreview>();
					}
					return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::OntoLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom, DraggedSectionRange };
				}
			}
			else
			{
				// Middle 50%
				if (bIsChildTrackRow)
				{
					// Can't drop onto child track
					// If ctrl is held, allow insert below, otherwise no preview
					if (bCtrlHeld || CurrentRowIndex == LastLayerIndex)
					{
						return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::BelowLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom };
					}
					return TOptional<FAnimMixerDropPreview>();
				}
				else
				{
					// Drop onto section layer
					return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::OntoLayer, CurrentRowIndex, VirtualChildTop, VirtualChildBottom, DraggedSectionRange };
				}
			}
		}

		LastRowIndex = CurrentRowIndex;
	}

	// Below all rows - always allowed (new layer at bottom)
	if (Context.VirtualMousePos.Y >= 0 && LastRowIndex >= 0)
	{
		// Get geometry of last layer
		FVirtualGeometry LastGeometry;
		for (TSharedPtr<FViewModel> ChildNode : const_cast<FAnimationMixerTrackModel*>(this)->GetChildren())
		{
			TViewModelPtr<FAnimationMixerLayerModel> LayerModel = ChildNode->CastThisShared<FAnimationMixerLayerModel>();
			if (LayerModel && LayerModel->GetRowIndex() == LastRowIndex)
			{
				if (IGeometryExtension* ChildGeometryExtension = ChildNode->CastThis<IGeometryExtension>())
				{
					LastGeometry = ChildGeometryExtension->GetVirtualGeometry();
				}
				break;
			}
		}
		return FAnimMixerDropPreview{ FAnimMixerDropPreview::EZone::BelowLayer, LastRowIndex, LastGeometry.Top, LastGeometry.NestedBottom };
	}

	return TOptional<FAnimMixerDropPreview>();
}

bool FAnimationMixerTrackModel::ApplyDropPreview(const FAnimMixerDropPreview& Preview, const FSectionVerticalDragContext& Context)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(GetTrack());
	if (!Context.DraggedSections || !Context.InitialSectionRowIndices || !MixerTrack)
	{
		return false;
	}

	int32 TargetRowIndex = Preview.TargetRowIndex;
	bool bInsertMode = (Preview.Zone == FAnimMixerDropPreview::EZone::AboveLayer ||
	                    Preview.Zone == FAnimMixerDropPreview::EZone::BelowLayer);

	// Adjust target index for "below" insertions
	if (Preview.Zone == FAnimMixerDropPreview::EZone::BelowLayer)
	{
		TargetRowIndex += 1;
	}

	// Check if sections are already at target row
	bool bNeedsRowChange = false;
	for (UMovieSceneSection* Section : *Context.DraggedSections)
	{
		if (Section && (Section->GetRowIndex() != TargetRowIndex || bInsertMode))
		{
			bNeedsRowChange = true;
			break;
		}
	}

	if (!bNeedsRowChange)
	{
		return false;
	}

	const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& Layers = MixerTrack->GetLayers();

	// Track all affected row indices for transition updates
	TSet<int32> AffectedRows;

	// For non-insert mode, add target row now. For insert mode, we add it after shift adjustment
	// because the destination row index doesn't change (it's the newly inserted row).
	if (!bInsertMode)
	{
		AffectedRows.Add(TargetRowIndex);
	}

	if (bInsertMode)
	{
		// Remove sections from source layers before shifting
		for (UMovieSceneSection* Section : *Context.DraggedSections)
		{
			if (Section)
			{
				int32 SourceRowIndex = Section->GetRowIndex();
				AffectedRows.Add(SourceRowIndex);
				if (SourceRowIndex >= 0 && SourceRowIndex < Layers.Num())
				{
					UMovieSceneAnimationMixerLayer* SourceLayer = Layers[SourceRowIndex];
					if (SourceLayer)
					{
						SourceLayer->Modify();
						SourceLayer->RemoveSection(Section);
					}
				}
			}
		}

		// Insert new layer and shift
		ShiftRowsDownFromIndex(TargetRowIndex, *Context.DraggedSections);

		// Add sections to target layer after shift
		const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& LayersAfterShift = MixerTrack->GetLayers();
		if (TargetRowIndex < LayersAfterShift.Num())
		{
			if (UMovieSceneAnimationMixerLayer* TargetLayer = LayersAfterShift[TargetRowIndex])
			{
				for (UMovieSceneSection* Section : *Context.DraggedSections)
				{
					if (Section)
					{
						Section->Modify();
						TargetLayer->Modify();
						if (!TargetLayer->ContainsSection(Section))
						{
							TargetLayer->AddSection(Section);
						}
						Section->SetRowIndex(TargetRowIndex);
					}
				}
			}
		}

		// After shift, source rows have shifted - update affected rows to post-shift indices
		// All source rows at or above TargetRowIndex have been shifted down by 1
		TSet<int32> ShiftedAffectedRows;
		for (int32 RowIndex : AffectedRows)
		{
			if (RowIndex >= TargetRowIndex)
			{
				ShiftedAffectedRows.Add(RowIndex + 1);
			}
			else
			{
				ShiftedAffectedRows.Add(RowIndex);
			}
		}
		AffectedRows = MoveTemp(ShiftedAffectedRows);

		// Now add the destination row - it doesn't shift because it's the newly inserted row
		AffectedRows.Add(TargetRowIndex);
	}
	else
	{
		// Overlap mode - move sections to target layer
		if (TargetRowIndex < Layers.Num())
		{
			if (UMovieSceneAnimationMixerLayer* TargetLayer = Layers[TargetRowIndex])
			{
				for (UMovieSceneSection* Section : *Context.DraggedSections)
				{
					if (Section && Section->GetRowIndex() != TargetRowIndex)
					{
						Section->Modify();

						// Remove from source layer
						int32 SourceRowIndex = Section->GetRowIndex();
						AffectedRows.Add(SourceRowIndex);
						if (SourceRowIndex >= 0 && SourceRowIndex < Layers.Num())
						{
							if (UMovieSceneAnimationMixerLayer* SourceLayer = Layers[SourceRowIndex])
							{
								SourceLayer->Modify();
								SourceLayer->RemoveSection(Section);
							}
						}

						// Add to target layer
						TargetLayer->Modify();
						if (!TargetLayer->ContainsSection(Section))
						{
							TargetLayer->AddSection(Section);
						}
						Section->SetRowIndex(TargetRowIndex);
					}
				}
			}
		}
	}

	// Update transitions for all affected rows
	MixerTrack->UpdateTransitionsForRows(AffectedRows);

	// Force UI update after applying the drop
	ForceUpdate();

	// ForceUpdate rebuilds layer view models in place but doesn't propagate section
	// row additions/removals to the outliner tree. Re-run each affected layer's
	// expansion-state path (the same path the user-facing collapse/re-expand action
	// takes) so the outliner picks up the change. 
	{
		const TArray<TObjectPtr<UMovieSceneAnimationMixerLayer>>& CurrentLayers = MixerTrack->GetLayers();
		FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
		for (int32 AffectedRow : AffectedRows)
		{
			if (AffectedRow < 0 || AffectedRow >= CurrentLayers.Num())
			{
				continue;
			}
			UMovieSceneAnimationMixerLayer* AffectedLayer = CurrentLayers[AffectedRow];
			if (!AffectedLayer)
			{
				continue;
			}
			for (TViewModelPtr<FAnimationMixerLayerModel> LayerModel : OutlinerChildren.IterateSubList<FAnimationMixerLayerModel>())
			{
				if (LayerModel && LayerModel->GetLayer() == AffectedLayer)
				{
					LayerModel->SetExpansion(LayerModel->IsExpanded());
					break;
				}
			}
		}
	}

	return true;
}

} // namespace UE::Sequencer
