// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixerLinkedAnimTrackProvider.h"
#include "AnimMixerBakeHelper.h"
#include "MovieSceneAnimationMixerTrackEditor.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "Exporters/AnimSeqExportOption.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Baker/SequencerBakerSubsystem.h"
#include "Baker/SequencerBaker.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "MixerLinkedAnimTrackProvider"

using namespace UE::Sequencer;

static bool DisableMixerTrackHelper(UMovieSceneSignedObject* InObject,  bool bInEvalDisabled)
{
	bool bWasChanged = false;
	if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(InObject))
	{
		if (Track->GetMaxRowIndex() == 0)
		{
			if (Track->IsEvalDisabled() != bInEvalDisabled)
			{
				bWasChanged = true;
				Track->Modify();
				Track->SetEvalDisabled(bInEvalDisabled);
			}
		}
		else
		{
			for (int32 RowIndex = 0; RowIndex <= Track->GetMaxRowIndex(); ++RowIndex)
			{
				if (Track->IsRowEvalDisabled(RowIndex) != bInEvalDisabled)
				{
					bWasChanged = true;
					Track->Modify();
					Track->SetRowEvalDisabled(bInEvalDisabled, RowIndex);
				}
			}
		}
	}
	else if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(InObject))
	{
		if (UMovieSceneTrack* SectionParent = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			int32 RowIndex = Section->GetRowIndex();
			if (SectionParent->IsRowEvalDisabled(RowIndex) != bInEvalDisabled)
			{
				bWasChanged = true;
				SectionParent->Modify();
				SectionParent->SetRowEvalDisabled(bInEvalDisabled, RowIndex);
			}
		}
	}
	return bWasChanged;
}

bool FMixerLinkedAnimTrackProvider::HasMixerTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || !SequencerPtr->GetFocusedMovieSceneSequence() || !SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return false;
	}
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	TArray<UMovieSceneTrack*> MixerTracks = MovieScene->FindTracks(UMovieSceneAnimationMixerTrack::StaticClass(), InBindingID, NAME_None);
	for (UMovieSceneTrack* Track : MixerTracks)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
		{
			return true;
		}
	}
	return false;
}

TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> FMixerLinkedAnimTrackProvider::FindMixerTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const
{
	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> Pair(nullptr, nullptr);
		
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || !SequencerPtr->GetFocusedMovieSceneSequence() || !SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return Pair;
	}

	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, InBindingID);
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	TArray<UMovieSceneTrack*> MixerTracks = MovieScene->FindTracks(UMovieSceneAnimationMixerTrack::StaticClass(), InBindingID, NAME_None);

	for (UMovieSceneTrack* Track : MixerTracks)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
		{
			if (UMovieSceneSkeletalAnimationTrack* ChildTrack = FindLinkedChildTrack(MixerTrack, LinkedItems))
			{
				TArray<UMovieSceneSection*> Sections = ChildTrack->GetAllSections();
				for (int32 Index = 0; Index < LinkedItems.Num(); ++Index)
				{
					FLevelSequenceAnimSequenceLinkItem* Item = LinkedItems[Index];
					if (Item && Item->bAutoBake == true)
					{
						UAnimSequence* LinkedAnimSequence = Item->ResolveAnimSequence();
						for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
						{
							if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Sections[SectionIndex]))
							{
								if (LinkedAnimSequence && LinkedAnimSequence == AnimSection->Params.Animation)
								{
									Pair.Key = MixerTrack;
									Pair.Value = AnimSection;
									return Pair;
								}
							}
						}
					}
				}
			}
			if (Pair.Key == nullptr) //pick first one
			{
				Pair.Key = MixerTrack;
			}
		}
	}
	return Pair;
}

UMovieSceneSkeletalAnimationTrack* FMixerLinkedAnimTrackProvider::FindLinkedChildTrack(
	UMovieSceneAnimationMixerTrack* MixerTrack,
	const TArray<FLevelSequenceAnimSequenceLinkItem*>& LinkedItems) const
{
	TArray<TObjectPtr<UMovieSceneTrack>> ChildTracks;
	MixerTrack->GetAllChildTracks(ChildTracks);

	for (const TObjectPtr<UMovieSceneTrack>& ChildTrack : ChildTracks)
	{
		if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(ChildTrack.Get()))
		{
			const TArray<UMovieSceneSection*>& Sections = AnimTrack->GetAllSections();
			for (FLevelSequenceAnimSequenceLinkItem* Item : LinkedItems)
			{
				if (Item && Item->bAutoBake)
				{
					UAnimSequence* LinkedAnimSequence = Item->ResolveAnimSequence();
					for (UMovieSceneSection* Section : Sections)
					{
						if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
						{
							if (LinkedAnimSequence && LinkedAnimSequence == AnimSection->Params.Animation)
							{
								return AnimTrack;
							}
						}
					}
				}
			}
		}
	}
	return nullptr;
}

bool FMixerLinkedAnimTrackProvider::CanHandleBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const
{
	return HasMixerTrack(InSequencer, InBindingID);
}

//this function expects a valid mixer(anim skel mesh) track WITH the linked anim sequence, which will only happen if the section(value) is valid
UMovieSceneTrack* FMixerLinkedAnimTrackProvider::GetLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const
{
	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> MixerTrack = FindMixerTrack(InSequencer, InBindingID);
	if (MixerTrack.Value)
	{
		if (UMovieSceneTrack* Track = MixerTrack.Value->GetTypedOuter<UMovieSceneTrack>())
		{
			return Track;
		}
	}
	return nullptr;
}

UMovieSceneTrack* FMixerLinkedAnimTrackProvider::GetOrCreateLinkedAnimTrack(
	const TWeakPtr<ISequencer>& InSequencer,
	FGuid InBindingID,
	FCommonAnimationTrackEditor& InTrackEditor)
{

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || !SequencerPtr->GetFocusedMovieSceneSequence() || !SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return nullptr;
	}

	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> MixerTrackSection = FindMixerTrack(InSequencer, InBindingID);
	UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrackSection.Key;
	if (MixerTrack == nullptr)
	{
		return nullptr; //shouldn't have we have at least one mixer track
	}
	if (MixerTrackSection.Value) //has section, return track
	{
		return MixerTrackSection.Key;
	}


	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	FScopedTransaction Transaction(LOCTEXT("CreateMixerLinkedAnimTrack_Transaction", "Create Mixer Linked Anim Track"));
	MovieScene->Modify();

	// Create a child skeletal animation track inside the mixer
	int32 NextRow = MixerTrack->ComputeNextAvailableRowIndex();
	UMovieSceneTrack* NewChildTrack = MixerTrack->AddChildTrack(InBindingID, UMovieSceneSkeletalAnimationTrack::StaticClass(), NextRow);
	UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(NewChildTrack);
	if (!AnimTrack)
	{
		return nullptr;
	}

	AnimTrack->Modify();
	const FString TrackName = TEXT("(Linked)");
	AnimTrack->SetDisplayName(FText::FromString(TrackName));
	AnimTrack->SetColorTint(UMovieSceneSkeletalAnimationTrack::LinkedAnimSeqTrackColor);
	USkeletalMeshComponent* SkelMeshComp = FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(InBindingID, SequencerPtr);
	USkeleton* Skeleton = FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(InBindingID, SequencerPtr);

	// Check if we already have a linked anim item to use, or need to create one
	bool bNeedToCreateLinkedAnim = true;
	int32 LinkedItemNum = INDEX_NONE;
	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(SequencerPtr, InBindingID);
	if (LinkedItems.Num() > 0)
	{
		for (int32 LinkedIndex = 0; LinkedIndex < LinkedItems.Num(); ++LinkedIndex)
		{
			FLevelSequenceAnimSequenceLinkItem* LinkedItem = LinkedItems[LinkedIndex];
			if (LinkedItem && LinkedItem->bUseCustomTimeRange == false)
			{
				bNeedToCreateLinkedAnim = false;
				LinkedItemNum = LinkedIndex;
			}
		}
	}
	if (bNeedToCreateLinkedAnim)
	{
		InTrackEditor.CreateLinkedAnimationSequence(SkelMeshComp, Skeleton, InBindingID, bNeedToCreateLinkedAnim);
		LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(SequencerPtr, InBindingID);
		LinkedItemNum = LinkedItems.Num() - 1;
	}

	if (LinkedItems.Num() > 0 && LinkedItemNum != INDEX_NONE)
	{
		if (FLevelSequenceAnimSequenceLinkItem* LinkedItem = LinkedItems[LinkedItemNum])
		{
			UAnimSequence* AnimSequence = LinkedItem->ResolveAnimSequence();
			TOptional<TRange<FFrameNumber>> OptionalRange = SequencerPtr->GetSubSequenceRange();
			TRange<FFrameNumber> Range = OptionalRange.IsSet() ? OptionalRange.GetValue() : SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
			FFrameNumber Frame = Range.GetLowerBoundValue();
			UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(AnimTrack->AddNewAnimationOnRow(Frame, AnimSequence, 0));
			if (NewSection)
			{
				LinkedItem->bUseCustomTimeRange = false;
				LinkedItem->bAutoBake = true;
				NewSection->SetRange(Range);
				NewChildTrack->SetEvalDisabled(true);
				return NewChildTrack;
			}
		}
	}

	return nullptr;
}

bool FMixerLinkedAnimTrackProvider::IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID, bool bIsolate)
{
	bool bWasChanged = false;

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || !SequencerPtr->GetFocusedMovieSceneSequence() || !SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return bWasChanged;
	}

	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, InBindingID);
	if (LinkedItems.Num() == 0)
	{
		return bWasChanged;
	}

	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	TArray<UMovieSceneTrack*> AllTracks = MovieScene->FindTracks(UMovieSceneTrack::StaticClass(), InBindingID, NAME_None);

	// Find which MixerTrack contains the linked anim child track, and find the child track
	UMovieSceneAnimationMixerTrack* LinkedMixerTrack = nullptr;
	UMovieSceneSkeletalAnimationTrack* ChildMixerTrack = nullptr;
	for (UMovieSceneTrack* Track : AllTracks)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
		{
			ChildMixerTrack = FindLinkedChildTrack(MixerTrack, LinkedItems);
			if(ChildMixerTrack)
			{
				LinkedMixerTrack = MixerTrack;
				break;
			}
		}
	}

	for (UMovieSceneTrack* Track : AllTracks)
	{
		if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track))
		{
			//main mixer track, need to disable it's sections when isolation
			if (MixerTrack == LinkedMixerTrack)
			{
				TArray<UMovieSceneSection*> Sections = MixerTrack->GetAllSections();
				for (UMovieSceneSection* Section : Sections)
				{
					if (DisableMixerTrackHelper(Section, bIsolate))
					{
						bWasChanged = true;
					}
				}
			}
			if (MixerTrack != LinkedMixerTrack)
			{
				// Other MixerTracks: disable when isolating, enable when un-isolating
				if (DisableMixerTrackHelper(MixerTrack, bIsolate))
				{
					bWasChanged = true;
				}
			}
		}
		else if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
		{
			// Internal Mixer AnimTrack we enable when isolating
			if (ChildMixerTrack == AnimTrack)
			{
				if (DisableMixerTrackHelper(AnimTrack, !bIsolate))
				{
					bWasChanged = true;
				}
			}
			// Standalone anim tracks (not inside a mixer): disable when isolating
			else if (DisableMixerTrackHelper(AnimTrack, bIsolate))
			{
				bWasChanged = true;
			}
		}
		else
		{
			// Control rig and other tracks: disable when isolating
			FString TrackName = Track->GetClass()->GetName();
			if (TrackName == FString("MovieSceneControlRigParameterTrack"))
			{
				if (DisableMixerTrackHelper(Track, bIsolate))
				{
					bWasChanged = true;
				}
			}
		}
	}

	if (USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>())
	{
		if (TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr))
		{
			SequencerBaker->IgnoreLastChange();
		}
	}

	return bWasChanged;
}

void FMixerLinkedAnimTrackProvider::UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (!LevelSequence || !LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		return;
	}

	TOptional<TRange<FFrameNumber>> OptionalRange = SequencerPtr->GetSubSequenceRange();
	TRange<FFrameNumber> Range = OptionalRange.IsSet() ? OptionalRange.GetValue() : SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();

	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, InBindingID);
	if (LinkedItems.Num() == 0)
	{
		return;
	}

	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> MixerTrackSection = FindMixerTrack(InSequencer, InBindingID);

	UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrackSection.Key;
	if (!MixerTrack)
	{
		return;
	}
	
	if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(MixerTrackSection.Value))
	{
		if (AnimSection->GetRange() != Range)
		{
			AnimSection->Modify();
			AnimSection->SetRange(Range);
		}

		if (UAnimSequenceBase* Animation = AnimSection->Params.Animation)
		{
			if (Animation->HasRootMotion() && !AnimSection->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
			{
				AnimSection->Modify();
				UMovieSceneRootMotionSettingsDecoration* RootMotionSettings = NewObject<UMovieSceneRootMotionSettingsDecoration>(
					AnimSection, UMovieSceneRootMotionSettingsDecoration::StaticClass(), NAME_None, RF_Transactional);
				AnimSection->AddDecoration(RootMotionSettings);
			}
		}
	}
}

bool FMixerLinkedAnimTrackProvider::TryBakeLinkedAnimSequence(
	const TWeakPtr<ISequencer>& InSequencer,
	FGuid InBindingID,
	UAnimSequence* InAnimSequence,
	const FLevelSequenceAnimSequenceLinkItem& InLinkItem)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || !InAnimSequence)
	{
		return false;
	}

	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> MixerTrackPair = FindMixerTrack(InSequencer, InBindingID);
	UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrackPair.Key;
	if (!MixerTrack)
	{
		return false;
	}

	USkeletalMeshComponent* SkelMeshComp = FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(InBindingID, SequencerPtr);
	if (!SkelMeshComp)
	{
		return false;
	}

	UAnimSeqExportOption* ExportOptions = NewObject<UAnimSeqExportOption>(GetTransientPackage(), NAME_None);
	ExportOptions->bExportTransforms = InLinkItem.bExportTransforms;
	ExportOptions->bExportMorphTargets = InLinkItem.bExportMorphTargets;
	ExportOptions->bExportAttributeCurves = InLinkItem.bExportAttributeCurves;
	ExportOptions->bExportMaterialCurves = InLinkItem.bExportMaterialCurves;
	ExportOptions->bSkipCurvesWithZeroValue = InLinkItem.bSkipCurvesWithZeroValue;
	ExportOptions->bRecordInWorldSpace = InLinkItem.bRecordInWorldSpace;
	ExportOptions->bEvaluateAllSkeletalMeshComponents = InLinkItem.bEvaluateAllSkeletalMeshComponents;
	ExportOptions->bSetRetargetSourceAsset = InLinkItem.bSetRetargetSourceAsset;
	ExportOptions->Interpolation = InLinkItem.Interpolation;
	ExportOptions->CurveInterpolation = InLinkItem.CurveInterpolation;
	ExportOptions->IncludeAnimationNames = InLinkItem.IncludeAnimationNames;
	ExportOptions->ExcludeAnimationNames = InLinkItem.ExcludeAnimationNames;
	ExportOptions->bRemoveExcludedCurves = InLinkItem.bRemoveExcludedCurves;
	ExportOptions->WarmUpFrames = InLinkItem.WarmUpFrames;
	ExportOptions->DelayBeforeStart = InLinkItem.DelayBeforeStart;
	ExportOptions->bUseCustomTimeRange = InLinkItem.bUseCustomTimeRange;
	ExportOptions->CustomStartFrame = InLinkItem.CustomStartFrame;
	ExportOptions->CustomEndFrame = InLinkItem.CustomEndFrame;
	ExportOptions->CustomDisplayRate = InLinkItem.CustomDisplayRate;
	ExportOptions->bUseCustomFrameRate = InLinkItem.bUseCustomFrameRate;
	ExportOptions->CustomFrameRate = InLinkItem.CustomFrameRate;
	ExportOptions->bTransactRecording = false;

	const bool bResult = UE::Sequencer::AnimMixerBake::ExportMixerToAnimSequence(
		SequencerPtr, InAnimSequence, ExportOptions, SkelMeshComp, MixerTrack);

	ExportOptions->MarkAsGarbage();
	return bResult;
}

bool FMixerLinkedAnimTrackProvider::DeleteLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID)
{
	TPair<UMovieSceneAnimationMixerTrack*, UMovieSceneSection*> MixerTrackPair = FindMixerTrack(InSequencer, InBindingID);
	UMovieSceneAnimationMixerTrack* MixerTrack = MixerTrackPair.Key;
	UMovieSceneSection* Section = MixerTrackPair.Value;

	if (MixerTrack && Section)
	{
		if (UMovieSceneTrack* ChildTrack = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			MixerTrack->RemoveChildTrack(ChildTrack);
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
