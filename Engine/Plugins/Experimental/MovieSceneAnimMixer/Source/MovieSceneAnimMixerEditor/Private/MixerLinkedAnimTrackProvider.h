// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneLinkedAnimTrackProvider.h"

namespace UE::Sequencer { class FAnimationMixerTrackEditor; }

class UMovieSceneAnimationMixerTrack;
class UMovieSceneSection;
class UMovieSceneSkeletalAnimationTrack;

/**
 * Linked animation track provider for the Animation Mixer.
 * When a binding has a UMovieSceneAnimationMixerTrack, this provider
 * creates the linked anim track as a child track inside the mixer
 * and handles isolation by toggling mixer tracks rather than individual tracks.
 */
class FMixerLinkedAnimTrackProvider : public IMovieSceneLinkedAnimTrackProvider
{
public:
	FMixerLinkedAnimTrackProvider(UE::Sequencer::FAnimationMixerTrackEditor* InTrackEditor)
		: TrackEditor(InTrackEditor)
	{}

	//~ IMovieSceneLinkedAnimTrackProvider interface
	virtual int32 GetLinkedAnimTrackPriority() const override { return 50; }
	virtual bool CanHandleBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const override;
	virtual UMovieSceneTrack* GetLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InInBindingID) const override;
	virtual UMovieSceneTrack* GetOrCreateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID, UE::Sequencer::FCommonAnimationTrackEditor& TrackEditor) override;
	virtual bool IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID, bool bIsolate) override;
	virtual void UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) override;
	virtual bool DeleteLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) override;
	virtual bool TryBakeLinkedAnimSequence(
		const TWeakPtr<ISequencer>& InSequencer,
		FGuid InBindingID,
		UAnimSequence* InAnimSequence,
		const FLevelSequenceAnimSequenceLinkItem& InLinkItem) override;

private:
	/** Whether or not it has a mixer track*/
	bool HasMixerTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InInBindingID) const;
	/** Find the MixerTrack on the binding and associated section, if any. It may return a mixer track with no section, which means
	we still need to create the section*/
	TPair<UMovieSceneAnimationMixerTrack*,UMovieSceneSection*> FindMixerTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid InBindingID) const;
	/** Find the child skeletal animation track inside a MixerTrack that contains the linked anim sequence. */
	UMovieSceneSkeletalAnimationTrack* FindLinkedChildTrack(
		UMovieSceneAnimationMixerTrack* MixerTrack,
		const TArray<struct FLevelSequenceAnimSequenceLinkItem*>& LinkedItems) const;

	UE::Sequencer::FAnimationMixerTrackEditor* TrackEditor = nullptr;
};
