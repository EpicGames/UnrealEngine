// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixerRootMotionTrail.h"

#include "AnimationMixerTrackEditMode.h"
#include "EditorModeManager.h"
#include "MixerTrailKeyTool.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Tools/MotionTrailOptions.h"
#include "Components/SkeletalMeshComponent.h"
#include "ISequencer.h"
#include "PrimitiveDrawInterface.h"

#define LOCTEXT_NAMESPACE "MixerRootMotionTrail"

namespace UE::Sequencer
{

using namespace UE::AIE;
using namespace UE::SequencerAnimTools;

FMixerRootMotionTrail::FMixerRootMotionTrail(
	UMovieSceneAnimationMixerTrack* InMixerTrack,
	TWeakPtr<ISequencer> InSequencer)
	: FTrail(InMixerTrack)
	, WeakMixerTrack(InMixerTrack)
	, WeakSequencer(InSequencer)
{
	CalculatedArrayOfTransforms = MakeShared<FArrayOfTransforms>();
	ArrayOfTransforms = MakeShared<FArrayOfTransforms>();

	// Parent space transforms are not used for root motion trails but FTrajectoryDrawInfo requires the parameter
	TSharedPtr<FArrayOfTransforms> DummyParentTransforms = MakeShared<FArrayOfTransforms>();

	DrawInfo = MakeUnique<FTrajectoryDrawInfo>(
		UMotionTrailToolOptions::GetTrailOptions()->TrailStyle,
		UMotionTrailToolOptions::GetTrailOptions()->DefaultColor,
		ArrayOfTransforms,
		DummyParentTransforms);

	KeyTool = MakeUnique<FMixerTrailKeyTool>(this);
}

FMixerRootMotionTrail::~FMixerRootMotionTrail() = default;

FTrailCurrentStatus FMixerRootMotionTrail::UpdateTrail(const FNewSceneContext& NewSceneContext)
{
	FTrailCurrentStatus Status;

	if (!WeakMixerTrack.IsValid())
	{
		Status.CacheState = ETrailCacheState::Dead;
		return Status;
	}

	if (bForceEvaluateNextTick)
	{
		Status.CacheState = ETrailCacheState::Stale;
		bForceEvaluateNextTick = false;
	}
	else
	{
		Status.CacheState = ETrailCacheState::UpToDate;
	}

	CacheState = Status.CacheState;
	return Status;
}

void FMixerRootMotionTrail::UpdateFinished(const TRange<FFrameNumber>& UpdatedRange, const TArray<int32>& IndicesToCalcluate, bool bDoneCalcuating)
{
	if (CalculatedArrayOfTransforms.IsValid() && ArrayOfTransforms.IsValid())
	{
		// Ensure the drawing array matches the calculated array size
		if (ArrayOfTransforms->Num() != CalculatedArrayOfTransforms->Num())
		{
			ArrayOfTransforms->SetNum(CalculatedArrayOfTransforms->Num());
		}

		for (int32 Index : IndicesToCalcluate)
		{
			if (Index >= 0 && Index < CalculatedArrayOfTransforms->Num())
			{
				ArrayOfTransforms->Transforms[Index] = CalculatedArrayOfTransforms->Transforms[Index];
			}
		}
	}

	CachedDrawData.PointsToDraw.SetNum(0);
	CacheState = ETrailCacheState::UpToDate;

	if (bDoneCalcuating && !bIsTracking)
	{
		KeyTool->BuildKeys();
	}
}

FText FMixerRootMotionTrail::GetName() const
{
	if (UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get())
	{
		return MixerTrack->GetDisplayName();
	}
	return LOCTEXT("InvalidTrail", "Invalid Mixer Trail");
}

void FMixerRootMotionTrail::Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating)
{
	if (CachedDrawData.PointsToDraw.Num() < 2)
	{
		return;
	}

	const float TrailThickness = UMotionTrailToolOptions::GetTrailOptions()->TrailThickness;
	FVector LastPoint = CachedDrawData.PointsToDraw[0];

	for (int32 Idx = 1; Idx < CachedDrawData.PointsToDraw.Num(); Idx++)
	{
		const FVector CurPoint = CachedDrawData.PointsToDraw[Idx];
		const FLinearColor CurColor = (Idx - 1 < CachedDrawData.Color.Num())
			? CachedDrawData.Color[Idx - 1]
			: FLinearColor::White;
		PDI->DrawLine(LastPoint, CurPoint, CurColor, SDPG_Foreground, TrailThickness);
		LastPoint = CurPoint;
	}

	// Update key view range from the latest CurrentFramesInfo before rendering
	if (CurrentFramesInfo)
	{
		KeyTool->UpdateViewRange(CurrentFramesInfo->ViewRange);
	}

	// Render keys on top of the trail line
	KeyTool->Render(Guid, View, PDI, bTrailIsEvaluating);
}

void FMixerRootMotionTrail::GetTrajectoryPointsForDisplay(
	const FCurrentFramesInfo& InCurrentFramesInfo,
	bool bIsEvaluating,
	TArray<FVector>& OutPoints,
	TArray<FFrameNumber>& OutFrames)
{
	if (FTrajectoryDrawInfo* DI = GetDrawInfo())
	{
		DI->GetTrajectoryPointsForDisplay(
			FTransform::Identity,
			FTransform::Identity,
			InCurrentFramesInfo,
			bIsEvaluating,
			OutPoints,
			OutFrames);
	}
}

void FMixerRootMotionTrail::ReadyToDrawTrail(
	FColorState& ColorState,
	const FCurrentFramesInfo* InCurrentFramesInfo,
	bool bIsEvaluating,
	bool bIsPinned)
{
	if (!InCurrentFramesInfo)
	{
		return;
	}

	CurrentFramesInfo = InCurrentFramesInfo;

	// Update any dirty key transforms from baked data — but not during tracking,
	// where ApplyWorldDelta positions are the source of truth for the gizmo.
	if (!bIsTracking)
	{
		KeyTool->UpdateKeys();
	}

	ColorState.ReadyForTrail(bIsPinned, GetDrawInfo()->GetStyle());
	bool bCalculateColor = (ColorState.GetStyle() == EMotionTrailTrailStyle::Time || ColorState.GetStyle() == EMotionTrailTrailStyle::Dashed);

	if (InCurrentFramesInfo->CurrentFrames.Num() > 0)
	{
		if (!InCurrentFramesInfo->bViewRangeIsEvalRange
			|| CachedDrawData.PointsToDraw.Num() != CachedDrawData.Color.Num()
			|| CachedDrawData.PointsToDraw.Num() != InCurrentFramesInfo->CurrentFrames.Num()
			|| CachedDrawData.Color.Num() != InCurrentFramesInfo->CurrentFrames.Num())
		{
			GetTrajectoryPointsForDisplay(*InCurrentFramesInfo, bIsEvaluating, CachedDrawData.PointsToDraw, CachedDrawData.Frames);

			bCalculateColor = true;
		}
		if (CachedDrawData.PointsToDraw.Num() > 1 && bCalculateColor)
		{
			CachedDrawData.Color.SetNum(CachedDrawData.PointsToDraw.Num());
			for (int32 Idx = 1; Idx < CachedDrawData.PointsToDraw.Num(); Idx++)
			{
				const FFrameNumber CurFrame = CachedDrawData.Frames[Idx - 1];
				GetColor(CurFrame, ColorState);
				CachedDrawData.Color[Idx - 1] = ColorState.CalculatedColor;
			}
		}
	}
}

void FMixerRootMotionTrail::ClearCachedData()
{
	CachedDrawData.PointsToDraw.SetNum(0);
	CachedDrawData.Frames.SetNum(0);
	CachedDrawData.Color.SetNum(0);
}

bool FMixerRootMotionTrail::HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bChanged = FTrail::HandleObjectsChanged(ReplacementMap);

	if (UObject* const* NewObject = ReplacementMap.Find(WeakMixerTrack.Get()))
	{
		WeakMixerTrack = Cast<UMovieSceneAnimationMixerTrack>(*NewObject);
		bChanged = true;
	}

	for (TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>& WeakDec : Decorations)
	{
		if (UObject* const* NewDec = ReplacementMap.Find(WeakDec.Get()))
		{
			WeakDec = Cast<UMovieSceneRootMotionSettingsDecoration>(*NewDec);
			bChanged = true;
		}
	}
	if (bChanged)
	{
		TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> DecCopy = Decorations;
		KeyTool->SetDecorations(MoveTemp(DecCopy));
	}

	return bChanged;
}

bool FMixerRootMotionTrail::HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click)
{
	bool bHandled = KeyTool->HandleClick(Guid, InViewportClient, HitProxy, Click);
	if (bHandled && KeyTool->IsSelected())
	{
		// Clear the mixer edit mode's root control selection so its gizmo
		// doesn't conflict with the trail key gizmo
		if (UAnimationMixerTrackEditMode* MixerEditMode = Cast<UAnimationMixerTrackEditMode>(
				GLevelEditorModeTools().GetActiveScriptableMode(UAnimationMixerTrackEditMode::ModeName)))
		{
			MixerEditMode->SelectNone();
		}
	}
	return bHandled;
}

bool FMixerRootMotionTrail::IsAnythingSelected() const
{
	return KeyTool->IsSelected();
}

bool FMixerRootMotionTrail::IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	return KeyTool->IsSelected(OutVectorPosition, OutRotation);
}

bool FMixerRootMotionTrail::IsAnythingSelected(TArray<FVector>& OutVectorPositions) const
{
	return KeyTool->IsSelected(OutVectorPositions);
}

void FMixerRootMotionTrail::SelectNone()
{
	KeyTool->ClearSelection();
}

bool FMixerRootMotionTrail::StartTracking()
{
	if (!KeyTool->IsSelected())
	{
		return false;
	}
	bIsTracking = true;

	// Snapshot the starting offset values from the selected key's decoration
	DragState.SnapshotFromDecoration(KeyTool->GetPrimarySelectedKeyDecoration(), FFrameTime(KeyTool->GetPrimarySelectedKeyTime()));

	return true;
}

bool FMixerRootMotionTrail::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset)
{
	if (!KeyTool->IsSelected())
	{
		return false;
	}

	UMovieSceneRootMotionSettingsDecoration* Decoration = KeyTool->GetPrimarySelectedKeyDecoration();
	if (!Decoration)
	{
		return false;
	}

	TSharedPtr<ISequencer> SequencerPin = WeakSequencer.Pin();
	const EAutoChangeMode AutoChangeMode = SequencerPin ? SequencerPin->GetAutoChangeMode() : EAutoChangeMode::None;
	const bool bShouldAutoKey = (AutoChangeMode == EAutoChangeMode::AutoKey || AutoChangeMode == EAutoChangeMode::All);

	// Accumulate the drag delta in component-local space, matching the
	// edit mode's InputDelta pattern. The offset channels receive absolute
	// values: DragStartOffset + AccumulatedDelta.
	if (USkeletalMeshComponent* Comp = WeakBoundComponent.Get())
	{
		FTransform ComponentTransform = Comp->GetComponentTransform();
		DragState.AccumulatedLocalDrag += ComponentTransform.InverseTransformVector(Pos);
	}
	else
	{
		DragState.AccumulatedLocalDrag += Pos;
	}
	DragState.AccumulatedLocalRot += Rot;

	FFrameNumber KeyTime = KeyTool->GetPrimarySelectedKeyTime();

	UAnimationMixerTrackEditMode::KeyOffsetChannels(
		Decoration,
		DragState.GetCurrentLocationOffset(),
		DragState.GetCurrentRotationOffset(),
		KeyTime,
		bShouldAutoKey);

	// Invalidate offset cache so the next bake sees updated offsets
	if (UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get())
	{
		MixerTrack->InvalidateAccumulatedOffsetCache();
	}

	// Invalidate cached skeleton poses so they re-evaluate with new offsets
	if (UAnimationMixerTrackEditMode* MixerEditMode = Cast<UAnimationMixerTrackEditMode>(
			GLevelEditorModeTools().GetActiveScriptableMode(UAnimationMixerTrackEditMode::ModeName)))
	{
		MixerEditMode->InvalidateAllCachedPoses();
	}

	// Move the key dot directly so the gizmo stays in sync
	for (FFrameNumber SelectedTime : KeyTool->GetSelectedKeyTimes())
	{
		KeyTool->ApplyWorldDelta(SelectedTime, Pos, Rot);
	}

	// Notify sequencer that data changed so the entity system picks up the
	// new channel values. Change detection on the hierarchy is suppressed
	// during Update() when a rebake is pending, so this won't invalidate trails.
	if (SequencerPin)
	{
		SequencerPin->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	// Request a synchronous rebake in the hierarchy's Update(), which runs
	// at the correct lifecycle point for entity system evaluation.
	bNeedsSynchronousRebake = true;

	return true;
}

bool FMixerRootMotionTrail::EndTracking()
{
	bIsTracking = false;

	// Invalidate the accumulated offset cache so later sections pick up the
	// changed offsets
	if (UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get())
	{
		MixerTrack->InvalidateAccumulatedOffsetCache();
	}

	// Now notify and re-evaluate. During the drag we skipped this to avoid
	// the trail going stale mid-drag (the renderer skips stale trails).
	ForceEvaluateNextTick();
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	return true;
}

UMovieSceneAnimationMixerTrack* FMixerRootMotionTrail::GetMixerTrack() const
{
	return WeakMixerTrack.Get();
}

void FMixerRootMotionTrail::SetDecorations(TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>&& InDecorations)
{
	Decorations = MoveTemp(InDecorations);
	// Pass a copy since the key tool takes ownership
	TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> DecCopy = Decorations;
	KeyTool->SetDecorations(MoveTemp(DecCopy));
}

void FMixerRootMotionTrail::SetBoundComponent(USkeletalMeshComponent* InComponent)
{
	WeakBoundComponent = InComponent;
}

FTransform FMixerRootMotionTrail::GetTransformAtFrame(FFrameNumber Frame) const
{
	if (!ArrayOfTransforms.IsValid() || !CurrentFramesInfo || CurrentFramesInfo->CurrentFrames.Num() == 0)
	{
		return FTransform::Identity;
	}

	const TArray<FFrameNumber>& Frames = CurrentFramesInfo->CurrentFrames;
	const TArray<int32>& Indices = CurrentFramesInfo->TransformIndices;
	const int32 NumTransforms = ArrayOfTransforms->Num();

	if (Indices.Num() != Frames.Num() || NumTransforms == 0)
	{
		return FTransform::Identity;
	}

	auto SafeGetTransform = [&](int32 FrameIdx) -> FTransform
	{
		const int32 TransformIdx = Indices[FrameIdx];
		return (TransformIdx >= 0 && TransformIdx < NumTransforms)
			? ArrayOfTransforms->Transforms[TransformIdx]
			: FTransform::Identity;
	};

	const int32 StartIndex = Algo::LowerBound(Frames, Frame);
	if (StartIndex >= Frames.Num())
	{
		return SafeGetTransform(Frames.Num() - 1);
	}
	if (Frame == Frames[StartIndex] || (StartIndex + 1) == Frames.Num())
	{
		return SafeGetTransform(StartIndex);
	}

	const FFrameNumber Frame1 = Frames[StartIndex];
	const FFrameNumber Frame2 = Frames[StartIndex + 1];
	if (Frame1 != Frame2)
	{
		const double FrameDiff = (double)(Frame2.Value - Frame1.Value);
		const double T = (double)(Frame.Value - Frame1.Value) / FrameDiff;

		FTransform KeyAtom1 = SafeGetTransform(StartIndex);
		FTransform KeyAtom2 = SafeGetTransform(StartIndex + 1);
		KeyAtom1.NormalizeRotation();
		KeyAtom2.NormalizeRotation();

		FTransform Result;
		Result.Blend(KeyAtom1, KeyAtom2, T);
		return Result;
	}

	return SafeGetTransform(StartIndex);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
