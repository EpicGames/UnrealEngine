// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trail.h"
#include "AnimationMixerTrackEditMode.h"
#include "Tools/EvaluateSequencerTools.h"

class ISequencer;
class UMovieSceneAnimationMixerTrack;
class UMovieSceneRootMotionSettingsDecoration;
class USkeletalMeshComponent;

namespace UE::Sequencer
{

class FAnimMixerTrailHierarchy;
class FMixerTrailKeyTool;

// Custom trail for animation mixer root motion trajectory visualization.
// Uses AnimMixerBakeEvaluation to get root motion transforms rather than
// reading component transforms, so it works for all root motion destination types.
class FMixerRootMotionTrail : public UE::SequencerAnimTools::FTrail
{
public:
	FMixerRootMotionTrail(
		UMovieSceneAnimationMixerTrack* InMixerTrack,
		TWeakPtr<ISequencer> InSequencer);
	~FMixerRootMotionTrail();

	// FTrail interface
	virtual UE::SequencerAnimTools::FTrailCurrentStatus UpdateTrail(const FNewSceneContext& NewSceneContext) override;
	virtual void UpdateFinished(const TRange<FFrameNumber>& UpdatedRange, const TArray<int32>& IndicesToCalcluate, bool bDoneCalcuating) override;
	virtual FText GetName() const override;
	virtual void Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating) override;
	virtual void GetTrajectoryPointsForDisplay(const UE::SequencerAnimTools::FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating, TArray<FVector>& OutPoints, TArray<FFrameNumber>& OutFrames) override;
	virtual void ClearCachedData() override;
	virtual void ReadyToDrawTrail(UE::SequencerAnimTools::FColorState& ColorState, const UE::SequencerAnimTools::FCurrentFramesInfo* InCurrentFramesInfo, bool bIsEvaluating, bool bIsPinned) override;
	virtual bool HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap) override;

	virtual bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, UE::SequencerAnimTools::FInputClick Click) override;
	virtual bool IsAnythingSelected() const override;
	virtual bool IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation) const override;
	virtual bool IsAnythingSelected(TArray<FVector>& OutVectorPositions) const override;
	virtual void SelectNone() override;
	virtual bool StartTracking() override;
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) override;
	virtual bool EndTracking() override;

	UMovieSceneAnimationMixerTrack* GetMixerTrack() const;

	// Set the decorations whose offset channels provide key times (one per section)
	void SetDecorations(TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>&& InDecorations);

	// Set the bound skeletal mesh component for world-to-local delta conversion
	void SetBoundComponent(USkeletalMeshComponent* InComponent);

	// Set the owning hierarchy for synchronous re-bake during key drags
	void SetHierarchy(FAnimMixerTrailHierarchy* InHierarchy) { OwningHierarchy = InHierarchy; }

	// Interpolate a transform from the baked trajectory at a specific frame
	FTransform GetTransformAtFrame(FFrameNumber Frame) const;

	// Provides the transform array that the trail hierarchy populates during bake evaluation
	TSharedPtr<UE::AIE::FArrayOfTransforms>& GetCalculatedTransforms() { return CalculatedArrayOfTransforms; }

	bool NeedsSynchronousRebake() const { return bNeedsSynchronousRebake; }
	void ClearSynchronousRebakeFlag() { bNeedsSynchronousRebake = false; }
	void RequestSynchronousRebake() { bNeedsSynchronousRebake = true; }

private:
	TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakMixerTrack;
	TWeakPtr<ISequencer> WeakSequencer;
	TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> Decorations;
	TWeakObjectPtr<USkeletalMeshComponent> WeakBoundComponent;
	FAnimMixerTrailHierarchy* OwningHierarchy = nullptr;

	// Transforms computed by bake evaluation
	TSharedPtr<UE::AIE::FArrayOfTransforms> CalculatedArrayOfTransforms;
	// Transforms used for drawing (copied from calculated, may include offset)
	TSharedPtr<UE::AIE::FArrayOfTransforms> ArrayOfTransforms;

	TUniquePtr<FMixerTrailKeyTool> KeyTool;

	struct FDrawCacheData
	{
		TArray<FVector> PointsToDraw;
		TArray<FFrameNumber> Frames;
		TArray<FLinearColor> Color;
	};
	mutable FDrawCacheData CachedDrawData;

	const UE::SequencerAnimTools::FCurrentFramesInfo* CurrentFramesInfo = nullptr;
	bool bIsTracking = false;
	bool bNeedsSynchronousRebake = false;

	FOffsetDragState DragState;
};

} // namespace UE::Sequencer
