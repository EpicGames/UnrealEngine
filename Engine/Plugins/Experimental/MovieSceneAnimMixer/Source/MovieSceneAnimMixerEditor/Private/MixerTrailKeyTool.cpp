// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixerTrailKeyTool.h"

#include "MixerRootMotionTrail.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Tools/MotionTrailOptions.h"
#include "PrimitiveDrawInterface.h"
#include "SceneView.h"

namespace UE::Sequencer
{

IMPLEMENT_HIT_PROXY(HMixerTrailKeyProxy, HHitProxy);

FMixerTrailKeyTool::FMixerTrailKeyTool(FMixerRootMotionTrail* InOwningTrail)
	: OwningTrail(InOwningTrail)
	, SelectedKeysTransform(FTransform::Identity)
	, CachedViewFrameRange(TRange<FFrameNumber>(0, 0))
{
}

void FMixerTrailKeyTool::SetDecorations(TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>&& InDecorations)
{
	Decorations = MoveTemp(InDecorations);
	BuildKeys();
}

void FMixerTrailKeyTool::BuildKeys()
{
	// Save previous selection by (frame, decoration) so we can restore it
	TArray<TPair<FFrameNumber, TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>> PreviousSelection;
	for (FMixerTrailKeyInfo* Info : CachedSelection)
	{
		PreviousSelection.Emplace(Info->FrameNumber, Info->Decoration);
	}

	Keys.Reset();
	CachedSelection.Reset();

	// Collect keys from all decorations. Each (frame, decoration) pair gets its own key.
	for (const TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>& WeakDec : Decorations)
	{
		UMovieSceneRootMotionSettingsDecoration* Decoration = WeakDec.Get();
		if (!Decoration)
		{
			continue;
		}

		TSet<FFrameNumber> UniqueKeyTimes;
		for (int32 i = 0; i < 3; ++i)
		{
			for (int32 Idx = 0; Idx < Decoration->Location[i].GetNumKeys(); ++Idx)
			{
				UniqueKeyTimes.Add(Decoration->Location[i].GetTimes()[Idx]);
			}
			for (int32 Idx = 0; Idx < Decoration->Rotation[i].GetNumKeys(); ++Idx)
			{
				UniqueKeyTimes.Add(Decoration->Rotation[i].GetTimes()[Idx]);
			}
		}

		for (const FFrameNumber& Time : UniqueKeyTimes)
		{
			TUniquePtr<FMixerTrailKeyInfo> Info = MakeUnique<FMixerTrailKeyInfo>();
			Info->FrameNumber = Time;
			Info->Decoration = Decoration;
			Info->bDirty = true;
			Keys.Add(MoveTemp(Info));
		}
	}

	// Sort by frame number for consistent rendering order
	Keys.Sort([](const TUniquePtr<FMixerTrailKeyInfo>& A, const TUniquePtr<FMixerTrailKeyInfo>& B)
	{
		return A->FrameNumber < B->FrameNumber;
	});

	// Restore selection by matching (frame, decoration)
	for (const TPair<FFrameNumber, TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>& Prev : PreviousSelection)
	{
		for (const TUniquePtr<FMixerTrailKeyInfo>& Info : Keys)
		{
			if (Info->FrameNumber == Prev.Key && Info->Decoration == Prev.Value)
			{
				CachedSelection.Add(Info.Get());
				break;
			}
		}
	}
}

void FMixerTrailKeyTool::DirtyKeyTransforms()
{
	for (const TUniquePtr<FMixerTrailKeyInfo>& Info : Keys)
	{
		Info->bDirty = true;
	}
}

void FMixerTrailKeyTool::UpdateKeys()
{
	for (const TUniquePtr<FMixerTrailKeyInfo>& Info : Keys)
	{
		if (Info->bDirty)
		{
			Info->Transform = OwningTrail->GetTransformAtFrame(Info->FrameNumber);
			Info->bDirty = false;
		}
	}
	UpdateSelectedKeysTransform();
}

void FMixerTrailKeyTool::UpdateViewRange(const TRange<FFrameNumber>& InViewRange)
{
	CachedViewFrameRange = InViewRange;
}

void FMixerTrailKeyTool::Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating)
{
	const UMotionTrailToolOptions* Options = UMotionTrailToolOptions::GetTrailOptions();
	if (!Options || !Options->bShowKeys || !PDI)
	{
		ClearSelection();
		return;
	}

	float KeySize = Options->KeySize;
	const bool bHitTesting = PDI->IsHitTesting();

	// Adjust key size for perspective
	const bool bIsPerspective = (View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f);
	if (bIsPerspective)
	{
		const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
		KeySize = KeySize / ZoomFactor;
	}

	for (const TUniquePtr<FMixerTrailKeyInfo>& Info : Keys)
	{
		if (!CachedViewFrameRange.Contains(Info->FrameNumber))
		{
			continue;
		}

		const FLinearColor KeyColor = CachedSelection.Contains(Info.Get())
			? Options->SelectedKeyColor
			: Options->KeyColor;

		const FVector Location = Info->Transform.GetLocation();

		if (bHitTesting)
		{
			PDI->SetHitProxy(new HMixerTrailKeyProxy(Guid, Info.Get()));
		}

		PDI->DrawPoint(Location, KeyColor, KeySize, SDPG_MAX);

		if (bHitTesting)
		{
			PDI->SetHitProxy(nullptr);
		}
	}
}

bool FMixerTrailKeyTool::HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, UE::SequencerAnimTools::FInputClick Click)
{
	if (HMixerTrailKeyProxy* HitProxy = HitProxyCast<HMixerTrailKeyProxy>(InHitProxy))
	{
		if (HitProxy->KeyInfo && HitProxy->Guid == Guid)
		{
			if (Click.bShiftIsDown)
			{
				CachedSelection.Add(HitProxy->KeyInfo);
			}
			else if (Click.bCtrlIsDown)
			{
				if (CachedSelection.Contains(HitProxy->KeyInfo))
				{
					CachedSelection.Remove(HitProxy->KeyInfo);
				}
				else
				{
					CachedSelection.Add(HitProxy->KeyInfo);
				}
			}
			else
			{
				CachedSelection.Reset();
				CachedSelection.Add(HitProxy->KeyInfo);
			}
			UpdateSelectedKeysTransform();
			return true;
		}
	}

	if (!Click.bShiftIsDown && !Click.bCtrlIsDown)
	{
		CachedSelection.Reset();
	}
	return false;
}

bool FMixerTrailKeyTool::IsSelected() const
{
	return CachedSelection.Num() > 0;
}

bool FMixerTrailKeyTool::IsSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	if (CachedSelection.Num() > 0)
	{
		UpdateSelectedKeysTransform();
		OutVectorPosition = SelectedKeysTransform.GetLocation();
		OutRotation = SelectedKeysTransform.GetRotation();
		return true;
	}
	return false;
}

bool FMixerTrailKeyTool::IsSelected(TArray<FVector>& OutVectorPositions) const
{
	if (CachedSelection.Num() > 0)
	{
		for (FMixerTrailKeyInfo* Info : CachedSelection)
		{
			OutVectorPositions.Add(Info->Transform.GetLocation());
		}
		return true;
	}
	return false;
}

void FMixerTrailKeyTool::ClearSelection()
{
	CachedSelection.Reset();
}

TArray<FFrameNumber> FMixerTrailKeyTool::GetSelectedKeyTimes() const
{
	TArray<FFrameNumber> Times;
	for (FMixerTrailKeyInfo* Info : CachedSelection)
	{
		Times.Add(Info->FrameNumber);
	}
	return Times;
}

FFrameNumber FMixerTrailKeyTool::GetPrimarySelectedKeyTime() const
{
	if (CachedSelection.Num() > 0)
	{
		// Return the first selected key's frame
		return (*CachedSelection.FindArbitraryElement())->FrameNumber;
	}
	return FFrameNumber(0);
}

UMovieSceneRootMotionSettingsDecoration* FMixerTrailKeyTool::GetPrimarySelectedKeyDecoration() const
{
	if ( FMixerTrailKeyInfo* const* Info = CachedSelection.FindArbitraryElement())
	{
		return (*Info)->Decoration.Get();
	}
	return nullptr;
}

void FMixerTrailKeyTool::ApplyWorldDelta(FFrameNumber KeyFrame, const FVector& WorldPosDelta, const FRotator& WorldRotDelta)
{
	for (const TUniquePtr<FMixerTrailKeyInfo>& Info : Keys)
	{
		if (Info->FrameNumber == KeyFrame)
		{
			FVector Location = Info->Transform.GetLocation() + WorldPosDelta;
			Info->Transform.SetLocation(Location);
			break;
		}
	}
	UpdateSelectedKeysTransform();
}

void FMixerTrailKeyTool::UpdateSelectedKeysTransform() const
{
	if (CachedSelection.Num() > 0)
	{
		FVector AverageLocation = FVector::ZeroVector;
		TOptional<FQuat> FirstRotation;
		for (FMixerTrailKeyInfo* Info : CachedSelection)
		{
			AverageLocation += Info->Transform.GetLocation();
			if (!FirstRotation.IsSet())
			{
				FirstRotation = Info->Transform.GetRotation();
			}
		}
		AverageLocation /= (double)CachedSelection.Num();
		SelectedKeysTransform.SetLocation(AverageLocation);
		if (FirstRotation.IsSet())
		{
			SelectedKeysTransform.SetRotation(FirstRotation.GetValue());
		}
	}
}

} // namespace UE::Sequencer
