// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trail.h"
#include "Misc/FrameNumber.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class HHitProxy;
class UMovieSceneRootMotionSettingsDecoration;

namespace UE::SequencerAnimTools { struct FInputClick; }

namespace UE::Sequencer
{

class FMixerRootMotionTrail;

struct FMixerTrailKeyInfo
{
	FFrameNumber FrameNumber;
	FTransform Transform;
	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> Decoration;
	bool bDirty = true;
};

// Hit proxy for clicking on mixer trail keys in the viewport
struct HMixerTrailKeyProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	FGuid Guid;
	FMixerTrailKeyInfo* KeyInfo;

	HMixerTrailKeyProxy(const FGuid& InGuid, FMixerTrailKeyInfo* InKeyInfo)
		: HHitProxy(HPP_UI)
		, Guid(InGuid)
		, KeyInfo(InKeyInfo)
	{
	}
};

// Renders and manages interactive offset keys on the mixer root motion trail.
// Keys are drawn at positions interpolated from the baked trajectory array.
class FMixerTrailKeyTool
{
public:
	explicit FMixerTrailKeyTool(FMixerRootMotionTrail* InOwningTrail);
	~FMixerTrailKeyTool() = default;

	void SetDecorations(TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>>&& InDecorations);

	// Rebuild the key list from all decoration channels
	void BuildKeys();

	// Mark all key transforms dirty so they re-interpolate from the trail cache
	void DirtyKeyTransforms();

	// Update dirty key transforms from the trail's baked trajectory
	void UpdateKeys();

	// Cache the current view frame range for culling key rendering
	void UpdateViewRange(const TRange<FFrameNumber>& InViewRange);

	void Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating);
	bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, UE::SequencerAnimTools::FInputClick Click);

	bool IsSelected() const;
	bool IsSelected(FVector& OutVectorPosition, FQuat& OutRotation) const;
	bool IsSelected(TArray<FVector>& OutVectorPositions) const;
	void ClearSelection();

	TArray<FFrameNumber> GetSelectedKeyTimes() const;
	FFrameNumber GetPrimarySelectedKeyTime() const;
	UMovieSceneRootMotionSettingsDecoration* GetPrimarySelectedKeyDecoration() const;

	// Directly move a key's cached transform by a world-space delta.
	// Used during gizmo drags to keep the gizmo in sync without re-evaluation.
	void ApplyWorldDelta(FFrameNumber KeyFrame, const FVector& WorldPosDelta, const FRotator& WorldRotDelta);

private:
	void UpdateSelectedKeysTransform() const;

	FMixerRootMotionTrail* OwningTrail;
	TArray<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> Decorations;

	TArray<TUniquePtr<FMixerTrailKeyInfo>> Keys;
	TSet<FMixerTrailKeyInfo*> CachedSelection;
	mutable FTransform SelectedKeysTransform;
	TRange<FFrameNumber> CachedViewFrameRange;
};

} // namespace UE::Sequencer
