// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerEditorModule.h"

#include "AnimBlueprintExtension_SequencerMixerTarget.h"
#include "AnimMixerDecorationMenuProviders.h"
#include "LayerWeightDecorationEditor.h"
#include "LayerWeightDecorationMenuProvider.h"
#include "IMovieSceneRootMotionOffsetProvider.h"
#include "IMovieSceneEditingContextLayerResolver.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "AnimMixerTargetMenuProviders.h"
#include "AnimBusTargetMenuProvider.h"
#include "AnimBusSectionMenuProvider.h"
#include "AnimBusSectionInterface.h"
#include "AnimTransitionSectionInterface.h"
#include "MovieSceneAnimBusSection.h"
#include "IAnimGraph_SequencerMixerTargetConnector.h"
#include "IMovieSceneAnimMixerTargetMenuProvider.h"
#include "ISequencerDecorationEditor.h"
#include "ISequencerModule.h"
#include "MVVM/Views/KeyDrawParams.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerTrackEditor.h"
#include "MovieSceneAnimCrossfadeTransitionSection.h"
#include "MovieSceneAnimInertialDeadBlendTransitionSection.h"
#include "RootMotionTargetDecorationEditor.h"
#include "RootMotionSettingsDecorationEditor.h"
#include "MovieSceneMirroringDecoration.h"
#include "Animation/MirrorDataTable.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Styling/SlateStyleRegistry.h"
#include "Widgets/Layout/SBox.h"
#include "Channels/BuiltInChannelEditors.h"
#include "MVVM/AnimationMixerTrackModel.h"
#include "Systems/MovieSceneAnimMixerSystem.h"
#include "Systems/MovieSceneRootMotionSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "GameFramework/Actor.h"


namespace Sequencer
{
	FKeyHandle EvaluateAndAddKey(FMovieSceneByteChannelDefaultOnly* InChannel, const TMovieSceneChannelData<uint8>& InChannelData, FFrameNumber InTime, ISequencer& InSequencer, uint8 InDefaultValue = 0)
	{
		return FKeyHandle::Invalid();
	}
	TOptional<FKeyHandle> AddKeyForExternalValue(
		FMovieSceneByteChannelDefaultOnly*         InChannel,
		const TMovieSceneExternalValue<uint8>&     InExternalValue,
		FFrameNumber                               InTime,
		ISequencer&                                InSequencer,
		const FGuid&                               InObjectBindingID,
		FTrackInstancePropertyBindings*            InPropertyBindings
		)
	{
		return TOptional<FKeyHandle>();
	}

	FKeyHandle AddOrUpdateKey(
		FMovieSceneByteChannelDefaultOnly* InChannel,
		UMovieSceneSection*                InSectionToKey,
		FFrameNumber                       InTime,
		ISequencer&                        InSequencer,
		const FGuid&                       InObjectBindingID,
		FTrackInstancePropertyBindings*    InPropertyBindings
		)
	{
		return FKeyHandle::Invalid();
	}

	FKeyHandle AddOrUpdateKey(
		FMovieSceneByteChannelDefaultOnly*         InChannel,
		UMovieSceneSection*                        SectionToKey,
		const TMovieSceneExternalValue<uint8>&     InExternalValue,
		FFrameNumber                               InTime,
		ISequencer&                                InSequencer,
		const FGuid&                               InObjectBindingID,
		FTrackInstancePropertyBindings*            InPropertyBindings)
	{
		return FKeyHandle::Invalid();
	}

	void CopyKeys(FMovieSceneByteChannelDefaultOnly* InChannel, const UMovieSceneSection* InSection, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> InHandles)
	{
	}
	void PasteKeys(FMovieSceneByteChannelDefaultOnly* InChannel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys)
	{
	}
	bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneByteChannelDefaultOnly>& ChannelHandle)
	{
		return false;
	}
	TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneByteChannelDefaultOnly>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
	{
		return CreateKeyEditor(Channel.Cast<FMovieSceneByteChannel>(), Params);
	}
	bool CanCreateKeyEditor(const FMovieSceneByteChannelDefaultOnly* Channel)
	{
		return true;
	}
	// FMovieSceneBoneMatchChannel overloads — discrete, non-keyable channel for bone match data.
	// These no-op implementations prevent the default template from trying to call Evaluate().

	FKeyHandle EvaluateAndAddKey(FMovieSceneBoneMatchChannel* InChannel, const TMovieSceneChannelData<FMovieSceneBoneMatchData>& InChannelData, FFrameNumber InTime, ISequencer& InSequencer, FMovieSceneBoneMatchData InDefaultValue = FMovieSceneBoneMatchData())
	{
		return FKeyHandle::Invalid();
	}

	FKeyHandle AddOrUpdateKey(
		FMovieSceneBoneMatchChannel* InChannel,
		UMovieSceneSection*          InSectionToKey,
		FFrameNumber                 InTime,
		ISequencer&                  InSequencer,
		const FGuid&                 InObjectBindingID,
		FTrackInstancePropertyBindings* InPropertyBindings)
	{
		return FKeyHandle::Invalid();
	}

	void CopyKeys(FMovieSceneBoneMatchChannel* InChannel, const UMovieSceneSection* InSection, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> InHandles)
	{
	}

	void PasteKeys(FMovieSceneBoneMatchChannel* InChannel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys)
	{
	}

	bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneBoneMatchChannel>& ChannelHandle)
	{
		return false;
	}

	bool CanCreateKeyEditor(const FMovieSceneBoneMatchChannel* Channel)
	{
		return false;
	}

	void DrawKeys(FMovieSceneBoneMatchChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
	{
		FKeyDrawParams ValidParams, InvalidParams;
		ValidParams.FillBrush = ValidParams.BorderBrush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
		ValidParams.FillTint = FLinearColor(0.2f, 0.8f, 0.2f, 1.0f); // green

		InvalidParams.FillBrush = InvalidParams.BorderBrush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
		InvalidParams.FillTint = FLinearColor(1.0f, 0.2f, 0.2f, 1.0f); // red

		TMovieSceneChannelData<FMovieSceneBoneMatchData> ChannelData = Channel->GetData();
		TArrayView<FMovieSceneBoneMatchData> Values = ChannelData.GetValues();

		for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeyHandles[Index]);
			if (KeyIndex != INDEX_NONE && Values[KeyIndex].bIsValid)
			{
				OutKeyDrawParams[Index] = ValidParams;
			}
			else
			{
				OutKeyDrawParams[Index] = InvalidParams;
			}
		}
	}
} // namespace Sequencer

inline bool EvaluateChannel(const FMovieSceneBoneMatchChannel* InChannel, FFrameTime InTime, FMovieSceneBoneMatchData& OutValue)
{
	return false;
}


// Include order matters here since this header needs to be included after the above overloads in order for ADL to see them correctly
#include "SequencerChannelInterface.h"

namespace UE::MovieScene
{
	static bool LayerHasRootMotionSettings(const UMovieSceneAnimationMixerLayer* Layer)
	{
		if (Layer->HasChildTrack())
		{
			if (const UMovieSceneTrack* Track = Layer->GetChildTrack())
			{
				if (Track->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
				{
					return true;
				}
			}
		}
		for (const UMovieSceneSection* Section : Layer->GetSections())
		{
			if (Section && Section->FindDecoration<UMovieSceneRootMotionSettingsDecoration>())
			{
				return true;
			}
		}
		return false;
	}

	FTransform FAnimMixerRootMotionOffsetProvider::GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) const
	{
		if (!Linker || !AnimatedObject)
		{
			return FTransform::Identity;
		}

		const UMovieSceneAnimMixerSystem* AnimMixerSystem = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
		if (!AnimMixerSystem)
		{
			return FTransform::Identity;
		}

		TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionData = AnimMixerSystem->FindRootMotion(FObjectKey(AnimatedObject));
		if (!RootMotionData)
		{
			if (const AActor* Actor = Cast<AActor>(AnimatedObject))
			{
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					RootMotionData = AnimMixerSystem->FindRootMotion(FObjectKey(RootComponent));
				}
			}
		}

		if (RootMotionData)
		{
			return RootMotionData->AppliedRootMotionOffset;
		}

		return FTransform::Identity;
	}

	// Decoration editor for UMovieSceneMirroringDecoration. Provides an icon and an
	// asset picker dropdown in the outliner for selecting the MirrorDataTable.
	class FMirroringDecorationEditor : public ISequencerDecorationEditor
	{
	public:
		virtual UClass* GetDecorationClass() const override { return UMovieSceneMirroringDecoration::StaticClass(); }

		virtual const FSlateBrush* GetIconBrush() const override
		{
			return FAppStyle::GetBrush("ClassIcon.MirrorDataTable");
		}

		virtual FSlateIcon GetMenuIcon() const override
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MirrorDataTable");
		}

		virtual TSharedPtr<SWidget> CreateOutlinerWidget(UObject& Decoration, TWeakPtr<ISequencer> Sequencer) override
		{
			UMovieSceneMirroringDecoration* MirrorDecoration = Cast<UMovieSceneMirroringDecoration>(&Decoration);
			if (!MirrorDecoration)
			{
				return nullptr;
			}

			TWeakObjectPtr<UMovieSceneMirroringDecoration> WeakDecoration(MirrorDecoration);
			return SNew(SBox)
				.MinDesiredWidth(100.0f)
				.MaxDesiredWidth(200.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(UMirrorDataTable::StaticClass())
						.ObjectPath_Lambda([WeakDecoration]()
						{
							if (UMovieSceneMirroringDecoration* Dec = WeakDecoration.Get())
							{
								return Dec->MirrorDataTable ? Dec->MirrorDataTable->GetPathName() : FString();
							}
							return FString();
						})
						.OnObjectChanged_Lambda([WeakDecoration, Sequencer](const FAssetData& AssetData)
						{
							UMovieSceneMirroringDecoration* Dec = WeakDecoration.Get();
							if (!Dec)
							{
								return;
							}
							FScopedTransaction Transaction(NSLOCTEXT("MirroringDecoration", "SetMirrorTable", "Set Mirror Data Table"));
							Dec->Modify();
							Dec->MirrorDataTable = Cast<UMirrorDataTable>(AssetData.GetAsset());
							if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
							{
								PinnedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
							}
						})
						.DisplayBrowse(false)
						.DisplayThumbnail(false)
				];
		}
	};

	FTransform FAnimMixerRootMotionOffsetProvider::GetRootMotionOffset(
		const UMovieSceneEntitySystemLinker* Linker,
		UObject* AnimatedObject,
		const UObject* EditingContext) const
	{
		if (!EditingContext)
		{
			return GetRootMotionOffset(Linker, AnimatedObject);
		}

		if (!Linker || !AnimatedObject)
		{
			return FTransform::Identity;
		}

		const UMovieSceneAnimMixerSystem* AnimMixerSystem = Linker->FindSystem<UMovieSceneAnimMixerSystem>();
		if (!AnimMixerSystem)
		{
			return FTransform::Identity;
		}

		// Find root motion data for this animated object (same lookup as the 2-arg version)
		TSharedPtr<FMovieSceneMixerRootMotionComponentData> RootMotionData = AnimMixerSystem->FindRootMotion(FObjectKey(AnimatedObject));
		if (!RootMotionData)
		{
			if (const AActor* Actor = Cast<AActor>(AnimatedObject))
			{
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					RootMotionData = AnimMixerSystem->FindRootMotion(FObjectKey(RootComponent));
				}
			}
		}
		if (!RootMotionData)
		{
			return FTransform::Identity;
		}

		// Resolve the editing context to a mixer layer. If the layer has no root
		// motion settings, this context did not produce root motion and the offset
		// should be Identity. If it has root motion settings, return the total
		// applied offset.
		TArray<IMovieSceneEditingContextLayerResolver*> Resolvers =
			IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneEditingContextLayerResolver>(
				IMovieSceneEditingContextLayerResolver::GetModularFeatureName());
		for (IMovieSceneEditingContextLayerResolver* Resolver : Resolvers)
		{
			if (!Resolver)
			{
				continue;
			}
			if (UObject* ResolvedLayerObj = Resolver->ResolveEditingContextToMixerLayer(Linker, EditingContext))
			{
				if (UMovieSceneAnimationMixerLayer* ResolvedLayer = Cast<UMovieSceneAnimationMixerLayer>(ResolvedLayerObj))
				{
					if (!LayerHasRootMotionSettings(ResolvedLayer))
					{
						return FTransform::Identity;
					}
					return RootMotionData->AppliedRootMotionOffset;
				}
			}
		}

		// No layer resolved. This happens for pose producers like Control Rigs running
		// alongside an Animation track with the mixer plugin enabled but no Animation Mixer
		// track (backwards-compat path -- no UMovieSceneAnimationMixerLayer objects exist).
		// The Control Rig in that configuration isn't the source of the root motion -- the
		// Animation track is -- so returning the total applied offset would cause the caller
		// (e.g. gizmo drawing) to wrongly subtract the Animation track's offset from the
		// Control Rig's HostingSceneComponentTransform, leaving the gizmos at the anim origin
		// while the skeleton and actor move forward.
		return FTransform::Identity;
	}

	void FAnimMixerBakeScope::BeginBakeScope()
	{
		UMovieSceneAnimMixerSystem::PushForceRootBoneDestinationScope();
	}

	void FAnimMixerBakeScope::EndBakeScope()
	{
		UMovieSceneAnimMixerSystem::PopForceRootBoneDestinationScope();
	}

	FLazyName SequencerModuleName("Sequencer");

	void FMovieSceneAnimMixerEditorModule::StartupModule()
	{
		using namespace UE::Sequencer;

		IModularFeatures::Get().RegisterModularFeature(IAnimGraph_SequencerMixerTargetConnector::GetModularFeatureName(), this);

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>(SequencerModuleName.Resolve());
		AnimationTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic( &FAnimationMixerTrackEditor::CreateTrackEditor));

		// Register custom track model for Animation Mixer tracks
		AnimationMixerTrackModelHandle = SequencerModule.RegisterTrackModel(FOnCreateTrackModel::CreateLambda([](UMovieSceneTrack* Track) -> TSharedPtr<FTrackModel>
			{
				if (Track && Track->IsA<UMovieSceneAnimationMixerTrack>())
				{
					return MakeShared<FAnimationMixerTrackModel>(Track);
				}
				return nullptr;
			}
		));

		SequencerModule.RegisterChannelInterface<FMovieSceneByteChannelDefaultOnly>();
		SequencerModule.RegisterChannelInterface<FMovieSceneBoneMatchChannel>();

		// Register custom section interfaces for transition sections
		UE::Sequencer::FAnimationMixerTrackEditor::RegisterCustomMixerAnimSection(
			UMovieSceneAnimCrossfadeTransitionSection::StaticClass(),
			FOnMakeSectionInterfaceDelegate::CreateLambda([](UMovieSceneSection& Section, UMovieSceneTrack& Track, FGuid ObjectBinding) -> TSharedRef<ISequencerSection>
			{
				return MakeShared<UE::Sequencer::FAnimTransitionSectionInterface>(Section, Track, ObjectBinding);
			}));

		UE::Sequencer::FAnimationMixerTrackEditor::RegisterCustomMixerAnimSection(
			UMovieSceneAnimInertialDeadBlendTransitionSection::StaticClass(),
			FOnMakeSectionInterfaceDelegate::CreateLambda([](UMovieSceneSection& Section, UMovieSceneTrack& Track, FGuid ObjectBinding) -> TSharedRef<ISequencerSection>
			{
				return MakeShared<UE::Sequencer::FAnimTransitionSectionInterface>(Section, Track, ObjectBinding);
			}));

		// Register target menu providers
		AutomaticTargetMenuProvider = MakeUnique<FAutomaticTargetMenuProvider>();
		AnimInstanceTargetMenuProvider = MakeUnique<FAnimInstanceTargetMenuProvider>();
		AnimBlueprintTargetMenuProvider = MakeUnique<FAnimBlueprintTargetMenuProvider>();
		AnimNextInjectionTargetMenuProvider = MakeUnique<FAnimNextInjectionTargetMenuProvider>();

		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AutomaticTargetMenuProvider.Get());
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimInstanceTargetMenuProvider.Get());
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimBlueprintTargetMenuProvider.Get());
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimNextInjectionTargetMenuProvider.Get());

		BusTargetMenuProvider = MakeUnique<FAnimBusTargetMenuProvider>();
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), BusTargetMenuProvider.Get());

		// Register bus section menu provider
		BusSectionMenuProvider = MakeUnique<FAnimBusSectionMenuProvider>();
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), BusSectionMenuProvider.Get());

		// Register decoration editor factories (per-Sequencer instances).
		ISequencerModule& SequencerModuleForDecorations = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

		RootMotionTargetDecorationEditorHandle = SequencerModuleForDecorations.RegisterDecorationEditor(
			FOnCreateDecorationEditor::CreateLambda([](TSharedRef<ISequencer>) -> TSharedRef<ISequencerDecorationEditor>
			{
				return MakeShared<UE::Sequencer::FRootMotionTargetDecorationEditor>();
			}));

		RootMotionSettingsDecorationEditorHandle = SequencerModuleForDecorations.RegisterDecorationEditor(
			FOnCreateDecorationEditor::CreateLambda([](TSharedRef<ISequencer> InSequencer) -> TSharedRef<ISequencerDecorationEditor>
			{
				return MakeShared<UE::Sequencer::FRootMotionSettingsDecorationEditor>(InSequencer);
			}));

		AnimMaskingDecorationEditorHandle = SequencerModuleForDecorations.RegisterDecorationEditor(
			FOnCreateDecorationEditor::CreateLambda([](TSharedRef<ISequencer>) -> TSharedRef<ISequencerDecorationEditor>
			{
				return MakeShared<FMovieSceneAnimationMaskDecorationEditor>();
			}));

		LayerWeightDecorationEditorHandle = SequencerModuleForDecorations.RegisterDecorationEditor(
			FOnCreateDecorationEditor::CreateLambda([](TSharedRef<ISequencer>) -> TSharedRef<ISequencerDecorationEditor>
			{
				return MakeShared<UE::Sequencer::FLayerWeightDecorationEditor>();
			}));

		MirroringDecorationEditorHandle = SequencerModuleForDecorations.RegisterDecorationEditor(
			FOnCreateDecorationEditor::CreateLambda([](TSharedRef<ISequencer>) -> TSharedRef<ISequencerDecorationEditor>
			{
				return MakeShared<FMirroringDecorationEditor>();
			}));

		// Register decoration menu providers
		MaskDecorationMenuProvider = MakeUnique<FMaskDecorationMenuProvider>();
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneDecorationMenuProvider::GetModularFeatureName(), MaskDecorationMenuProvider.Get());

		LayerWeightDecorationMenuProvider = MakeUnique<FLayerWeightDecorationMenuProvider>();
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneDecorationMenuProvider::GetModularFeatureName(), LayerWeightDecorationMenuProvider.Get());
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneRootMotionOffsetProvider::GetModularFeatureName(), &AnimMixerRootMotionProvider);
		IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimSequenceBakeScope::GetModularFeatureName(), &BakeScope);
	}

	void FMovieSceneAnimMixerEditorModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimSequenceBakeScope::GetModularFeatureName(), &BakeScope);
		IModularFeatures::Get().UnregisterModularFeature(IMovieSceneRootMotionOffsetProvider::GetModularFeatureName(), &AnimMixerRootMotionProvider);

		// Unregister decoration editor factories.
		if (FModuleManager::Get().IsModuleLoaded("Sequencer"))
		{
			ISequencerModule& SequencerModuleForDecorations = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
			if (RootMotionSettingsDecorationEditorHandle.IsValid())
			{
				SequencerModuleForDecorations.UnRegisterDecorationEditor(RootMotionSettingsDecorationEditorHandle);
			}
			if (RootMotionTargetDecorationEditorHandle.IsValid())
			{
				SequencerModuleForDecorations.UnRegisterDecorationEditor(RootMotionTargetDecorationEditorHandle);
			}
			if (AnimMaskingDecorationEditorHandle.IsValid())
			{
				SequencerModuleForDecorations.UnRegisterDecorationEditor(AnimMaskingDecorationEditorHandle);
			}
			if (LayerWeightDecorationEditorHandle.IsValid())
			{
				SequencerModuleForDecorations.UnRegisterDecorationEditor(LayerWeightDecorationEditorHandle);
			}
			if (MirroringDecorationEditorHandle.IsValid())
			{
				SequencerModuleForDecorations.UnRegisterDecorationEditor(MirroringDecorationEditorHandle);
			}
		}

		// Unregister decoration menu providers
		if (MaskDecorationMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneDecorationMenuProvider::GetModularFeatureName(), MaskDecorationMenuProvider.Get());
		}
		if (LayerWeightDecorationMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneDecorationMenuProvider::GetModularFeatureName(), LayerWeightDecorationMenuProvider.Get());
		}

		// Unregister target menu providers
		if (AutomaticTargetMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AutomaticTargetMenuProvider.Get());
		}
		if (AnimInstanceTargetMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimInstanceTargetMenuProvider.Get());
		}
		if (AnimBlueprintTargetMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimBlueprintTargetMenuProvider.Get());
		}
		if (AnimNextInjectionTargetMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), AnimNextInjectionTargetMenuProvider.Get());
		}
		if (BusTargetMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName(), BusTargetMenuProvider.Get());
		}

		// Unregister bus section menu provider
		if (BusSectionMenuProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), BusSectionMenuProvider.Get());
		}

		// Unregister custom section interface for transition sections
		UE::Sequencer::FAnimationMixerTrackEditor::UnregisterCustomMixerAnimSection(UMovieSceneAnimCrossfadeTransitionSection::StaticClass());
		UE::Sequencer::FAnimationMixerTrackEditor::UnregisterCustomMixerAnimSection(UMovieSceneAnimInertialDeadBlendTransitionSection::StaticClass());

		if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>(SequencerModuleName.Resolve()))
		{
			SequencerModule->UnregisterTrackModel(AnimationMixerTrackModelHandle);
		}

		IModularFeatures::Get().UnregisterModularFeature(IAnimGraph_SequencerMixerTargetConnector::GetModularFeatureName(), this);
	}

	void FMovieSceneAnimMixerEditorModule::GetSequencerMixerTargetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
	{
		OutExtensions.Add(UAnimBlueprintExtension_SequencerMixerTarget::StaticClass());
	}
}

IMPLEMENT_MODULE(UE::MovieScene::FMovieSceneAnimMixerEditorModule, MovieSceneAnimMixerEditor)