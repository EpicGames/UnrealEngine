// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimGraph_SequencerMixerTargetConnector.h"
#include "IMovieSceneAnimSequenceBakeScope.h"
#include "IMovieSceneRootMotionOffsetProvider.h"
#include "Modules/ModuleInterface.h"
#include "MovieSceneAnimMixerEditor/MovieSceneAnimationMaskDecorationEditor.h"

class FAutomaticTargetMenuProvider;
class FAnimInstanceTargetMenuProvider;
class FAnimBlueprintTargetMenuProvider;
class FAnimNextInjectionTargetMenuProvider;
class FAnimBusTargetMenuProvider;
class FAnimBusSectionMenuProvider;
class FMaskDecorationMenuProvider;
class FLayerWeightDecorationMenuProvider;


namespace UE::MovieScene
{
	class FAnimMixerRootMotionOffsetProvider : public IMovieSceneRootMotionOffsetProvider
	{
	public:
		virtual FTransform GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) const override;
		virtual FTransform GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject, const UObject* EditingContext) const override;
	};

	/**
	 * Forwards Begin/End to UMovieSceneAnimMixerSystem::Push/PopForceRootBoneDestinationScope
	 * so the binding-level AnimSeq/CR bake recorder captures mixer root motion on the
	 * root bone rather than seeing it applied to the actor.
	 */
	class FAnimMixerBakeScope : public IMovieSceneAnimSequenceBakeScope
	{
	public:
		virtual void BeginBakeScope() override;
		virtual void EndBakeScope() override;
	};

	class FMovieSceneAnimMixerEditorModule : public IModuleInterface, public IAnimGraph_SequencerMixerTargetConnector
	{
	public:

		// IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		virtual void GetSequencerMixerTargetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;

	private:
		FDelegateHandle AnimationTrackEditorHandle;
		FDelegateHandle AnimationMixerTrackModelHandle;

		// Target menu providers
		TUniquePtr<FAutomaticTargetMenuProvider> AutomaticTargetMenuProvider;
		TUniquePtr<FAnimInstanceTargetMenuProvider> AnimInstanceTargetMenuProvider;
		TUniquePtr<FAnimBlueprintTargetMenuProvider> AnimBlueprintTargetMenuProvider;
		TUniquePtr<FAnimNextInjectionTargetMenuProvider> AnimNextInjectionTargetMenuProvider;
		TUniquePtr<FAnimBusTargetMenuProvider> BusTargetMenuProvider;

		// Bus section menu provider
		TUniquePtr<FAnimBusSectionMenuProvider> BusSectionMenuProvider;

		// Decoration editor factory handles
		FDelegateHandle RootMotionTargetDecorationEditorHandle;
		FDelegateHandle RootMotionSettingsDecorationEditorHandle;
		FDelegateHandle AnimMaskingDecorationEditorHandle;
		FDelegateHandle LayerWeightDecorationEditorHandle;
		FDelegateHandle MirroringDecorationEditorHandle;

		// Decoration menu providers
		TUniquePtr<FMaskDecorationMenuProvider> MaskDecorationMenuProvider;
		TUniquePtr<FLayerWeightDecorationMenuProvider> LayerWeightDecorationMenuProvider;
		FAnimMixerRootMotionOffsetProvider AnimMixerRootMotionProvider;

		// Scope provider that forces mixer root motion onto the root bone during
		// binding-level AnimSeq / CR bake recording.
		FAnimMixerBakeScope BakeScope;
	};
}


