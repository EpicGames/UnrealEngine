// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

// Style set containing icons for the Animation Mixer outliner rows.
// Used by the mixer track, layer model, section interfaces, and decoration
// editors to resolve per-track/row brushes.
class FMovieSceneAnimMixerEditorStyle final : public FSlateStyleSet
{
public:

	FMovieSceneAnimMixerEditorStyle()
		: FSlateStyleSet("MovieSceneAnimMixerEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/MovieSceneAnimMixer/Content/Editor/Slate"));

		Set("Tracks.AnimMixer",          new IMAGE_BRUSH_SVG("AnimMixer_16",          Icon16x16));
		Set("Tracks.MixLayer",           new IMAGE_BRUSH_SVG("MixLayer_16",           Icon16x16));
		Set("Tracks.Transition",         new IMAGE_BRUSH_SVG("Transition_16",         Icon16x16));
		Set("Tracks.SkelAnim",           new IMAGE_BRUSH_SVG("SkelAnim_16",           Icon16x16));
		Set("Tracks.Masking",            new IMAGE_BRUSH_SVG("Masking_16",            Icon16x16));
		Set("Tracks.Bus",                new IMAGE_BRUSH_SVG("Bus_16",                Icon16x16));
		Set("Tracks.RootMotion",         new IMAGE_BRUSH_SVG("RootMotion_16",         Icon16x16));
		Set("Tracks.RootMotionSettings", new IMAGE_BRUSH_SVG("RootMotionSettings_16", Icon16x16));
		Set("Tracks.RootMotionTarget",   new IMAGE_BRUSH_SVG("RootMotionTarget_16",   Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FMovieSceneAnimMixerEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FMovieSceneAnimMixerEditorStyle& Get()
	{
		static FMovieSceneAnimMixerEditorStyle Inst;
		return Inst;
	}
};
