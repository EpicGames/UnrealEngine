// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimationMixerTrackEditor.h"
#include "MixerLinkedAnimTrackProvider.h"
#include "MovieSceneAnimMixerEditorStyle.h"
#include "IMovieSceneLinkedAnimTrackProvider.h"

#include "AnimMixerTrailHierarchy.h"
#include "InteractiveToolManager.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "MotionTrailTool.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "AnimationMixerTrackEditMode.h"
#include "EditorModeManager.h"
#include "MovieScene.h"
#include "Animation/AnimSequenceBase.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "UObject/UObjectIterator.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ClassIconFinder.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"
#include "IMovieSceneAnimMixerTargetMenuProvider.h"
#include "AnimatedRange.h"
#include "IMovieSceneModule.h"
#include "ISequencer.h"
#include "MovieSceneAnimationMixerItemInterface.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimMixerMaskSection.h"
#include "MovieSceneTrack.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Views/ViewUtilities.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "SequencerCommonHelpers.h"
#include "SequencerSectionPainter.h"
#include "SequencerUtilities.h"
#include "TimeToPixel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AnimBusSectionInterface.h"
#include "AnimMixerBusUtils.h"
#include "AnimMixerEditorBusUtils.h"
#include "MovieSceneAnimBusSection.h"
#include "MVVM/AnimationMixerTrackModel.h"

#include "AnimMixerBakeHelper.h"

#define LOCTEXT_NAMESPACE "MovieSceneAnimationMixerTrackEditor"

namespace UE::Sequencer
{

/**
 * Combo widget for selecting the animation mixer target type in the outliner.
 * Displays the current target icon and short name, and provides a dropdown menu
 * to change the target using registered IMovieSceneAnimMixerTargetMenuProvider instances.
 */
class SAnimMixerTargetComboWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimMixerTargetComboWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakObjectPtr<UMovieSceneAnimationMixerTrack> InWeakTrack, TWeakPtr<ISequencer> InWeakSequencer, FGuid InObjectBinding)
	{
		WeakTrack = InWeakTrack;
		WeakSequencer = InWeakSequencer;
		ObjectBinding = InObjectBinding;

		ChildSlot
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.WidthOverride(120.f)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SComboButton)
				.ToolTipText(this, &SAnimMixerTargetComboWidget::GetTargetDisplayName)
				.OnGetMenuContent(this, &SAnimMixerTargetComboWidget::GetMenuContent)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 4.f, 0.f)
					[
						SNew(SImage)
						.Image(this, &SAnimMixerTargetComboWidget::GetTargetIcon)
						.Visibility(this, &SAnimMixerTargetComboWidget::GetTargetIconVisibility)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &SAnimMixerTargetComboWidget::GetTargetDisplayName)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					]
				]
			]
		];
	}

private:
	TSharedRef<SWidget> GetMenuContent()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		// Get the bound object for target menu providers
		UObject* BoundObject = nullptr;
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer && ObjectBinding.IsValid())
		{
			TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectBinding, Sequencer->GetFocusedTemplateID());
			if (BoundObjects.Num() > 0)
			{
				BoundObject = BoundObjects[0].Get();
			}
		}

		// Find all registered target menu providers
		TArray<IMovieSceneAnimMixerTargetMenuProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimMixerTargetMenuProvider>(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName());

		// Sort providers by priority (lower values appear first)
		Providers.Sort([](const IMovieSceneAnimMixerTargetMenuProvider& A, const IMovieSceneAnimMixerTargetMenuProvider& B)
		{
			return A.GetTargetMenuPriority() < B.GetTargetMenuPriority();
		});

		// Callback when a target is selected
		auto OnTargetSelected = [this](TInstancedStruct<FMovieSceneMixedAnimationTarget> Target)
		{
			SetTarget(MoveTemp(Target));
		};

		// Populate menu from all providers
		UMovieSceneAnimationMixerTrack* Track = WeakTrack.Get();
		for (IMovieSceneAnimMixerTargetMenuProvider* Provider : Providers)
		{
			if (Provider)
			{
				Provider->PopulateTargetMenu(MenuBuilder, BoundObject, OnTargetSelected, Sequencer, Track);
			}
		}

		return MenuBuilder.MakeWidget();
	}

	const FSlateBrush* GetTargetIcon() const
	{
		UMovieSceneAnimationMixerTrack* Track = WeakTrack.Get();
		if (Track)
		{
			const FMovieSceneMixedAnimationTarget* Target = Track->MixedAnimationTarget.GetPtr<FMovieSceneMixedAnimationTarget>();
			if (Target)
			{
				return Target->GetIcon();
			}
		}
		return nullptr;
	}

	EVisibility GetTargetIconVisibility() const
	{
		return GetTargetIcon() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetTargetDisplayName() const
	{
		UMovieSceneAnimationMixerTrack* Track = WeakTrack.Get();
		if (Track)
		{
			const FMovieSceneMixedAnimationTarget* Target = Track->MixedAnimationTarget.GetPtr<FMovieSceneMixedAnimationTarget>();
			if (Target)
			{
				return Target->GetShortDisplayName();
			}
		}
		return LOCTEXT("UnknownTarget", "Unknown");
	}

	void SetTarget(TInstancedStruct<FMovieSceneMixedAnimationTarget> NewTarget)
	{
		UMovieSceneAnimationMixerTrack* Track = WeakTrack.Get();
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Track && Sequencer)
		{
			FScopedTransaction Transaction(LOCTEXT("SetAnimMixerTarget", "Set Animation Mixer Target"));
			Track->Modify();
			TInstancedStruct<FMovieSceneMixedAnimationTarget> PreviousTarget = Track->MixedAnimationTarget;
			Track->MixedAnimationTarget = MoveTemp(NewTarget);

			{
				FAnimMixerBusValidationResult Result = FAnimMixerBusUtils::ValidateBusTopology(
					AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(Track, *Sequencer));
				if (Result.HasErrors())
				{
					Transaction.Cancel();
					Track->MixedAnimationTarget = MoveTemp(PreviousTarget);

					FNotificationInfo Info(FText::FromString(Result.Errors[0]));
					Info.ExpireDuration = 4.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
					return;
				}
			}

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}

	TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakTrack;
	TWeakPtr<ISequencer> WeakSequencer;
	FGuid ObjectBinding;
};

const FSlateBrush* FAnimMixerAnimationSection::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.SkelAnim");
}

int32 FAnimMixerAnimationSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	// Check if we're in section outliner mode (layer expanded, each section on its own row)
	// In outliner mode, sections don't visually overlap so no transition padding is needed
	const bool bIsInOutlinerMode = Painter.SectionModel &&
		Painter.SectionModel->FindAncestorOfType<UE::Sequencer::FSectionOutlinerModel>().IsValid();

	// Calculate transition padding using the accurate time converter before painting
	// This value will be used by GetContentPadding() which is called after OnPaintSection()
	CachedTransitionPaddingLeft = bIsInOutlinerMode ? 0.0f : CalculateTransitionPadding(Painter.GetTimeConverter());

	return FCommonAnimationSection::OnPaintSection(Painter);
}

FMargin FAnimMixerAnimationSection::GetContentPadding() const
{
	FMargin BasePadding = FCommonAnimationSection::GetContentPadding();

	// Add the transition padding that was calculated in OnPaintSection
	BasePadding.Left += CachedTransitionPaddingLeft;

	return BasePadding;
}

float FAnimMixerAnimationSection::CalculateTransitionPadding(const FTimeToPixel& TimeConverter) const
{
	// Check if there's a transition section overlapping the start of this section
	// If so, calculate extra left padding so the text appears after the transition
	UMovieSceneSection* Section = WeakSection.Get();
	if (!Section || !Section->HasStartFrame())
	{
		return 0.0f;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return 0.0f;
	}

	// Find any transition that references this section as the "to" section
	TArray<UMovieSceneAnimTransitionSectionBase*> Transitions = MixerTrack->FindTransitionsForSection(Section);
	for (UMovieSceneAnimTransitionSectionBase* Transition : Transitions)
	{
		if (Transition && Transition->ToSection == Section)
		{
			// This is the "to" section of a transition - calculate padding for the transition width
			TRange<FFrameNumber> TransitionRange = Transition->GetRange();
			if (!TransitionRange.IsEmpty() && TransitionRange.HasLowerBound() && TransitionRange.HasUpperBound())
			{
				// Use the accurate time converter to calculate pixel width
				FFrameNumber TransitionStart = TransitionRange.GetLowerBoundValue();
				FFrameNumber TransitionEnd = TransitionRange.GetUpperBoundValue();

				float StartPixel = TimeConverter.FrameToPixel(TransitionStart);
				float EndPixel = TimeConverter.FrameToPixel(TransitionEnd);
				float TransitionPixels = EndPixel - StartPixel;

				// Add padding with a small margin
				return TransitionPixels + 4.0f;
			}
			break;
		}
	}

	return 0.0f;
}

void FAnimMixerAnimationSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	FCommonAnimationSection::BuildSectionContextMenu(MenuBuilder, ObjectBinding);

	// Add "Create Transition To..." submenu for overlapping sections
	UMovieSceneSection* Section = GetSectionObject();
	if (Section)
	{
		UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
		if (MixerTrack)
		{
			TArray<UMovieSceneSection*> OverlappingSections = MixerTrack->FindOverlappingSections(Section);
			if (OverlappingSections.Num() > 0)
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("CreateTransition_Label", "Create Transition To..."),
					LOCTEXT("CreateTransition_Tooltip", "Create a transition between this section and an overlapping section"),
					FNewMenuDelegate::CreateSP(this, &FAnimMixerAnimationSection::PopulateCreateTransitionMenu),
					/*bInOpenSubMenuOnClick=*/ false,
					FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Transition")
				);
			}

			// Mirror the modifier/decoration entries available in the outliner '+' menu.
			UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayerForSection(Section);
			if (Layer)
			{
				SequencerHelpers::BuildDecorationMenu(MenuBuilder, Layer, ObjectBinding, Sequencer,
					LOCTEXT("LayerModifiers", "Layer Modifiers"));

				if (Layer->HasChildTrack())
				{
					SequencerHelpers::BuildDecorationMenu(MenuBuilder, Layer->GetChildTrack(), ObjectBinding, Sequencer,
						LOCTEXT("TrackModifiers", "Track Modifiers"));
				}
				else
				{
					SequencerHelpers::BuildDecorationMenu(MenuBuilder, Section, ObjectBinding, Sequencer);
				}
			}
		}
	}
}

void FAnimMixerAnimationSection::PopulateCreateTransitionMenu(FMenuBuilder& MenuBuilder)
{
	UMovieSceneSection* Section = GetSectionObject();
	if (!Section)
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return;
	}

	TArray<UMovieSceneSection*> OverlappingSections = MixerTrack->FindOverlappingSections(Section);

	// Get all non-abstract transition section classes
	TArray<UClass*> TransitionClasses;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (Class->IsChildOf(UMovieSceneAnimTransitionSectionBase::StaticClass()) &&
			!Class->HasAnyClassFlags(CLASS_Abstract) &&
			Class != UMovieSceneAnimTransitionSectionBase::StaticClass())
		{
			TransitionClasses.Add(Class);
		}
	}

	// Sort by display name
	TransitionClasses.Sort([](const UClass& A, const UClass& B)
	{
		return A.GetDisplayNameText().CompareTo(B.GetDisplayNameText()) < 0;
	});

	for (UMovieSceneSection* OtherSection : OverlappingSections)
	{
		// Skip sections without start frames (unbounded sections can't have meaningful transitions)
		if (!Section->HasStartFrame() || !OtherSection->HasStartFrame())
		{
			continue;
		}

		// Determine from/to based on start time
		UMovieSceneSection* FromSection = Section;
		UMovieSceneSection* ToSection = OtherSection;
		if (OtherSection->GetInclusiveStartFrame() < Section->GetInclusiveStartFrame())
		{
			FromSection = OtherSection;
			ToSection = Section;
		}

		// Check if a transition already exists for this pair
		UMovieSceneAnimTransitionSectionBase* ExistingTransition = MixerTrack->FindTransitionForPair(FromSection, ToSection);
		if (ExistingTransition)
		{
			continue; // Skip - transition already exists
		}

		// Get a display name for the other section
		FText OtherSectionName = FText::FromString(OtherSection->GetName());
		if (UMovieSceneSkeletalAnimationSection* OtherAnimSection = Cast<UMovieSceneSkeletalAnimationSection>(OtherSection))
		{
			if (UAnimSequenceBase* Animation = OtherAnimSection->GetAnimation())
			{
				OtherSectionName = FText::FromString(Animation->GetName());
			}
		}

		// Add submenu for this overlapping section with transition type options
		MenuBuilder.AddSubMenu(
			OtherSectionName,
			FText::Format(LOCTEXT("CreateTransitionTo_Tooltip", "Create a transition to {0}"), OtherSectionName),
			FNewMenuDelegate::CreateSP(this, &FAnimMixerAnimationSection::PopulateTransitionTypeMenu, OtherSection, TransitionClasses)
		);
	}
}

void FAnimMixerAnimationSection::PopulateTransitionTypeMenu(FMenuBuilder& MenuBuilder, UMovieSceneSection* OtherSection, TArray<UClass*> TransitionClasses)
{
	UMovieSceneSection* Section = GetSectionObject();
	if (!Section || !OtherSection)
	{
		return;
	}

	// Skip sections without start frames
	if (!Section->HasStartFrame() || !OtherSection->HasStartFrame())
	{
		return;
	}

	// Determine from/to based on start time
	UMovieSceneSection* FromSection = Section;
	UMovieSceneSection* ToSection = OtherSection;
	if (OtherSection->GetInclusiveStartFrame() < Section->GetInclusiveStartFrame())
	{
		FromSection = OtherSection;
		ToSection = Section;
	}

	for (UClass* TransitionClass : TransitionClasses)
	{
		FText DisplayName = TransitionClass->GetDisplayNameText();

		MenuBuilder.AddMenuEntry(
			DisplayName,
			FText::Format(LOCTEXT("CreateTransitionOfType_Tooltip", "Create a {0} transition"), DisplayName),
			FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.Transition"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimMixerAnimationSection::CreateTransitionToSection, FromSection, ToSection, TransitionClass)
			)
		);
	}
}

void FAnimMixerAnimationSection::CreateTransitionToSection(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection, UClass* TransitionClass)
{
	if (!FromSection || !ToSection || !TransitionClass)
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(FromSection->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreateTransition", "Create Transition"));

	MixerTrack->CreateTransitionSectionOfType(FromSection, ToSection, TransitionClass);
}

int32 FAnimationMixerTrackEditor::MixerEditModeRefCount = 0;
TMap<const UClass*, FOnMakeSectionInterfaceDelegate> FAnimationMixerTrackEditor::MakeSectionInterfaceCallbacks;

void FAnimationMixerTrackEditor::RegisterMakeSectionInterfaceCallback(const UClass* SectionClass, FOnMakeSectionInterfaceDelegate Callback)
{
	MakeSectionInterfaceCallbacks.Add(SectionClass, MoveTemp(Callback));
}

void FAnimationMixerTrackEditor::UnregisterMakeSectionInterfaceCallback(const UClass* SectionClass)
{
	MakeSectionInterfaceCallbacks.Remove(SectionClass);
}

TSharedRef<ISequencerTrackEditor> FAnimationMixerTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FAnimationMixerTrackEditor(InSequencer));
}

FAnimationMixerTrackEditor::FAnimationMixerTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FCommonAnimationTrackEditor(InSequencer)
{
}

void FAnimationMixerTrackEditor::OnInitialize()
{
	FCommonAnimationTrackEditor::OnInitialize();

	// Register as animation mixer item menu provider
	IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);

	// Register as linked anim track provider for mixer bindings
	MixerLinkedAnimProvider = MakeUnique<FMixerLinkedAnimTrackProvider>(this);
	IModularFeatures::Get().RegisterModularFeature(
		IMovieSceneLinkedAnimTrackProvider::GetModularFeatureName(), MixerLinkedAnimProvider.Get());

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer && Sequencer->IsLevelEditorSequencer())
	{
		bRegisteredLevelEditorMode = true;

		++MixerEditModeRefCount;
		GLevelEditorModeTools().ActivateMode(UAnimationMixerTrackEditMode::ModeName);
		if (UAnimationMixerTrackEditMode* MixerEditMode = Cast<UAnimationMixerTrackEditMode>(
				GLevelEditorModeTools().GetActiveScriptableMode(UAnimationMixerTrackEditMode::ModeName)))
		{
			MixerEditMode->SetSequencer(Sequencer);
		}

		// Register to add mixer trail hierarchies when the motion trail tool sets up
		AdditionalTrailHierarchiesHandle = UMotionTrailTool::OnCreateAdditionalTrailHierarchies.AddRaw(
			this, &FAnimationMixerTrackEditor::OnCreateAdditionalTrailHierarchies);

		// If the motion trail tool is already active, add the mixer hierarchy now
		// since SetupIntegration already broadcast before we registered.
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
		if (LevelEditorModule)
		{
			TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();
			if (LevelEditor.IsValid())
			{
				if (UMotionTrailTool* TrailTool = Cast<UMotionTrailTool>(
						LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveTool(EToolSide::Left)))
				{
					TWeakPtr<ISequencer> WeakSeq = Sequencer;
					MixerTrailHierarchy = MakeShared<FAnimMixerTrailHierarchy>(WeakSeq);
					MixerTrailHierarchy->Initialize();
					TrailTool->AddTrailHierarchy(MixerTrailHierarchy);

					if (UAnimationMixerTrackEditMode* MixerEditMode = Cast<UAnimationMixerTrackEditMode>(
							GLevelEditorModeTools().GetActiveScriptableMode(UAnimationMixerTrackEditMode::ModeName)))
					{
						MixerEditMode->SetTrailHierarchy(MixerTrailHierarchy);
					}
				}
			}
		}
	}
}

void FAnimationMixerTrackEditor::OnRelease()
{
	// Unregister as animation mixer item menu provider
	IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);

	// Unregister linked anim track provider
	if (MixerLinkedAnimProvider.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(
			IMovieSceneLinkedAnimTrackProvider::GetModularFeatureName(), MixerLinkedAnimProvider.Get());
		MixerLinkedAnimProvider.Reset();
	}

	if (bRegisteredLevelEditorMode)
	{
		UMotionTrailTool::OnCreateAdditionalTrailHierarchies.Remove(AdditionalTrailHierarchiesHandle);
		MixerTrailHierarchy.Reset();

		if (--MixerEditModeRefCount == 0)
		{
			GLevelEditorModeTools().DeactivateMode(UAnimationMixerTrackEditMode::ModeName);
		}

		bRegisteredLevelEditorMode = false;
	}

	FCommonAnimationTrackEditor::OnRelease();
}

const UClass* FAnimationMixerTrackEditor::GetHandledMixerItemClass() const
{
	// Animation Mixer track editor doesn't handle any specific mixer item class
	// It only provides the "Animation Mixer" entry in the Object Binding Animation menu
	return nullptr;
}

FText FAnimationMixerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AnimationMixerTrackEditor_DisplayName", "Animation Mixer");
}

void FAnimationMixerTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// Animation Mixer contributes to the Animation submenu via PopulateObjectBindingAnimationMenu instead.
	// The base class FCommonAnimationTrackEditor::BuildObjectBindingTrackMenu creates the Animation submenu,
	// so we override to empty here to avoid creating duplicate Animation submenus.
}

void FAnimationMixerTrackEditor::PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex)
{
	// Animation Mixer track editor doesn't contribute skeletal animations to the mixer '+' menu.
	// FSkeletalAnimationTrackEditor (which also inherits from FCommonAnimationTrackEditor) handles that.
}

void FAnimationMixerTrackEditor::PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu)
{
	if (ObjectClass != nullptr && (ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass())))
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddAnimationTrack", "Animation Mixer"),
			LOCTEXT("AddAnimationTrackTooltip", "Adds a new animation track for playing back Anim Sequences and other sources of animation."),
			FNewMenuDelegate::CreateSP(this, &FAnimationMixerTrackEditor::BuildTargetSelectionMenu, TArray<FGuid>(ObjectBindings), ObjectClass),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None,
			false,
			FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.AnimMixer")
		);
	}
}

void FAnimationMixerTrackEditor::BuildTargetSelectionMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, const UClass* ObjectClass)
{
	// Get the bound object for target menu providers
	UObject* BoundObject = nullptr;
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer && ObjectBindings.Num() > 0)
	{
		TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectBindings[0], Sequencer->GetFocusedTemplateID());
		if (BoundObjects.Num() > 0)
		{
			BoundObject = BoundObjects[0].Get();
		}
	}

	// Find all registered target menu providers
	TArray<IMovieSceneAnimMixerTargetMenuProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimMixerTargetMenuProvider>(IMovieSceneAnimMixerTargetMenuProvider::GetModularFeatureName());

	// Sort providers by priority (lower values appear first)
	Providers.Sort([](const IMovieSceneAnimMixerTargetMenuProvider& A, const IMovieSceneAnimMixerTargetMenuProvider& B)
	{
		return A.GetTargetMenuPriority() < B.GetTargetMenuPriority();
	});

	// Callback when a target is selected
	auto OnTargetSelected = [this, ObjectBindings](TInstancedStruct<FMovieSceneMixedAnimationTarget> Target)
	{
		HandleAddMixerTrackWithTarget(ObjectBindings, MoveTemp(Target));
	};

	// Populate menu from all providers
	for (IMovieSceneAnimMixerTargetMenuProvider* Provider : Providers)
	{
		if (Provider)
		{
			Provider->PopulateTargetMenu(MenuBuilder, BoundObject, OnTargetSelected, Sequencer, nullptr);
		}
	}
}

void FAnimationMixerTrackEditor::HandleAddMixerTrackWithTarget(TArray<FGuid> ObjectBindings, TInstancedStruct<FMovieSceneMixedAnimationTarget> Target)
{
	UMovieScene* MovieScene = GetFocusedMovieScene();

	if (MovieScene == nullptr)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddMixerTrack_Transaction", "Add Mixer Track"));

	MovieScene->Modify();

	for (const FGuid& Guid : ObjectBindings)
	{
		UMovieSceneAnimationMixerTrack* MixerTrack = MovieScene->AddTrack<UMovieSceneAnimationMixerTrack>(Guid);
		if (MixerTrack)
		{
			MixerTrack->SetDisplayName(MixerTrack->GetDefaultDisplayName());
			MixerTrack->MixedAnimationTarget = Target;

			{
				FAnimMixerBusValidationResult Result = FAnimMixerBusUtils::ValidateBusTopology(
					AnimMixerEditorBusUtils::GatherMixerTracksForSameObject(MixerTrack, *GetSequencer()));
				if (Result.HasErrors())
				{
					Transaction.Cancel();

					FNotificationInfo Info(FText::FromString(Result.Errors[0]));
					Info.ExpireDuration = 4.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
					return;
				}
			}

			GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}
	}
}

TSharedPtr<SWidget> FAnimationMixerTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerEditorViewModel> Editor       = Params.Editor->CastThisShared<FSequencerEditorViewModel>();
	TViewModelPtr<IOutlinerExtension>        OutlinerItem = Params.ViewModel.ImplicitCast();
	if (!Editor || !OutlinerItem)
	{
		return SNullWidget::NullWidget;
	}

	// The mixer track row aggregates many heterogeneous channels (layer weights, transition
	// blend channels, root motion offsets, etc.) that the generic key path cannot represent
	// faithfully. Suppress the key columns at this level; layers and sections still expose them.
	if (ColumnName == FCommonOutlinerNames::KeyFrame || ColumnName == FCommonOutlinerNames::Nav)
	{
		if (TViewModelPtr<FAnimationMixerTrackModel> TrackModel = Params.ViewModel.ImplicitCast())
		{
			return nullptr;
		}
	}

	if (ColumnName == FCommonOutlinerNames::Add)
	{
		// Don't show '+' button on layers with child tracks
		if (TViewModelPtr<ITrackRowExtension> TrackRowExtension = Params.ViewModel.ImplicitCast())
		{
			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(TrackRowExtension->GetParentTrack());
			if (MixerTrack)
			{
				const int32 LayerRowIndex = TrackRowExtension->GetRowIndex();
				UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayer(LayerRowIndex);
				if (Layer && Layer->HasChildTrack())
				{
					// For child track layers, show '+' button only if there are compatible decorations
					UMovieSceneTrack* ChildTrack = Layer->GetChildTrack();
					if (!ChildTrack || !SequencerHelpers::HasCompatibleDecorations(ChildTrack))
					{
						return SNullWidget::NullWidget;
					}

					return MakeAddButton(
						LOCTEXT("AddModifier", "Modifier"),
						FOnGetContent::CreateSP(
							this,
							&FAnimationMixerTrackEditor::BuildAddSectionSubMenu,
							TWeakViewModelPtr<IOutlinerExtension>(OutlinerItem),
							TWeakViewModelPtr<FSequencerEditorViewModel>(Editor)
						),
						Params.ViewModel);
				}
			}
		}

		return MakeAddButton(
			LOCTEXT("AddSection", "Section"),
			FOnGetContent::CreateSP(
				this,
				&FAnimationMixerTrackEditor::BuildAddSectionSubMenu,
				TWeakViewModelPtr<IOutlinerExtension>(OutlinerItem),
				TWeakViewModelPtr<FSequencerEditorViewModel>(Editor)
			),
			Params.ViewModel);
	}

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		// Only show target combo on the track itself, not on layers
		if (TViewModelPtr<FAnimationMixerTrackModel> TrackModel = Params.ViewModel.ImplicitCast())
		{
			if (UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(TrackModel->GetTrack()))
			{
				// Get the object binding for this track
				FGuid ObjectBindingGuid;
				if (TViewModelPtr<IObjectBindingExtension> ObjectBinding = Params.ViewModel->FindAncestorOfType<IObjectBindingExtension>())
				{
					ObjectBindingGuid = ObjectBinding->GetObjectGuid();
				}

				return SNew(SAnimMixerTargetComboWidget, MixerTrack, GetSequencer(), ObjectBindingGuid);
			}
		}
	}

	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

bool FAnimationMixerTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return true;
}

bool FAnimationMixerTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneAnimationMixerTrack::StaticClass());
}

const FSlateBrush* FAnimationMixerTrackEditor::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.AnimMixer");
}

TSubclassOf<UMovieSceneCommonAnimationTrack> FAnimationMixerTrackEditor::GetTrackClass() const
{
	return UMovieSceneAnimationMixerTrack::StaticClass();
}

TSharedRef<ISequencerSection> FAnimationMixerTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	
	if (SectionObject.IsA<UMovieSceneSkeletalAnimationSection>())
	{
		return MakeShared<FAnimMixerAnimationSection>(SectionObject, GetSequencer());
	}

	if (SectionObject.IsA<UMovieSceneAnimBusSection>())
	{
		return MakeShared<FAnimBusSectionInterface>(SectionObject, Track, ObjectBinding, GetSequencer());
	}

	if (FOnMakeSectionInterfaceDelegate* MakeSectionInterfaceDelegate = MakeSectionInterfaceCallbacks.Find(SectionObject.GetClass()))
	{
		if ((*MakeSectionInterfaceDelegate).IsBound())
		{
			return (*MakeSectionInterfaceDelegate).Execute(SectionObject, Track, ObjectBinding);
		}
	}

	return FMovieSceneTrackEditor::MakeSectionInterface(SectionObject, Track, ObjectBinding);
}

TSharedRef<SWidget> FAnimationMixerTrackEditor::BuildAddSectionSubMenu(TWeakViewModelPtr<IOutlinerExtension> WeakViewModel, TWeakViewModelPtr<FSequencerEditorViewModel> WeakEditor)
{
	TViewModelPtr<ITrackExtension> Track = WeakViewModel.ImplicitPin();
	TViewModelPtr<ITrackRowExtension> LayerExtension = WeakViewModel.ImplicitPin();
	UMovieSceneAnimationMixerTrack* MixerTrack = LayerExtension ? Cast<UMovieSceneAnimationMixerTrack>(LayerExtension->GetParentTrack())
													   : Track ? Cast<UMovieSceneAnimationMixerTrack>(Track->GetTrack()) : nullptr;
	if (!MixerTrack)
	{
		return SNullWidget::NullWidget;
	}


	// Lambda for creating sections with defaults
	auto CreateNewSection = [WeakViewModel, WeakEditor](FTopLevelAssetPath ClassPath)
	{
		UClass* SectionClass = FSoftClassPath(ClassPath.ToString()).TryLoadClass<UMovieSceneSection>();
		if (!SectionClass)
		{
			return;
		}

		check(SectionClass && SectionClass->IsChildOf(UMovieSceneSection::StaticClass()) && SectionClass->ImplementsInterface(UMovieSceneAnimationMixerItemInterface::StaticClass()));

		TViewModelPtr<ITrackExtension>           Track = WeakViewModel.ImplicitPin();
		TViewModelPtr<ITrackRowExtension> LayerExtension = WeakViewModel.ImplicitPin();
		TViewModelPtr<FSequencerEditorViewModel> Editor    = WeakEditor.Pin();
		TSharedPtr<ISequencer>                   Sequencer = Editor ? Editor->GetSequencer() : nullptr;

		UMovieSceneAnimationMixerTrack* MixerTrack = LayerExtension ? Cast<UMovieSceneAnimationMixerTrack>(LayerExtension->GetParentTrack())
														   : Track ? Cast<UMovieSceneAnimationMixerTrack>(Track->GetTrack()) : nullptr;
		if (Sequencer && MixerTrack)
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("AddSectionTransaction", "Add New {0} Section"), SectionClass->GetDisplayNameText()));

			MixerTrack->Modify();

			UMovieSceneSection* NewSection = NewObject<UMovieSceneSection>(MixerTrack, SectionClass, NAME_None, RF_Transactional);

			// Determine row index based on whether we're adding to a layer model or the track model
			int32 NewRowIndex;
			if (LayerExtension)
			{
				// Adding to a specific layer - use the layer's row index
				NewRowIndex = LayerExtension->GetRowIndex();
			}
			else
			{
				// Adding to the track model - find next available row
				int32 MaxRowIndex = -1;
				for (UMovieSceneSection* Section : MixerTrack->GetAllSections())
				{
					MaxRowIndex = FMath::Max(MaxRowIndex, Section->GetRowIndex());
				}

				NewRowIndex = FMath::Max(MaxRowIndex + 1, MixerTrack->GetNextChildTrackRowIndex());
			}

			NewSection->SetRowIndex(NewRowIndex);

			// Use the FSectionParameter overload to ensure lifecycle events are fired.
			// The mixer track's OnSectionAddedImpl assigns sections to a layer, which is
			// required for the section to appear in the UI.
			static_cast<UMovieSceneTrack*>(MixerTrack)->AddSection(UMovieSceneTrack::FSectionParameter{*NewSection});

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}
	};

	// Lambda for creating child tracks with defaults
	auto CreateNewChildTrack = [WeakViewModel, WeakEditor](FGuid ObjectBinding, FTopLevelAssetPath ClassPath)
	{
		UClass* TrackClass = FSoftClassPath(ClassPath.ToString()).TryLoadClass<UMovieSceneTrack>();
		if (!TrackClass)
		{
			return;
		}

		check(TrackClass && TrackClass->IsChildOf(UMovieSceneTrack::StaticClass()) && TrackClass->ImplementsInterface(UMovieSceneAnimationMixerItemInterface::StaticClass()));

		TViewModelPtr<ITrackExtension>           Track = WeakViewModel.ImplicitPin(); 
		TViewModelPtr<ITrackRowExtension> LayerExtension = WeakViewModel.ImplicitPin();

		TViewModelPtr<FSequencerEditorViewModel> Editor    = WeakEditor.Pin();
		TSharedPtr<ISequencer>                   Sequencer = Editor ? Editor->GetSequencer() : nullptr;

		UMovieSceneAnimationMixerTrack* MixerTrack = LayerExtension ? Cast<UMovieSceneAnimationMixerTrack>(LayerExtension->GetParentTrack())
														   : Track ? Cast<UMovieSceneAnimationMixerTrack>(Track->GetTrack()) : nullptr;
		if (Sequencer && MixerTrack)
		{
			// Cannot add child tracks to a non-empty layer
			if (LayerExtension)
			{
				const int32 LayerRowIndex = LayerExtension->GetRowIndex();
				UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayer(LayerRowIndex);
				if (Layer && !Layer->IsEmpty())
				{
					// Layer already has content - cannot add a child track
					return;
				}
			}

			FScopedTransaction Transaction(FText::Format(LOCTEXT("AddChildTrackTransaction", "Add New {0} Child Track"), TrackClass->GetDisplayNameText()));

			MixerTrack->Modify();

			// Determine row index based on whether we're adding to a layer model or the track model
			int32 NewRowIndex;
			if (LayerExtension)
			{
				// Adding to a specific layer - use the layer's row index
				NewRowIndex = LayerExtension->GetRowIndex();
			}
			else
			{
				// Adding to the track model - find next available row
				int32 MaxRowIndex = -1;
				for (UMovieSceneSection* Section : MixerTrack->GetAllSections())
				{
					MaxRowIndex = FMath::Max(MaxRowIndex, Section->GetRowIndex());
				}

				NewRowIndex = FMath::Max(MaxRowIndex + 1, MixerTrack->GetNextChildTrackRowIndex());
			}

			UMovieSceneTrack* NewChildTrack = MixerTrack->AddChildTrack(ObjectBinding, TrackClass, NewRowIndex);

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}
	};

	FMenuBuilder MenuBuilder(true, nullptr);

	// Layer management
	// Only show on the track-level menu (not on individual layer menus)
	if (!LayerExtension.IsValid())
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("LayerManagement_Label", "Layer Management"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddLayer", "Layer"),
				LOCTEXT("AddLayer_Tooltip", "Add an empty layer at the end of the track"),
				FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.MixLayer"),
				FUIAction(FExecuteAction::CreateLambda([WeakViewModel, WeakEditor]()
				{
					TViewModelPtr<ITrackExtension>           Track     = WeakViewModel.ImplicitPin();
					TViewModelPtr<FSequencerEditorViewModel> Editor    = WeakEditor.Pin();
					TSharedPtr<ISequencer>                   Sequencer = Editor ? Editor->GetSequencer() : nullptr;

					UMovieSceneAnimationMixerTrack* MixerTrack = Track ? Cast<UMovieSceneAnimationMixerTrack>(Track->GetTrack()) : nullptr;
					if (Sequencer && MixerTrack)
					{
						FScopedTransaction Transaction(LOCTEXT("AddLayerTransaction", "Add Layer"));
						MixerTrack->Modify();

						// Compute next available row index
						int32 NextRowIndex = MixerTrack->ComputeNextAvailableRowIndex();

						// Create an empty layer at the next available row
						MixerTrack->GetOrCreateLayer(NextRowIndex);

						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					}
				}))
			);
		}
		MenuBuilder.EndSection();
	}
	else
	{
		// Insert layer options - only show on layer rows
		const int32 CurrentLayerIndex = LayerExtension->GetRowIndex();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("LayerManagement_Label", "Layer Management"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("InsertLayerAbove", "Insert Layer Above"),
				LOCTEXT("InsertLayerAbove_Tooltip", "Insert a new empty layer above this one"),
				FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.MixLayer"),
				FUIAction(FExecuteAction::CreateLambda([WeakViewModel, WeakEditor, CurrentLayerIndex]()
				{
					TViewModelPtr<ITrackRowExtension>        LayerExt  = WeakViewModel.ImplicitPin();
					TViewModelPtr<FSequencerEditorViewModel> Editor    = WeakEditor.Pin();
					TSharedPtr<ISequencer>                   Sequencer = Editor ? Editor->GetSequencer() : nullptr;

					UMovieSceneAnimationMixerTrack* MixerTrack = LayerExt ? Cast<UMovieSceneAnimationMixerTrack>(LayerExt->GetParentTrack()) : nullptr;
					if (Sequencer && MixerTrack)
					{
						FScopedTransaction Transaction(LOCTEXT("InsertLayerAboveTransaction", "Insert Layer Above"));
						MixerTrack->Modify();

						MixerTrack->InsertLayer(CurrentLayerIndex);

						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
					}
				}))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("InsertLayerBelow", "Insert Layer Below"),
				LOCTEXT("InsertLayerBelow_Tooltip", "Insert a new empty layer below this one"),
				FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.MixLayer"),
				FUIAction(FExecuteAction::CreateLambda([WeakViewModel, WeakEditor, CurrentLayerIndex]()
				{
					TViewModelPtr<ITrackRowExtension>        LayerExt  = WeakViewModel.ImplicitPin();
					TViewModelPtr<FSequencerEditorViewModel> Editor    = WeakEditor.Pin();
					TSharedPtr<ISequencer>                   Sequencer = Editor ? Editor->GetSequencer() : nullptr;

					UMovieSceneAnimationMixerTrack* MixerTrack = LayerExt ? Cast<UMovieSceneAnimationMixerTrack>(LayerExt->GetParentTrack()) : nullptr;
					if (Sequencer && MixerTrack)
					{
						FScopedTransaction Transaction(LOCTEXT("InsertLayerBelowTransaction", "Insert Layer Below"));
						MixerTrack->Modify();

						MixerTrack->InsertLayer(CurrentLayerIndex + 1);

						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
					}
				}))
			);
		}
		MenuBuilder.EndSection();
	}

	// Time Warp entry — anchored on any time-warp-capable section in the layer. For layers that
	// wrap a child track (e.g. Control Rig), the sections live on the child track, and the
	// section owner MakeTimeWarpMenuEntry needs is the child track model rather than the layer.
	if (LayerExtension)
	{
		auto FindTimeWarpSection = [](auto&& Sections) -> UMovieSceneSection*
		{
			for (UMovieSceneSection* Section : Sections)
			{
				if (Section && Section->GetTimeWarp())
				{
					return Section;
				}
			}
			return nullptr;
		};

		UMovieSceneSection* TimeWarpAnchor = nullptr;
		if (UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayer(LayerExtension->GetRowIndex()))
		{
			if (Layer->HasChildTrack())
			{
				if (UMovieSceneTrack* ChildTrack = Layer->GetChildTrack())
				{
					TimeWarpAnchor = FindTimeWarpSection(ChildTrack->GetAllSections());
				}
			}
			else
			{
				TimeWarpAnchor = FindTimeWarpSection(Layer->GetSections());
			}
		}

		TViewModelPtr<ISectionOwnerExtension> SectionOwner;
		if (TimeWarpAnchor)
		{
			TSharedPtr<FSequencerEditorViewModel> EditorVM       = WeakEditor.Pin();
			FViewModelPtr                         RootModel      = EditorVM ? EditorVM->GetRootModel() : nullptr;
			FSectionModelStorageExtension*        SectionStorage = RootModel ? RootModel->CastDynamic<FSectionModelStorageExtension>() : nullptr;
			TSharedPtr<FSectionModel>             AnchorModel    = SectionStorage ? SectionStorage->FindModelForSection(TimeWarpAnchor) : nullptr;
			if (AnchorModel)
			{
				SectionOwner = AnchorModel->GetParentSectionOwnerModel();
			}
		}

		if (SectionOwner)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarp_Label", "Time Warp"));
			FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, SectionOwner);
			MenuBuilder.EndSection();
		}
	}

	// Discover all item classes (sections and tracks) via Asset Registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TSet<FTopLevelAssetPath> AllItemClasses;

	{
		// Get all sections
		FTopLevelAssetPath SectionClassPath(UMovieSceneSection::StaticClass());
		TSet<FTopLevelAssetPath> SectionClasses;
		AssetRegistryModule.Get().GetDerivedClassNames({ SectionClassPath }, TSet<FTopLevelAssetPath>(), SectionClasses);
		AllItemClasses.Append(SectionClasses);

		// Get all tracks
		FTopLevelAssetPath TrackClassPath(UMovieSceneTrack::StaticClass());
		TSet<FTopLevelAssetPath> TrackClasses;
		AssetRegistryModule.Get().GetDerivedClassNames({ TrackClassPath }, TSet<FTopLevelAssetPath>(), TrackClasses);
		AllItemClasses.Append(TrackClasses);
	}

	// Filter to only classes implementing the mixer item interface
	// Also exclude transition sections - they are created via the transition context menu, not the add menu
	for (auto It = AllItemClasses.CreateIterator(); It; ++It)
	{
		const UClass* Class = FSoftClassPath(It->ToString()).ResolveClass();
		if (!Class || Class->HasMetaData("Hidden") || !Class->ImplementsInterface(UMovieSceneAnimationMixerItemInterface::StaticClass()))
		{
			It.RemoveCurrent();
		}
		else
		{
			IMovieSceneAnimationMixerItemInterface* MixerItem = Cast<IMovieSceneAnimationMixerItemInterface>(Class->GetDefaultObject());
			if (MixerItem && !MixerItem->IsVisibleInAddSectionMenu())
			{
				// Transition sections are created via context menu, not the add menu, and decoration owned sections are added in the decoration menu
				It.RemoveCurrent();
			}
		}
	}

	// Find all registered menu providers
	TArray<IMovieSceneAnimMixerItemMenuProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimMixerItemMenuProvider>(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName());

	// Sort providers by priority (lower values appear first)
	Providers.Sort([](const IMovieSceneAnimMixerItemMenuProvider& A, const IMovieSceneAnimMixerItemMenuProvider& B)
	{
		return A.GetMixerItemMenuPriority() < B.GetMixerItemMenuPriority();
	});

	TSet<const UClass*> HandledClasses;

	TViewModelPtr<FSequencerEditorViewModel> Editor = WeakEditor.Pin();
	TSharedPtr<ISequencer> Sequencer = Editor ? Editor->GetSequencer() : nullptr;

	FViewModelPtr Model = WeakViewModel.Pin();
	TViewModelPtr<IObjectBindingExtension> ObjectBinding = Model ? Model->FindAncestorOfType<IObjectBindingExtension>() : nullptr;
	FGuid ObjectBindingGuid = ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid();

	TArray<FGuid> ObjectBindings;
	if (ObjectBindingGuid.IsValid())
	{
		ObjectBindings.Add(ObjectBindingGuid);
	}

	// Determine target row index for menu provider callbacks
	int32 TargetRowIndex = INDEX_NONE;
	bool bCanAddChildTracks = true;

	bool bHasChildTrack = false;
	if (LayerExtension)
	{
		// Adding to a specific layer - use the layer's row index
		TargetRowIndex = LayerExtension->GetRowIndex();

		// Check if we can add child tracks (only if not on a non-empty layer)
		UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayer(TargetRowIndex);
		if (Layer && !Layer->IsEmpty())
		{
			bCanAddChildTracks = false;
		}
		if (Layer && Layer->HasChildTrack())
		{
			bHasChildTrack = true;
		}
	}
	else
	{
		// Adding to the track model - use INDEX_NONE for auto-row assignment
		TargetRowIndex = INDEX_NONE;
	}

	// Animation items - skip for child track layers since the child track occupies the entire layer
	if (!bHasChildTrack && (AllItemClasses.Num() > 0 || Providers.Num() > 0))
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("AnimationCategoryLabel", "Animation:"));

		// First, call all menu providers
		for (IMovieSceneAnimMixerItemMenuProvider* Provider : Providers)
		{
			if (Provider)
			{
				const UClass* HandledClass = Provider->GetHandledMixerItemClass();
				if (HandledClass)
				{
					// Skip child track providers if we can't add child tracks to this layer
					const bool bIsTrackProvider = HandledClass->IsChildOf(UMovieSceneTrack::StaticClass());
					if (bIsTrackProvider && !bCanAddChildTracks)
					{
						HandledClasses.Add(HandledClass);
						continue;
					}

					HandledClasses.Add(HandledClass);
					Provider->PopulateAddMixerItemMenu(MenuBuilder, ObjectBindings, MixerTrack, Sequencer, TargetRowIndex);
				}
			}
		}

		// Then, add generic entries for items without menu providers
		for (const FTopLevelAssetPath& ClassPath : AllItemClasses)
		{
			const UClass* Class = FSoftClassPath(ClassPath.ToString()).ResolveClass();
			if (!Class || HandledClasses.Contains(Class))
			{
				continue;
			}

			// Get display info from interface
			const UObject* CDO = Class->GetDefaultObject();
			const IMovieSceneAnimationMixerItemInterface* ItemInterface = Cast<IMovieSceneAnimationMixerItemInterface>(CDO);

			if (ItemInterface)
			{
				FText DisplayName = ItemInterface->GetDisplayName();
				if (DisplayName.IsEmpty())
				{
					DisplayName = Class->GetDisplayNameText();
				}

				FText Description = ItemInterface->GetDescription();
				if (Description.IsEmpty())
				{
					Description = Class->GetToolTipText();
				}

				// Determine if this is a section or track and use appropriate creation lambda
				bool bIsSection = Class->IsChildOf(UMovieSceneSection::StaticClass());
				bool bIsTrack = Class->IsChildOf(UMovieSceneTrack::StaticClass());

				if (bIsSection)
				{
					MenuBuilder.AddMenuEntry(
						DisplayName,
						Description,
						ItemInterface->GetIcon(),
						FUIAction(FExecuteAction::CreateLambda(CreateNewSection, ClassPath))
					);
				}
				else if (bIsTrack)
				{
					// Don't show child track options if adding to a non-empty layer
					if (bCanAddChildTracks)
					{
						MenuBuilder.AddMenuEntry(
							DisplayName,
							Description,
							ItemInterface->GetIcon(),
							FUIAction(FExecuteAction::CreateLambda(CreateNewChildTrack, ObjectBindingGuid, ClassPath))
						);
					}
				}
			}
		}

		MenuBuilder.EndSection();
	}

	// Add decoration/modifier menu entries for compatible decorations
	// For track-level: add decorations to the track itself
	// For layer-level: add decorations to the layer and its sections
	if (LayerExtension)
	{
		// Layer-level: get the layer and add its compatible decorations
		UMovieSceneAnimationMixerLayer* Layer = MixerTrack->GetLayer(LayerExtension->GetRowIndex());
		if (Layer)
		{
			SequencerHelpers::BuildDecorationMenu(MenuBuilder, Layer, ObjectBindingGuid, Sequencer,
				LOCTEXT("LayerModifiers", "Layer Modifiers"));

			if (Layer->HasChildTrack())
			{
				// Show decorations compatible with the child track (e.g., root motion settings)
				SequencerHelpers::BuildDecorationMenu(MenuBuilder, Layer->GetChildTrack(), ObjectBindingGuid, Sequencer,
					LOCTEXT("TrackModifiers", "Track Modifiers"));
			}
			else
			{
				// For single-section layers, also show decorations compatible with the section
				TArray<UMovieSceneSection*> LayerSections = Layer->GetSections();
				if (LayerSections.Num() == 1 && LayerSections[0])
				{
					SequencerHelpers::BuildDecorationMenu(MenuBuilder, LayerSections[0], ObjectBindingGuid, Sequencer);
				}
			}
		}
	}
	else
	{
		// Track-level: add decorations to the track
		SequencerHelpers::BuildDecorationMenu(MenuBuilder, MixerTrack, ObjectBindingGuid, Sequencer);
	}

	return MenuBuilder.MakeWidget();
}

void FAnimationMixerTrackEditor::HandleAddAnimationTrackMenuEntryExecute(TArray<FGuid> ObjectBindings)
{
	UMovieScene* MovieScene = GetFocusedMovieScene();

	if (MovieScene == nullptr)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddAnimationTrack_Transaction", "Add Animation Track"));

	MovieScene->Modify();

	for (const FGuid& Guid : ObjectBindings)
	{
		FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(Guid, UMovieSceneAnimationMixerTrack::StaticClass());
		if (TrackResult.bWasCreated)
		{
			UMovieSceneAnimationMixerTrack* AnimTrack = CastChecked<UMovieSceneAnimationMixerTrack>(TrackResult.Track);
			AnimTrack->SetDisplayName(AnimTrack->GetDefaultDisplayName());
		}
	}
}

void FAnimationMixerTrackEditor::OnCreateAdditionalTrailHierarchies(
	TArray<TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>>& TrailHierarchies,
	TWeakPtr<ISequencer> WeakSequencer)
{
	// Only add mixer trail hierarchies for our own sequencer
	TSharedPtr<ISequencer> OurSequencer = GetSequencer();
	TSharedPtr<ISequencer> TargetSequencer = WeakSequencer.Pin();
	if (!OurSequencer || !TargetSequencer || OurSequencer != TargetSequencer)
	{
		return;
	}

	MixerTrailHierarchy = MakeShared<FAnimMixerTrailHierarchy>(WeakSequencer);
	MixerTrailHierarchy->Initialize();
	TrailHierarchies.Add(MixerTrailHierarchy);

	// Give the edit mode a weak reference to the hierarchy so it can check for trail key selection
	if (UAnimationMixerTrackEditMode* MixerEditMode = Cast<UAnimationMixerTrackEditMode>(
			GLevelEditorModeTools().GetActiveScriptableMode(UAnimationMixerTrackEditMode::ModeName)))
	{
		MixerEditMode->SetTrailHierarchy(MixerTrailHierarchy);
	}
}

void FAnimationMixerTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Track);
	if (!MixerTrack) return;

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr) return;

	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence() ? SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
	if (!MovieScene) return;

	FGuid ObjectBinding;
	if (!MovieScene->FindTrackBinding(*MixerTrack, ObjectBinding)) return;

	USkeletalMeshComponent* SkelMeshComp = AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);
	if (!SkelMeshComp || !SkelMeshComp->GetSkeletalMeshAsset()) return;

	USkeleton* Skeleton = SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
	if (!Skeleton) return;

	UObject* BoundObject = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);

	MenuBuilder.BeginSection("AnimMixerBake", LOCTEXT("AnimMixerBake", "Bake"));
	{
		UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter LayerFilter;
		LayerFilter.bSkipRootMotionConversion = false;
		UE::Sequencer::AnimMixerBake::BuildBakeMenuSection(
			MenuBuilder, SequencerPtr, MixerTrack, SkelMeshComp, Skeleton,
			ObjectBinding, BoundObject, bFilterAssetBySkeleton,
			LayerFilter, // empty filter = whole track
			// ChildTrackFactory: create bake track as a child track at the end of the mixer
			[WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack), ObjectBinding]
			(UMovieScene* InMovieScene, TSubclassOf<UMovieSceneTrack> TrackClass) -> UMovieSceneTrack*
			{
				UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
				if (!MixerTrack) { return nullptr; }
				MixerTrack->Modify();
				int32 NewRowIndex = MixerTrack->ComputeNextAvailableRowIndex();
				return MixerTrack->AddChildTrack(ObjectBinding, TrackClass, NewRowIndex);
			},
			// Section layers get muted via SetRowEvalDisabled on the mixer track; child-track layers
			// own their own UMovieSceneTrack and must be disabled directly.
			[this, WeakMixerTrack = TWeakObjectPtr<UMovieSceneAnimationMixerTrack>(MixerTrack)]
			(UMovieSceneTrack* /*BakeTrack*/, UObject* /*ControlRigObj*/, bool bSuccess)
			{
				TSharedPtr<ISequencer> Seq = GetSequencer();
				UMovieSceneAnimationMixerTrack* MixerTrack = WeakMixerTrack.Get();
				if (!Seq || !MixerTrack || !bSuccess) return;

				const int32 NewLayerIndex = MixerTrack->ComputeNextAvailableRowIndex() - 1;
				MixerTrack->Modify();
				for (const TObjectPtr<UMovieSceneAnimationMixerLayer>& Layer : MixerTrack->GetLayers())
				{
					if (!Layer || Layer->GetLayerIndex() >= NewLayerIndex) { continue; }
					if (UMovieSceneTrack* ChildTrack = Layer->GetChildTrack())
					{
						MovieSceneHelpers::DisableTrack(ChildTrack);
					}
					else
					{
						MixerTrack->SetRowEvalDisabled(true, Layer->GetLayerIndex());
					}
				}

				Seq->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			});
		MenuBuilder.AddSeparator();
	}
	MenuBuilder.EndSection();
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

