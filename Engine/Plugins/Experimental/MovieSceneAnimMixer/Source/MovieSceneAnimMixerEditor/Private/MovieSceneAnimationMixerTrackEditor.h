// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Containers/Ticker.h"
#include "MovieSceneTrackEditor.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "StructUtils/InstancedStruct.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"
#include "MixerLinkedAnimTrackProvider.h"

struct FTimeToPixel;
class UMovieSceneAnimationMixerTrack;

namespace UE::SequencerAnimTools { class FTrailHierarchy; }
namespace UE::Sequencer { class FAnimMixerTrailHierarchy; }

DECLARE_DELEGATE_RetVal_ThreeParams(TSharedRef<ISequencerSection>, FOnMakeSectionInterfaceDelegate, UMovieSceneSection&, UMovieSceneTrack&, FGuid);

namespace UE::Sequencer
{

// Section interface for skeletal animation sections within a mixer track. Handles transition
// padding, context menus for creating transitions between overlapping sections, and
// disables the legacy bone matching UI. 
class MOVIESCENEANIMMIXEREDITOR_API FAnimMixerAnimationSection
	: public FCommonAnimationSection
{
public:

	FAnimMixerAnimationSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
		: FCommonAnimationSection(InSection, InSequencer)
	{
	}

	virtual bool ShouldShowLegacyBoneMatching() const override { return false; }
	virtual const FSlateBrush* GetIconBrush() const override;

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual FMargin GetContentPadding() const override;
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;

	void PopulateCreateTransitionMenu(FMenuBuilder& MenuBuilder);
	void PopulateTransitionTypeMenu(FMenuBuilder& MenuBuilder, UMovieSceneSection* OtherSection, TArray<UClass*> TransitionClasses);
	void CreateTransitionToSection(UMovieSceneSection* FromSection, UMovieSceneSection* ToSection, UClass* TransitionClass);

private:

	float CalculateTransitionPadding(const FTimeToPixel& TimeConverter) const;

	// Cached transition padding calculated in OnPaintSection for use in GetContentPadding
	mutable float CachedTransitionPaddingLeft = 0.0f;
};

class MOVIESCENEANIMMIXEREDITOR_API FAnimationMixerTrackEditor
	: public FCommonAnimationTrackEditor
	, public IMovieSceneAnimMixerItemMenuProvider
{
public:

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);


	static void RegisterCustomMixerAnimSection(const UClass* InClass, FOnMakeSectionInterfaceDelegate InMakeSectionInterfaceDelegate)
	{
		MakeSectionInterfaceCallbacks.Add(TTuple<const UClass*, FOnMakeSectionInterfaceDelegate>{InClass, InMakeSectionInterfaceDelegate});
	}

	static void UnregisterCustomMixerAnimSection(const UClass* InClass)
	{
		MakeSectionInterfaceCallbacks.Remove(InClass);
	}

public:

	FAnimationMixerTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void OnInitialize() override;
	virtual void OnRelease() override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual TSubclassOf<UMovieSceneCommonAnimationTrack> GetTrackClass() const override;
	virtual bool ShouldActivateSkeletalAnimEditMode() const override { return false; }
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

	// IMovieSceneAnimMixerItemMenuProvider interface
	virtual const UClass* GetHandledMixerItemClass() const override;
	virtual void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) override;
	virtual void PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu) override;
	virtual int32 GetObjectBindingAnimationMenuPriority() const override { return 0; } // Highest priority - appears first

private:
	using FCommonAnimationTrackEditor::BuildAddAnimationSubMenu;

	TSharedRef<SWidget> BuildAddSectionSubMenu(TWeakViewModelPtr<IOutlinerExtension> WeakViewModel, TWeakViewModelPtr<FSequencerEditorViewModel> WeakEditor);
	void HandleAddAnimationTrackMenuEntryExecute(TArray<FGuid> ObjectBindings);
	void HandleAddMixerTrackWithTarget(TArray<FGuid> ObjectBindings, TInstancedStruct<FMovieSceneMixedAnimationTarget> Target);
	void BuildTargetSelectionMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, const UClass* ObjectClass);
	void PopulateAddAnimationMenu(FMenuBuilder& MenuBuilder, TWeakViewModelPtr<IOutlinerExtension> WeakViewModel, TWeakViewModelPtr<FSequencerEditorViewModel> WeakEditor);

	bool FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton);
	void OnAnimationAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track);
	void OnAnimationAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track);

	void OnCreateAdditionalTrailHierarchies(
		TArray<TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>>& TrailHierarchies,
		TWeakPtr<ISequencer> WeakSequencer);

	FDelegateHandle AdditionalTrailHierarchiesHandle;
	TSharedPtr<FAnimMixerTrailHierarchy> MixerTrailHierarchy;

	bool bRegisteredLevelEditorMode = false;

	static int32 MixerEditModeRefCount;
	static void RegisterMakeSectionInterfaceCallback(const UClass* SectionClass, FOnMakeSectionInterfaceDelegate Callback);
	static void UnregisterMakeSectionInterfaceCallback(const UClass* SectionClass);

public:
	// Filter state for Bake To Control Rig (passed by ref to shared helpers; public for layer model access)
	bool bFilterAssetBySkeleton = true;

	static TMap<const UClass*, FOnMakeSectionInterfaceDelegate> MakeSectionInterfaceCallbacks;
	TUniquePtr<FMixerLinkedAnimTrackProvider> MixerLinkedAnimProvider;
};


} // namespace UE::Sequencer
