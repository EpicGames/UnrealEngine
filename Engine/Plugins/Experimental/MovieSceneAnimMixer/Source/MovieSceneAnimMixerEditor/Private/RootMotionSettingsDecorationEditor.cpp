// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionSettingsDecorationEditor.h"

#include "MovieSceneAnimMixerEditorStyle.h"
#include "AnimMixerBakeEvaluation.h"
#include "AnimMixerBoneMatching.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneSection.h"
#include "SBoneMatchDialog.h"
#include "ScopedTransaction.h"
#include "SEnumCombo.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "MovieSceneToolHelpers.h"
#include "MVVM/Views/SKeyNavigationButtons.h"
#include "Systems/MovieSceneRootMotionSystem.h"

#define LOCTEXT_NAMESPACE "RootMotionSettingsDecorationEditor"

namespace UE::Sequencer
{

// Returns true if the given section has at least one overlapping non-transition
// section on the same mixer track that wouldn't create a bone match dependency cycle.
static bool HasOverlappingReferenceSections(UMovieSceneSection* Section)
{
	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return false;
	}

	for (UMovieSceneSection* Other : MixerTrack->GetAllSections())
	{
		if (Other && Other != Section
			&& !Other->IsA<UMovieSceneAnimTransitionSectionBase>()
			&& !TRange<FFrameNumber>::Intersection(Section->GetRange(), Other->GetRange()).IsEmpty()
			&& !WouldCreateBoneMatchCycle(Other, Section))
		{
			return true;
		}
	}
	return false;
}

UClass* FRootMotionSettingsDecorationEditor::GetDecorationClass() const
{
	return UMovieSceneRootMotionSettingsDecoration::StaticClass();
}

const FSlateBrush* FRootMotionSettingsDecorationEditor::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.RootMotionSettings");
}

FSlateIcon FRootMotionSettingsDecorationEditor::GetMenuIcon() const
{
	return FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.RootMotionSettings");
}

void FRootMotionSettingsDecorationEditor::OnSectionInterfaceCreated(
	UMovieSceneSection& Section,
	UObject& Decoration,
	TWeakPtr<ISequencer> Sequencer)
{
	UMovieSceneRootMotionSettingsDecoration* Settings = Cast<UMovieSceneRootMotionSettingsDecoration>(&Decoration);
	if (!Settings)
	{
		return;
	}

	if (TrackedDecorations.Contains(Settings))
	{
		return;
	}

	Settings->BoneMatchChannel.OnKeyMovedEvent().AddRaw(this, &FRootMotionSettingsDecorationEditor::HandleBoneMatchKeyMoved);
	Settings->BoneMatchChannel.OnKeyDeletedEvent().AddRaw(this, &FRootMotionSettingsDecorationEditor::HandleBoneMatchKeyDeleted);

	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakDecoration(Settings);

	FTrackedSectionInfo Info;
	Info.Section = &Section;
	Info.DecorationSigHandle = Settings->OnSignatureChanged().AddRaw(this,
		&FRootMotionSettingsDecorationEditor::HandleDecorationSignatureChanged, WeakDecoration);
	// Section moves/resizes don't always touch the decoration's signature.
	Info.SectionSigHandle = Section.OnSignatureChanged().AddRaw(this,
		&FRootMotionSettingsDecorationEditor::HandleDecorationSignatureChanged, WeakDecoration);
	Info.LastDecorationSignature = Settings->GetSignature();
	Info.LastSectionSignature = Section.GetSignature();

	TrackedDecorations.Add(Settings, MoveTemp(Info));
}

FRootMotionSettingsDecorationEditor::FTrackedSectionInfo* FRootMotionSettingsDecorationEditor::FindTrackedInfoForChannel(
	FMovieSceneChannel* Channel, UMovieSceneRootMotionSettingsDecoration*& OutDecoration)
{
	OutDecoration = nullptr;
	for (TPair<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>, FTrackedSectionInfo>& Pair : TrackedDecorations)
	{
		UMovieSceneRootMotionSettingsDecoration* Decoration = Pair.Key.Get();
		if (Decoration && &Decoration->BoneMatchChannel == Channel)
		{
			OutDecoration = Decoration;
			return &Pair.Value;
		}
	}
	return nullptr;
}

FRootMotionSettingsDecorationEditor::~FRootMotionSettingsDecorationEditor()
{
	for (TPair<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>, FTrackedSectionInfo>& Pair : TrackedDecorations)
	{
		if (UMovieSceneRootMotionSettingsDecoration* Decoration = Pair.Key.Get())
		{
			Decoration->BoneMatchChannel.OnKeyMovedEvent().RemoveAll(this);
			Decoration->BoneMatchChannel.OnKeyDeletedEvent().RemoveAll(this);
			if (Pair.Value.DecorationSigHandle.IsValid())
			{
				Decoration->OnSignatureChanged().Remove(Pair.Value.DecorationSigHandle);
			}
		}
		if (UMovieSceneSection* Section = Pair.Value.Section.Get())
		{
			if (Pair.Value.SectionSigHandle.IsValid())
			{
				Section->OnSignatureChanged().Remove(Pair.Value.SectionSigHandle);
			}
		}
	}
}

void FRootMotionSettingsDecorationEditor::HandleBoneMatchKeyMoved(FMovieSceneChannel* Channel, const TArray<FKeyMoveEventItem>& Items)
{
	using namespace UE::MovieScene;

	UMovieSceneRootMotionSettingsDecoration* Decoration = nullptr;
	FTrackedSectionInfo* Info = FindTrackedInfoForChannel(Channel, Decoration);
	if (!Info)
	{
		return;
	}

	UMovieSceneSection* Section = Info->Section.Get();
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!Section || !SequencerPtr || !Decoration || Items.IsEmpty())
	{
		return;
	}

	if (!Decoration->HasBoneMatch())
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return;
	}

	TMovieSceneChannelData<FMovieSceneBoneMatchData> Data = Decoration->BoneMatchChannel.GetData();
	TArrayView<FMovieSceneBoneMatchData> Values = Data.GetValues();
	if (Items[0].NewIndex < 0 || Items[0].NewIndex >= Values.Num())
	{
		return;
	}

	FMovieSceneBoneMatchData& KeyValue = Values[Items[0].NewIndex];
	const FFrameNumber KeyTime = Items[0].NewFrame;

	// Validate: key time must be within the target section's range
	bool bValid = Section->GetRange().Contains(KeyTime);

	// Validate: the underlying mix must have at least one contributing
	// section at the key time for the match to be meaningful.
	// If a specific reference section is set, the key must be within
	// its overlap with the target. Otherwise (AtCurrentTime mode),
	// any section on the same layer or below that covers the key time
	// counts.
	if (bValid)
	{
		UMovieSceneSection* Ref = KeyValue.ReferenceSection.Get();
		if (Ref)
		{
			TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(Section->GetRange(), Ref->GetRange());
			if (Overlap.IsEmpty() || !Overlap.Contains(KeyTime))
			{
				bValid = false;
			}
		}
		else
		{
			const int32 TargetRow = Section->GetRowIndex();
			bool bHasUnderlyingPose = false;
			for (UMovieSceneSection* Other : MixerTrack->GetAllSections())
			{
				if (Other && Other != Section
					&& !Other->IsA<UMovieSceneAnimTransitionSectionBase>()
					&& Other->GetRowIndex() <= TargetRow
					&& Other->GetRange().Contains(KeyTime))
				{
					bHasUnderlyingPose = true;
					break;
				}
			}
			if (!bHasUnderlyingPose)
			{
				bValid = false;
			}
		}
	}

	if (!bValid)
	{
		KeyValue.bIsValid = false;
		KeyValue.bIsDirty = true;
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		return;
	}

	AnimMixerBoneMatching::FBoneMatchingContext Context;
	Context.Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	Context.InstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();
	Context.MixerTrack = MixerTrack;
	Context.CurrentTime = FFrameTime(KeyTime);

	FMovieSceneBoneMatchData Result = AnimMixerBoneMatching::ComputeBoneMatch(
		Section, KeyValue, Context);

	KeyValue = Result;

	// Propagate rematches through dependency chain (A->B->C)
	if (Result.bIsValid)
	{
		TSet<UMovieSceneSection*> Visited;
		Visited.Add(Section);
		AnimMixerBoneMatching::PropagateRematch(Section, Context, Visited);
	}

	// RefreshImmediately so the viewport returns to the playhead pose after the bake.
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

void FRootMotionSettingsDecorationEditor::HandleBoneMatchKeyDeleted(FMovieSceneChannel* Channel, const TArray<FKeyAddOrDeleteEventItem>& Items)
{
	UMovieSceneRootMotionSettingsDecoration* Decoration = nullptr;
	FTrackedSectionInfo* Info = FindTrackedInfoForChannel(Channel, Decoration);
	if (!Info)
	{
		return;
	}

	UMovieSceneSection* Section = Info->Section.Get();
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!Section || !SequencerPtr)
	{
		return;
	}

	Section->InvalidateChannelProxy();
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

void FRootMotionSettingsDecorationEditor::HandleDecorationSignatureChanged(
	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakDecoration)
{
	if (bHandlingSignatureChange || !WeakDecoration.IsValid())
	{
		return;
	}

	// Defer to Tick: Modify() broadcasts OnSignatureChanged BEFORE the channel mutation,
	// so post-mutation state isn't readable until the next frame.
	PendingRematches.Add(WeakDecoration);
}

void FRootMotionSettingsDecorationEditor::Tick(float /*DeltaTime*/)
{
	using namespace UE::MovieScene;

	for (auto It = TrackedDecorations.CreateIterator(); It; ++It)
	{
		if (!It.Key().Get() || !It.Value().Section.Get())
		{
			It.RemoveCurrent();
		}
	}

	if (PendingRematches.IsEmpty())
	{
		return;
	}

	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!SequencerPtr)
	{
		PendingRematches.Reset();
		return;
	}

	TSet<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> ToProcess = MoveTemp(PendingRematches);
	PendingRematches.Reset();

	// Collect work under the reentrance guard so signature events fired during
	// collection don't re-enter Tick. Open the transaction only if we have work.
	struct FRematchWork
	{
		UMovieSceneSection* Section;
		UMovieSceneAnimationMixerTrack* MixerTrack;
	};
	TArray<FRematchWork, TInlineAllocator<4>> Work;

	{
		TGuardValue<bool> ReentranceGuard(bHandlingSignatureChange, true);

		for (const TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>& WeakDecoration : ToProcess)
		{
			UMovieSceneRootMotionSettingsDecoration* Decoration = WeakDecoration.Get();
			if (!Decoration)
			{
				continue;
			}

			FTrackedSectionInfo* Info = TrackedDecorations.Find(WeakDecoration);
			if (!Info)
			{
				continue;
			}

			UMovieSceneSection* Section = Info->Section.Get();
			if (!Section)
			{
				continue;
			}

			// Skip if neither signature actually changed since last rematch
			// (re-broadcasts from outer-chain regen don't represent a real edit).
			const FGuid CurrentDecorationSignature = Decoration->GetSignature();
			const FGuid CurrentSectionSignature = Section->GetSignature();
			if (CurrentDecorationSignature == Info->LastDecorationSignature
				&& CurrentSectionSignature == Info->LastSectionSignature)
			{
				continue;
			}
			Info->LastDecorationSignature = CurrentDecorationSignature;
			Info->LastSectionSignature = CurrentSectionSignature;

			UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
			if (!MixerTrack)
			{
				continue;
			}

			bool bHasDependentSection = false;
			for (UMovieSceneSection* Other : MixerTrack->GetAllSections())
			{
				if (!Other || Other == Section || Other->IsA<UMovieSceneAnimTransitionSectionBase>())
				{
					continue;
				}
				UMovieSceneRootMotionSettingsDecoration* OtherDecoration = Other->FindDecoration<UMovieSceneRootMotionSettingsDecoration>();
				if (!OtherDecoration || !OtherDecoration->HasBoneMatch())
				{
					continue;
				}

				TMovieSceneChannelData<FMovieSceneBoneMatchData> OtherData = OtherDecoration->BoneMatchChannel.GetData();
				TArrayView<const FFrameNumber> OtherTimes = OtherData.GetTimes();
				if (OtherTimes.IsEmpty())
				{
					continue;
				}
				const FFrameNumber OtherKeyTime = OtherTimes[0];
				UMovieSceneSection* OtherRef = OtherDecoration->GetBoneMatchData().ReferenceSection.Get();

				if (AnimMixerBoneMatching::DoesSectionAffectBoneMatch(Section, Other, OtherKeyTime, OtherRef))
				{
					bHasDependentSection = true;
					break;
				}
			}

			if (bHasDependentSection)
			{
				Work.Add({ Section, MixerTrack });
			}
		}
	}

	if (Work.IsEmpty())
	{
		return;
	}

	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!Linker)
	{
		return;
	}

	TGuardValue<bool> ReentranceGuard(bHandlingSignatureChange, true);

	const FInstanceHandle RootInstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();

	// Rebuild the accumulated-offset cache up front. The bone-match bakes
	// below run inside an FBakeEvaluationScope, and PreFlush skips cache
	// rebuilds while bake is active.
	TSet<UMovieSceneAnimationMixerTrack*> SeenTracks;
	for (const FRematchWork& Item : Work)
	{
		if (Item.MixerTrack && !SeenTracks.Contains(Item.MixerTrack))
		{
			SeenTracks.Add(Item.MixerTrack);
			Item.MixerTrack->RebuildDirtyAccumulatedOffsetCache(Linker, RootInstanceHandle);
		}
	}

	for (const FRematchWork& Item : Work)
	{
		AnimMixerBoneMatching::FBoneMatchingContext Context;
		Context.Linker = Linker;
		Context.InstanceHandle = RootInstanceHandle;
		Context.MixerTrack = Item.MixerTrack;
		Context.CurrentTime = SequencerPtr->GetLocalTime().Time;

		TSet<UMovieSceneSection*> Visited;
		Visited.Add(Item.Section);
		AnimMixerBoneMatching::PropagateRematch(Item.Section, Context, Visited);
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

void FRootMotionSettingsDecorationEditor::BuildSectionContextMenu(
	FMenuBuilder& MenuBuilder,
	UMovieSceneSection& Section,
	UObject& Decoration,
	const FGuid& ObjectBinding,
	TWeakPtr<ISequencer> Sequencer)
{
	UMovieSceneRootMotionSettingsDecoration* Settings = Cast<UMovieSceneRootMotionSettingsDecoration>(&Decoration);
	if (!Settings)
	{
		return;
	}

	MenuBuilder.AddSubMenu(
		LOCTEXT("RootTransform_Label", "Root Transform"),
		LOCTEXT("RootTransform_Tooltip", "Options for root transform behavior from this section"),
		FNewMenuDelegate::CreateRaw(this, &FRootMotionSettingsDecorationEditor::PopulateRootTransformMenu,
			TWeakObjectPtr<UMovieSceneSection>(&Section),
			TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>(Settings),
			ObjectBinding,
			Sequencer)
	);
}

void FRootMotionSettingsDecorationEditor::PopulateRootTransformMenu(
	FMenuBuilder& MenuBuilder,
	TWeakObjectPtr<UMovieSceneSection> WeakSection,
	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakSettings,
	FGuid ObjectBinding,
	TWeakPtr<ISequencer> Sequencer)
{
	UMovieSceneSection* Section = WeakSection.Get();
	UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get();
	if (!Section || !Settings)
	{
		return;
	}

	const bool bBoneMatchInvalid = Settings->HasBoneMatch() && !Settings->GetBoneMatchData().bIsValid;

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RootTransformModeSection_Label", "Root Transform Mode"));
	{
		FCanExecuteAction AlwaysExecute = FCanExecuteAction::CreateLambda([] { return true; });

		auto IsMode = [WeakSettings](EMovieSceneRootMotionTransformMode Mode)
		{
			if (UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get())
			{
				return Settings->GetRootTransformMode() == Mode;
			}
			return Mode == EMovieSceneRootMotionTransformMode::Asset;
		};

		auto SetMode = [WeakSettings, WeakSection, Sequencer](EMovieSceneRootMotionTransformMode Mode)
		{
			UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get();
			UMovieSceneSection* Section = WeakSection.Get();
			if (Settings && Section)
			{
				Settings->InvalidateDecorationChannelProxy();
				Settings->Modify();
				Settings->TransformMode.SetDefault((uint8)Mode);
				if (TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin())
				{
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			}
		};

		auto AddModeEntry = [&](FText Label, FText Tooltip, FText TxnLabel, EMovieSceneRootMotionTransformMode Mode)
		{
			MenuBuilder.AddMenuEntry(
				Label, Tooltip, FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SetMode, TxnLabel, Mode]{ FScopedTransaction Txn(TxnLabel); SetMode(Mode); }),
					AlwaysExecute,
					FIsActionChecked::CreateLambda([IsMode, Mode]{ return IsMode(Mode); })
				),
				NAME_None, EUserInterfaceActionType::RadioButton);
		};

		AddModeEntry(
			LOCTEXT("RootTransformMode_None", "From Animation"),
			LOCTEXT("RootTransformMode_None_Tip", "Use the root motion transform directly from the animation asset"),
			LOCTEXT("SetRootTransformNone", "Use Root Transform from Asset"),
			EMovieSceneRootMotionTransformMode::Asset);

		AddModeEntry(
			LOCTEXT("RootTransformMode_Offset", "Offset From Animation"),
			LOCTEXT("RootTransformMode_Offset_Tooltip", "Offset the root motion transform using keyframed values"),
			LOCTEXT("SetRootTransformOffset", "Offset Root Transform from Asset"),
			EMovieSceneRootMotionTransformMode::Offset);

		AddModeEntry(
			LOCTEXT("RootTransformMode_Override", "Manual Override"),
			LOCTEXT("RootTransformMode_Override_Tooltip", "Completely override the root motion transform using keyframed values"),
			LOCTEXT("SetRootTransformOverride", "Override Root Transform"),
			EMovieSceneRootMotionTransformMode::Override);

		AddModeEntry(
			LOCTEXT("RootTransformMode_AccumulatedOffset", "Accumulated Offset"),
			LOCTEXT("RootTransformMode_AccumulatedOffset_Tooltip", "Automatically chain from previous KeepState sections. The section starts where the previous section ended."),
			LOCTEXT("SetRootTransformAccumulatedOffset", "Use Accumulated Offset"),
			EMovieSceneRootMotionTransformMode::AccumulatedOffset);

		// Bone Match is display-only (set via the Match Bone dialog, not directly selectable)
		MenuBuilder.AddMenuEntry(
			bBoneMatchInvalid
				? LOCTEXT("RootTransformMode_BoneMatchInvalid", "Bone Match (invalid)")
				: LOCTEXT("RootTransformMode_BoneMatch", "Bone Match"),
			LOCTEXT("RootTransformMode_BoneMatch_Tooltip", "Position based on bone alignment. Set automatically by the Match Bone dialog."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]{}),
				FCanExecuteAction::CreateLambda([] { return false; }),
				FIsActionChecked::CreateLambda([IsMode]{ return IsMode(EMovieSceneRootMotionTransformMode::BoneMatch); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	// Re-center option (only in Offset mode)
	if (Settings->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::Offset)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("RootTransformOffset_Label", "Offset"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RecenterRootTransform", "Re-center Root Transform"),
				LOCTEXT("RecenterRootTransform_Tooltip", "Center the root transform for this animation around its current position"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FRootMotionSettingsDecorationEditor::RecenterRootTransform,
						WeakSection.Get(), WeakSettings.Get(), Sequencer)
				)
			);
		}
		MenuBuilder.EndSection();
	}

	const bool bHasReferenceSections = HasOverlappingReferenceSections(Section);

	// Bone Matching
	MenuBuilder.BeginSection(NAME_None,
		bBoneMatchInvalid
			? LOCTEXT("BoneMatchingInvalid_Label", "Bone Match (invalid)")
			: LOCTEXT("BoneMatching_Label", "Bone Matching"));
	{
		if (Settings->HasBoneMatch())
		{
			// Re-match: directly re-compute with existing settings (no dialog)
			auto GetMatchTimeModeLabel = [](EBoneMatchTimeMode Mode) -> FText
			{
				switch (Mode)
				{
				case EBoneMatchTimeMode::AtCurrentTime:            return LOCTEXT("TimeMode_Current", "Current Time");
				case EBoneMatchTimeMode::AtStartOfSelectedSection: return LOCTEXT("TimeMode_StartSel", "Start of This Section");
				case EBoneMatchTimeMode::AtEndOfSelectedSection:   return LOCTEXT("TimeMode_EndSel", "End of This Section");
				case EBoneMatchTimeMode::AtStartOfReferenceSection:return LOCTEXT("TimeMode_StartRef", "Start of Reference");
				case EBoneMatchTimeMode::AtEndOfReferenceSection:  return LOCTEXT("TimeMode_EndRef", "End of Reference");
				case EBoneMatchTimeMode::InBetween:                return LOCTEXT("TimeMode_Between", "Midpoint");
				default:                                           return LOCTEXT("TimeMode_Unknown", "Unknown");
				}
			};

			FMovieSceneBoneMatchData CurrentMatch = Settings->GetBoneMatchData();
			FText RematchLabel = FText::Format(LOCTEXT("RematchBone", "Re-match ({0})"),
				FText::FromName(CurrentMatch.BoneName));
			FText RematchTooltip = FText::Format(LOCTEXT("RematchBone_Tooltip", "Re-compute bone match: {0}, time: {1}"),
				FText::FromName(CurrentMatch.BoneName),
				GetMatchTimeModeLabel(CurrentMatch.MatchTimeMode));

			// Re-match and edit are only available when there's a valid
			// reference section to match against
			const bool bCanRematch = bHasReferenceSections;
			FText DisabledTooltip = LOCTEXT("BoneMatch_NoOverlap_Tooltip", "No overlapping root-motion-producing sections available for bone matching");

			MenuBuilder.AddMenuEntry(
				RematchLabel,
				bCanRematch ? RematchTooltip : DisabledTooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FRootMotionSettingsDecorationEditor::RematchBone,
						WeakSection.Get(), WeakSettings.Get(), Sequencer),
					FCanExecuteAction::CreateLambda([bCanRematch] { return bCanRematch; })
				)
			);

			// Edit: open the dialog to change match parameters
			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditBoneMatch", "Edit Bone Match..."),
				bCanRematch
					? LOCTEXT("EditBoneMatch_Tooltip", "Open dialog to change bone match settings")
					: DisabledTooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FRootMotionSettingsDecorationEditor::OpenBoneMatchDialog,
						WeakSection.Get(), WeakSettings.Get(), ObjectBinding, Sequencer),
					FCanExecuteAction::CreateLambda([bCanRematch] { return bCanRematch; })
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearBoneMatch", "Clear Bone Match"),
				LOCTEXT("ClearBoneMatch_Tooltip", "Remove the bone match transform offset and revert to previous transform mode"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FRootMotionSettingsDecorationEditor::ClearBoneMatch,
						WeakSection.Get(), WeakSettings.Get(), Sequencer)
				)
			);
		}
		else
		{
			// No existing match: open dialog
			MenuBuilder.AddMenuEntry(
				LOCTEXT("MatchBone", "Match Bone..."),
				bHasReferenceSections
					? LOCTEXT("MatchBone_Tooltip", "Open bone match dialog to align this section's root transform with a neighboring section")
					: LOCTEXT("MatchBone_NoRef_Tooltip", "No other sections available for bone matching"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FRootMotionSettingsDecorationEditor::OpenBoneMatchDialog,
						WeakSection.Get(), WeakSettings.Get(), ObjectBinding, Sequencer),
					FCanExecuteAction::CreateLambda([bHasReferenceSections] { return bHasReferenceSections; })
				)
			);
		}
	}
	MenuBuilder.EndSection();

	// Skeleton Visualization
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("SkeletonVisualization_Label", "Visualization"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowSkeleton", "Show Skeleton"),
			LOCTEXT("ShowSkeleton_Tooltip", "Draw this section's skeleton in the viewport"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakSettings]
				{
					if (UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get())
					{
						Settings->bShowSkeleton = !Settings->bShowSkeleton;
					}
				}),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda([WeakSettings]
				{
					if (const UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get())
					{
						return Settings->bShowSkeleton;
					}
					return false;
				})
			),
			NAME_None, EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

void FRootMotionSettingsDecorationEditor::RecenterRootTransform(
	UMovieSceneSection* Section,
	UMovieSceneRootMotionSettingsDecoration* Settings,
	TWeakPtr<ISequencer> InSequencer)
{
	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr || !Section || !Settings)
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	if (!MixerTrack || !Linker)
	{
		return;
	}

	// Evaluate this section in isolation at the current time,
	// skipping root motion space conversion to get the animation-space transform
	FInstanceHandle InstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();
	FQualifiedFrameTime Time = SequencerPtr->GetLocalTime();

	AnimMixerBakeEvaluation::FBakeFilter Filter;
	Filter.IncludeOnlySections.Add(FObjectKey(Section));
	Filter.bSkipRootMotionConversion = true;

	AnimMixerBakeEvaluation::FBakeResult Result =
		AnimMixerBakeEvaluation::EvaluateAtTime(
			Linker, InstanceHandle, MixerTrack,
			Time.Time, Filter);

	FScopedTransaction Transaction(LOCTEXT("RecenterRootTransform", "Re-center Root Transform"));
	Settings->Modify();
	Settings->RootOriginLocation = Result.RootMotionTransform.GetTranslation();
}

void FRootMotionSettingsDecorationEditor::OpenBoneMatchDialog(
	UMovieSceneSection* Section,
	UMovieSceneRootMotionSettingsDecoration* Settings,
	FGuid ObjectBinding,
	TWeakPtr<ISequencer> InSequencer)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!Section || !SequencerPtr || !Settings)
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return;
	}

	TSharedRef<SBoneMatchDialog> Dialog = SNew(SBoneMatchDialog)
		.Sequencer(SequencerPtr.Get())
		.TargetSection(Section)
		.MixerTrack(MixerTrack)
		.Decoration(Settings)
		.ObjectBinding(ObjectBinding)
		.OnAccepted(FOnBoneMatchAccepted::CreateStatic(
			&FRootMotionSettingsDecorationEditor::OnBoneMatchAccepted,
			Section, Settings, InSequencer));

	Dialog->OpenDialog(true);
}

void FRootMotionSettingsDecorationEditor::OnBoneMatchAccepted(
	const FMovieSceneBoneMatchData& MatchSettings,
	UMovieSceneSection* Section,
	UMovieSceneRootMotionSettingsDecoration* Settings,
	TWeakPtr<ISequencer> InSequencer)
{
	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!Section || !SequencerPtr || !Settings)
	{
		return;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Section->GetTypedOuter<UMovieSceneTrack>());
	if (!MixerTrack)
	{
		return;
	}

	AnimMixerBoneMatching::FBoneMatchingContext Context;
	Context.Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	Context.InstanceHandle = SequencerPtr->GetEvaluationTemplate().GetRootInstanceHandle();
	Context.MixerTrack = MixerTrack;
	Context.CurrentTime = SequencerPtr->GetLocalTime().Time;

	const EMovieSceneRootMotionTransformMode PreviousMode = Settings->GetRootTransformMode();

	FScopedTransaction Transaction(LOCTEXT("BoneMatch", "Bone Match"));

	// Invalidate before Modify() so the outliner rebuilds children for the new mode's channel set.
	Settings->InvalidateDecorationChannelProxy();
	Settings->Modify();

	FMovieSceneBoneMatchData Result = AnimMixerBoneMatching::ComputeBoneMatch(
		Section, MatchSettings, Context);

	// Store the result as a key on the channel at the resolved match time
	Settings->BoneMatchChannel.Reset();
	if (Result.bIsValid)
	{
		Settings->BoneMatchChannel.GetData().AddKey(Result.MatchTime, Result);
		Settings->TransformMode.SetDefault((uint8)EMovieSceneRootMotionTransformMode::BoneMatch);

		// Propagate rematches through dependency chain
		TSet<UMovieSceneSection*> Visited;
		Visited.Add(Section);
		AnimMixerBoneMatching::PropagateRematch(Section, Context, Visited);
	}
	Section->InvalidateChannelProxy();

	// On mode change, the outliner needs StructureItemsChanged to rebuild children.
	// Otherwise RefreshImmediately so the viewport returns to the playhead pose after the bake.
	const bool bModeChanged = Result.bIsValid
		&& PreviousMode != EMovieSceneRootMotionTransformMode::BoneMatch;
	SequencerPtr->NotifyMovieSceneDataChanged(bModeChanged
		? EMovieSceneDataChangeType::MovieSceneStructureItemsChanged
		: EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

void FRootMotionSettingsDecorationEditor::RematchBone(
	UMovieSceneSection* Section,
	UMovieSceneRootMotionSettingsDecoration* Settings,
	TWeakPtr<ISequencer> InSequencer)
{
	if (!Section || !Settings || !Settings->HasBoneMatch())
	{
		return;
	}

	OnBoneMatchAccepted(Settings->GetBoneMatchData(), Section, Settings, InSequencer);
}

void FRootMotionSettingsDecorationEditor::ClearBoneMatch(
	UMovieSceneSection* Section,
	UMovieSceneRootMotionSettingsDecoration* Settings,
	TWeakPtr<ISequencer> InSequencer)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr || !Section || !Settings)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ClearBoneMatch", "Clear Bone Match"));
	Settings->Modify();
	Settings->BoneMatchChannel.Reset();

	// Revert transform mode if it was set to BoneMatch
	if (Settings->GetRootTransformMode() == EMovieSceneRootMotionTransformMode::BoneMatch)
	{
		Settings->TransformMode.SetDefault((uint8)EMovieSceneRootMotionTransformMode::Asset);
	}
	Section->InvalidateChannelProxy();

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

TSharedPtr<SWidget> FRootMotionSettingsDecorationEditor::CreateKeyFrameWidget(
	UObject& Decoration,
	UMovieSceneSection& Section,
	FMovieSceneChannelHandle ChannelHandle,
	TWeakPtr<ISequencer> Sequencer,
	const FGuid& ObjectBindingID,
	TSharedPtr<FViewModel> ChannelGroupModel)
{
	UMovieSceneRootMotionSettingsDecoration* Settings = Cast<UMovieSceneRootMotionSettingsDecoration>(&Decoration);
	if (!Settings)
	{
		return nullptr;
	}

	// Only customize the BoneMatchChannel
	FMovieSceneChannel* RawChannel = ChannelHandle.Get();
	if (RawChannel != static_cast<FMovieSceneChannel*>(&Settings->BoneMatchChannel))
	{
		return nullptr;
	}

	TWeakObjectPtr<UMovieSceneSection> WeakSection = &Section;
	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakSettings = Settings;
	TWeakPtr<ISequencer> InSequencer = Sequencer;
	FGuid ObjectBinding = ObjectBindingID;

	auto CanMatchBone = [WeakSection]() -> bool
	{
		UMovieSceneSection* Section = WeakSection.Get();
		return Section && HasOverlappingReferenceSections(Section);
	};

	FText Tooltip = Settings->HasBoneMatch()
		? LOCTEXT("EditBoneMatch_KeyFrame_Tooltip", "Edit bone match settings")
		: LOCTEXT("MatchBone_KeyFrame_Tooltip", "Open bone match dialog");

	return SNew(SKeyNavigationButtons, ChannelGroupModel)
		.Buttons(EKeyNavigationButtons::AddKey)
		.AddKeyToolTip(Tooltip)
		.IsEnabled_Lambda(CanMatchBone)
		.OnAddKey(SKeyNavigationButtons::FOnAddKey::CreateLambda(
			[this, WeakSection, WeakSettings, ObjectBinding, InSequencer](FFrameTime, TSharedPtr<FViewModel>)
			{
				UMovieSceneSection* Section = WeakSection.Get();
				UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get();
				if (Section && Settings)
				{
					OpenBoneMatchDialog(Section, Settings, ObjectBinding, InSequencer);
				}
			}));
}

TSharedPtr<SWidget> FRootMotionSettingsDecorationEditor::CreateKeyEditor(
	UObject& Decoration,
	UMovieSceneSection& Section,
	FMovieSceneChannelHandle ChannelHandle,
	TWeakPtr<ISequencer> Sequencer,
	const FGuid& ObjectBindingID)
{
	UMovieSceneRootMotionSettingsDecoration* Settings = Cast<UMovieSceneRootMotionSettingsDecoration>(&Decoration);
	if (!Settings)
	{
		return nullptr;
	}

	// Only customize the TransformMode channel
	FMovieSceneChannel* RawChannel = ChannelHandle.Get();
	if (RawChannel != static_cast<FMovieSceneChannel*>(&Settings->TransformMode))
	{
		return nullptr;
	}

	UEnum* Enum = Settings->TransformMode.GetEnum();
	if (!Enum)
	{
		return nullptr;
	}

	TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakSettings = Settings;
	TWeakObjectPtr<UMovieSceneSection> WeakSection = &Section;
	TWeakPtr<ISequencer> InSequencer = Sequencer;

	return MovieSceneToolHelpers::MakeEnumComboBox(
		Enum,
		TAttribute<int32>::CreateLambda([WeakSettings]() -> int32
		{
			if (UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get())
			{
				return (int32)Settings->TransformMode.GetDefault().Get(0);
			}
			return 0;
		}),
		SEnumComboBox::FOnEnumSelectionChanged::CreateLambda(
			[WeakSettings, WeakSection, InSequencer, ObjectBindingID](int32 Selection, ESelectInfo::Type)
			{
				UMovieSceneRootMotionSettingsDecoration* Settings = WeakSettings.Get();
				UMovieSceneSection* Section = WeakSection.Get();
				if (!Settings || !Section)
				{
					return;
				}

				// Bone Match requires an actual match key on the channel to be valid.
				// If the user picks it without one, open the dialog instead of setting the mode;
				// the dialog's accept handler switches the mode on success.
				if (Selection == (int32)EMovieSceneRootMotionTransformMode::BoneMatch
					&& !Settings->HasBoneMatch())
				{
					FRootMotionSettingsDecorationEditor::OpenBoneMatchDialog(
						Section, Settings, ObjectBindingID, InSequencer);
					return;
				}

				FScopedTransaction Transaction(LOCTEXT("SetTransformMode", "Set Transform Mode"));
				// Invalidate channel proxy before Modify() so listeners see the dirty flag.
				Settings->InvalidateDecorationChannelProxy();
				Settings->Modify();
				Settings->TransformMode.SetDefault((uint8)Selection);
				if (TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin())
				{
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			})
	);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
