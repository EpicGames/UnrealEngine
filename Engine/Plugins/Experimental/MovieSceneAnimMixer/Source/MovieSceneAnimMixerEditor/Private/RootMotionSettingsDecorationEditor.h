// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerDecorationEditor.h"
#include "ISequencerSection.h"
#include "Channels/MovieSceneChannel.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/WeakObjectPtr.h"

class UMovieSceneRootMotionSettingsDecoration;
struct FMovieSceneBoneMatchData;

namespace UE::Sequencer
{

// Decoration editor for UMovieSceneRootMotionSettingsDecoration.
// Provides the "Root Transform" right-click context menu with transform mode
// selection, re-center, and bone matching.
class FRootMotionSettingsDecorationEditor : public ISequencerDecorationEditor
{
public:
	explicit FRootMotionSettingsDecorationEditor(TSharedRef<ISequencer> InSequencer)
		: WeakSequencer(InSequencer)
	{}

	virtual ~FRootMotionSettingsDecorationEditor() override;

	virtual UClass* GetDecorationClass() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FSlateIcon GetMenuIcon() const override;

	virtual void OnSectionInterfaceCreated(
		UMovieSceneSection& Section,
		UObject& Decoration,
		TWeakPtr<ISequencer> Sequencer) override;

	virtual void Tick(float DeltaTime) override;

	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder,
		UMovieSceneSection& Section,
		UObject& Decoration,
		const FGuid& ObjectBinding,
		TWeakPtr<ISequencer> Sequencer) override;

	virtual TSharedPtr<SWidget> CreateKeyEditor(
		UObject& Decoration,
		UMovieSceneSection& Section,
		FMovieSceneChannelHandle ChannelHandle,
		TWeakPtr<ISequencer> Sequencer,
		const FGuid& ObjectBindingID) override;

	virtual TSharedPtr<SWidget> CreateKeyFrameWidget(
		UObject& Decoration,
		UMovieSceneSection& Section,
		FMovieSceneChannelHandle ChannelHandle,
		TWeakPtr<ISequencer> Sequencer,
		const FGuid& ObjectBindingID,
		TSharedPtr<UE::Sequencer::FViewModel> ChannelGroupModel) override;

private:
	void PopulateRootTransformMenu(
		FMenuBuilder& MenuBuilder,
		TWeakObjectPtr<UMovieSceneSection> Section,
		TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> Settings,
		FGuid ObjectBinding,
		TWeakPtr<ISequencer> Sequencer);

	void RecenterRootTransform(
		UMovieSceneSection* Section,
		UMovieSceneRootMotionSettingsDecoration* Settings,
		TWeakPtr<ISequencer> Sequencer);

	static void OpenBoneMatchDialog(
		UMovieSceneSection* Section,
		UMovieSceneRootMotionSettingsDecoration* Settings,
		FGuid ObjectBinding,
		TWeakPtr<ISequencer> Sequencer);

	static void OnBoneMatchAccepted(
		const FMovieSceneBoneMatchData& MatchSettings,
		UMovieSceneSection* Section,
		UMovieSceneRootMotionSettingsDecoration* Settings,
		TWeakPtr<ISequencer> Sequencer);

	void RematchBone(
		UMovieSceneSection* Section,
		UMovieSceneRootMotionSettingsDecoration* Settings,
		TWeakPtr<ISequencer> Sequencer);

	void ClearBoneMatch(
		UMovieSceneSection* Section,
		UMovieSceneRootMotionSettingsDecoration* Settings,
		TWeakPtr<ISequencer> Sequencer);

	void HandleBoneMatchKeyMoved(FMovieSceneChannel* Channel, const TArray<FKeyMoveEventItem>& Items);
	void HandleBoneMatchKeyDeleted(FMovieSceneChannel* Channel, const TArray<FKeyAddOrDeleteEventItem>& Items);
	void HandleDecorationSignatureChanged(TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration> WeakDecoration);

	struct FTrackedSectionInfo
	{
		TWeakObjectPtr<UMovieSceneSection> Section;
		FDelegateHandle DecorationSigHandle;
		FDelegateHandle SectionSigHandle;
		FGuid LastDecorationSignature;
		FGuid LastSectionSignature;
	};

	FTrackedSectionInfo* FindTrackedInfoForChannel(FMovieSceneChannel* Channel, UMovieSceneRootMotionSettingsDecoration*& OutDecoration);

	TMap<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>, FTrackedSectionInfo> TrackedDecorations;
	TSet<TWeakObjectPtr<UMovieSceneRootMotionSettingsDecoration>> PendingRematches;
	TWeakPtr<ISequencer> WeakSequencer;

	// Guards Tick from re-entering when PropagateRematch's Modify() fires signature events on dependents.
	bool bHandlingSignatureChange = false;
};

} // namespace UE::Sequencer
