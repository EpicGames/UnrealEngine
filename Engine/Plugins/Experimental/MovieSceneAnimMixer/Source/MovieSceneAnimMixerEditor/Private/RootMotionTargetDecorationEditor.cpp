// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionTargetDecorationEditor.h"

#include "MovieSceneAnimMixerEditorStyle.h"
#include "MovieSceneRootMotionSection.h"
#include "MovieSceneRootMotionTargetDecoration.h"
#include "SequencerSectionPainter.h"

#define LOCTEXT_NAMESPACE "RootMotionTargetDecorationEditor"

namespace UE::Sequencer
{

//------------------------------------------------------------------------------
// FRootMotionTargetSectionInterface
//------------------------------------------------------------------------------

FRootMotionTargetSectionInterface::FRootMotionTargetSectionInterface(
	UMovieSceneSection& InSection,
	UMovieSceneTrack& /*InTrack*/,
	FGuid /*InObjectBinding*/)
	: SectionPtr(&InSection)
{
}

UMovieSceneSection* FRootMotionTargetSectionInterface::GetSectionObject()
{
	return SectionPtr.Get();
}

FText FRootMotionTargetSectionInterface::GetSectionTitle() const
{
	return FText::GetEmpty();
}

int32 FRootMotionTargetSectionInterface::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	// Use the mixer item tint for the section background
	if (const UMovieSceneRootMotionSection* RootMotionSection = Cast<UMovieSceneRootMotionSection>(SectionPtr.Get()))
	{
		FLinearColor Tint = FLinearColor(RootMotionSection->GetMixerItemTint());
		return Painter.PaintSectionBackground(Tint);
	}
	return Painter.PaintSectionBackground();
}

//------------------------------------------------------------------------------
// FRootMotionTargetDecorationEditor
//------------------------------------------------------------------------------

UClass* FRootMotionTargetDecorationEditor::GetDecorationClass() const
{
	return UMovieSceneRootMotionTargetDecoration::StaticClass();
}

const FSlateBrush* FRootMotionTargetDecorationEditor::GetIconBrush() const
{
	return FMovieSceneAnimMixerEditorStyle::Get().GetBrush("Tracks.RootMotionTarget");
}

FSlateIcon FRootMotionTargetDecorationEditor::GetMenuIcon() const
{
	return FSlateIcon(FMovieSceneAnimMixerEditorStyle::Get().GetStyleSetName(), "Tracks.RootMotionTarget");
}

TSharedPtr<ISequencerSection> FRootMotionTargetDecorationEditor::MakeSectionInterface(
	UMovieSceneSection& Section,
	UMovieSceneTrack& OwningTrack,
	FGuid ObjectBinding)
{
	return MakeShared<FRootMotionTargetSectionInterface>(Section, OwningTrack, ObjectBinding);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
