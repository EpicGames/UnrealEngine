// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerDecorationEditor.h"
#include "ISequencerSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class UMovieSceneSection;
class UMovieSceneTrack;

namespace UE::Sequencer
{

/**
 * Section interface for root motion sections displayed under decoration outliners.
 * Provides basic display with the channel data for root motion destination.
 */
class FRootMotionTargetSectionInterface
	: public ISequencerSection
	, public TSharedFromThis<FRootMotionTargetSectionInterface>
{
public:
	FRootMotionTargetSectionInterface(UMovieSceneSection& InSection, UMovieSceneTrack& InTrack, FGuid InObjectBinding);

	//~ ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual bool IsReadOnly() const override { return false; }

private:
	TWeakObjectPtr<UMovieSceneSection> SectionPtr;
};

/**
 * Decoration editor for UMovieSceneRootMotionTargetDecoration.
 * Creates section interfaces for the root motion sections that the decoration provides.
 */
class FRootMotionTargetDecorationEditor : public ISequencerDecorationEditor
{
public:
	virtual UClass* GetDecorationClass() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FSlateIcon GetMenuIcon() const override;
	virtual TSharedPtr<ISequencerSection> MakeSectionInterface(
		UMovieSceneSection& Section,
		UMovieSceneTrack& OwningTrack,
		FGuid ObjectBinding) override;
};

} // namespace UE::Sequencer
