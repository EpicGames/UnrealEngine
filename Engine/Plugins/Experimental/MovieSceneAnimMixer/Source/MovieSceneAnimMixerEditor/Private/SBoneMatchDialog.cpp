// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBoneMatchDialog.h"

#include "BoneSelectionWidget.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneSection.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBoneMatchDialog"

static USkeletalMeshComponent* FindBoundSkeletalMeshComponent(ISequencer* Sequencer, const FGuid& ObjectBinding)
{
	if (!Sequencer || !ObjectBinding.IsValid())
	{
		return nullptr;
	}

	TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectBinding, Sequencer->GetFocusedTemplateID());
	for (const TWeakObjectPtr<>& WeakObj : BoundObjects)
	{
		UObject* Obj = WeakObj.Get();
		if (!Obj)
		{
			continue;
		}

		USkeletalMeshComponent* SkelComp = nullptr;
		if (AActor* Actor = Cast<AActor>(Obj))
		{
			SkelComp = Actor->FindComponentByClass<USkeletalMeshComponent>();
		}
		else
		{
			SkelComp = Cast<USkeletalMeshComponent>(Obj);
		}

		if (SkelComp && SkelComp->GetSkeletalMeshAsset())
		{
			return SkelComp;
		}
	}
	return nullptr;
}

static FString GetSectionDisplayName(const UMovieSceneSection* Section, ISequencer* Sequencer)
{
	using namespace UE::Sequencer;

	if (Sequencer)
	{
		if (TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
		{
			if (FSectionModelStorageExtension* Storage = ViewModel->GetRootModel()->CastDynamic<FSectionModelStorageExtension>())
			{
				if (TSharedPtr<FSectionModel> Model = Storage->FindModelForSection(Section))
				{
					if (TSharedPtr<ISequencerSection> SectionInterface = Model->GetSectionInterface())
					{
						FText Title = SectionInterface->GetSectionTitle();
						if (!Title.IsEmpty())
						{
							return Title.ToString();
						}
					}
				}
			}
		}
	}
	return Section->GetName();
}

void SBoneMatchDialog::Construct(const FArguments& InArgs)
{
	Sequencer = InArgs._Sequencer;
	TargetSection = InArgs._TargetSection;
	MixerTrack = InArgs._MixerTrack;
	Decoration = InArgs._Decoration;
	ObjectBinding = InArgs._ObjectBinding;
	OnAccepted = InArgs._OnAccepted;

	check(Sequencer && TargetSection && MixerTrack && Decoration);

	// Initialize from existing settings
	const FMovieSceneBoneMatchData Existing = Decoration->GetBoneMatchData();
	BoneName = Existing.BoneName;
	if (BoneName.IsNone())
	{
		// Default to the root bone (index 0)
		if (USkeletalMeshComponent* SkelComp = FindBoundSkeletalMeshComponent(Sequencer, ObjectBinding))
		{
			const FReferenceSkeleton& RefSkel = SkelComp->GetSkeletalMeshAsset()->GetRefSkeleton();
			if (RefSkel.GetNum() > 0)
			{
				BoneName = RefSkel.GetBoneName(0);
			}
		}
	}
	ReferenceSection = Existing.ReferenceSection;
	MatchTimeMode = (uint8)Existing.MatchTimeMode;
	bMatchLocationX = Existing.bMatchLocationX;
	bMatchLocationY = Existing.bMatchLocationY;
	bMatchLocationZ = Existing.bMatchLocationZ;
	bMatchRotationX = Existing.bMatchRotationX;
	bMatchRotationY = Existing.bMatchRotationY;
	bMatchRotationZ = Existing.bMatchRotationZ;

	BuildReferenceSectionList();
	RebuildMatchTimeModeOptions();

	auto MakeCheckbox = [](const FText& Label, bool& BoolRef) -> TSharedRef<SHorizontalBox>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([&BoolRef]() { return BoolRef ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([&BoolRef](ECheckBoxState State) { BoolRef = (State == ECheckBoxState::Checked); })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 12.f, 0.f)
			[
				SNew(STextBlock).Text(Label)
			];
	};

	FString TargetSectionName = GetSectionDisplayName(TargetSection, Sequencer);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(16.f)
		[
			SNew(SVerticalBox)

			// Header showing which section we're matching
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 12.f)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("ThisSectionHeader", "This Section: {0}"), FText::FromString(TargetSectionName)))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			]

			// Bone picker
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(STextBlock).Text(LOCTEXT("BoneLabel", "Bone"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SBoneSelectionWidget)
					.OnGetSelectedBone_Lambda([this](bool& bMultipleValues) -> FName
					{
						bMultipleValues = false;
						return BoneName;
					})
					.OnBoneSelectionChanged(FOnBoneSelectionChanged::CreateSP(this, &SBoneMatchDialog::OnBoneSelectionChanged))
					.OnGetReferenceSkeleton_Lambda([this]() -> const FReferenceSkeleton&
					{
						static FReferenceSkeleton EmptySkeleton;
						if (USkeletalMeshComponent* SkelComp = FindBoundSkeletalMeshComponent(Sequencer, ObjectBinding))
						{
							return SkelComp->GetSkeletalMeshAsset()->GetRefSkeleton();
						}
						return EmptySkeleton;
					})
				]
			]

			// Reference section
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(STextBlock).Text(LOCTEXT("ReferenceSectionLabel", "Reference Section"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&ReferenceSectionNames)
					.InitiallySelectedItem(ReferenceSectionNames.IsValidIndex(SelectedReferenceSectionIndex) ? ReferenceSectionNames[SelectedReferenceSectionIndex] : TSharedPtr<FString>())
					.OnSelectionChanged(this, &SBoneMatchDialog::OnReferenceSectionChanged)
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
					{
						return SNew(STextBlock).Text(FText::FromString(*Item));
					})
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							if (SelectedReferenceSectionIndex >= 0 && SelectedReferenceSectionIndex < ReferenceSectionNames.Num())
							{
								return FText::FromString(*ReferenceSectionNames[SelectedReferenceSectionIndex]);
							}
							return LOCTEXT("NoRefSection", "(None)");
						})
					]
				]
			]

			// Match time mode
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(STextBlock).Text(LOCTEXT("MatchTimeLabel", "Match Time"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&MatchTimeModeNames)
					.InitiallySelectedItem(MatchTimeModeNames.IsValidIndex(SelectedMatchTimeModeIndex) ? MatchTimeModeNames[SelectedMatchTimeModeIndex] : TSharedPtr<FString>())
					.OnSelectionChanged(this, &SBoneMatchDialog::OnMatchTimeModeChanged)
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
					{
						return SNew(STextBlock).Text(FText::FromString(*Item));
					})
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							if (SelectedMatchTimeModeIndex >= 0 && SelectedMatchTimeModeIndex < MatchTimeModeNames.Num())
							{
								return FText::FromString(*MatchTimeModeNames[SelectedMatchTimeModeIndex]);
							}
							return LOCTEXT("NoMatchTime", "Start of Section");
						})
					]
				]
			]

			// Location matching
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 4.f, 0.f, 4.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(STextBlock).Text(LOCTEXT("LocationLabel", "Match Location"))
				]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("X", "X"), bMatchLocationX) ]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("Y", "Y"), bMatchLocationY) ]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("Z", "Z"), bMatchLocationZ) ]
			]

			// Rotation matching
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(STextBlock).Text(LOCTEXT("RotationLabel", "Match Rotation"))
				]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("Roll", "Roll"), bMatchRotationX) ]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("Pitch", "Pitch"), bMatchRotationY) ]
				+ SHorizontalBox::Slot().AutoWidth() [ MakeCheckbox(LOCTEXT("Yaw", "Yaw"), bMatchRotationZ) ]
			]

			// OK / Cancel
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 8.f, 0.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().FillWidth(1.f) [ SNew(SSpacer) ]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 0.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SBoneMatchDialog::OnOKClicked)
					.IsEnabled_Lambda([this]() { return !BoneName.IsNone(); })
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked_Lambda([this]()
					{
						CloseDialog();
						return FReply::Handled();
					})
				]
			]
		]
	];
}

void SBoneMatchDialog::BuildReferenceSectionList()
{
	ReferenceSections.Reset();
	ReferenceSectionNames.Reset();
	SelectedReferenceSectionIndex = INDEX_NONE;

	if (!TargetSection || !MixerTrack)
	{
		return;
	}

	for (UMovieSceneSection* Section : MixerTrack->GetAllSections())
	{
		if (!Section || Section == TargetSection)
		{
			continue;
		}

		if (Section->IsA<UMovieSceneAnimTransitionSectionBase>())
		{
			continue;
		}

		// Only include sections that overlap in time
		if (TRange<FFrameNumber>::Intersection(TargetSection->GetRange(), Section->GetRange()).IsEmpty())
		{
			continue;
		}

		// Exclude sections whose bone match chain references the target,
		// since that would create a dependency cycle
		if (WouldCreateBoneMatchCycle(Section, TargetSection))
		{
			continue;
		}

		ReferenceSections.Add(Section);

		FString Name = GetSectionDisplayName(Section, Sequencer);
		if (Section->HasStartFrame() && Section->HasEndFrame() && Sequencer)
		{
			FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
			FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
			FFrameNumber StartDisplay = FFrameRate::TransformTime(FFrameTime(Section->GetInclusiveStartFrame()), TickResolution, DisplayRate).FloorToFrame();
			FFrameNumber EndDisplay = FFrameRate::TransformTime(FFrameTime(Section->GetExclusiveEndFrame()), TickResolution, DisplayRate).FloorToFrame();
			Name += FString::Printf(TEXT(" [%d-%d]"), StartDisplay.Value, EndDisplay.Value);
		}
		ReferenceSectionNames.Add(MakeShared<FString>(Name));

		// Select the one that matches current settings
		if (ReferenceSection.IsValid() && ReferenceSection.Get() == Section)
		{
			SelectedReferenceSectionIndex = ReferenceSections.Num() - 1;
		}
	}

	// Default to the first reference section if none was previously selected
	if (SelectedReferenceSectionIndex == INDEX_NONE && ReferenceSections.Num() > 0)
	{
		SelectedReferenceSectionIndex = 0;
		ReferenceSection = ReferenceSections[0];
	}
}

void SBoneMatchDialog::RebuildMatchTimeModeOptions()
{
	FilteredMatchTimeModes.Reset();
	MatchTimeModeNames.Reset();
	SelectedMatchTimeModeIndex = 0;

	UMovieSceneSection* RefSection = nullptr;
	FString RefName = TEXT("Reference");
	if (SelectedReferenceSectionIndex >= 0 && SelectedReferenceSectionIndex < ReferenceSections.Num())
	{
		RefSection = ReferenceSections[SelectedReferenceSectionIndex].Get();
		if (RefSection)
		{
			RefName = GetSectionDisplayName(RefSection, Sequencer);
		}
	}

	// Compute the overlap range between target and reference sections.
	// Only match times that fall within the overlap are valid.
	TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::All();
	if (TargetSection && RefSection)
	{
		TRange<FFrameNumber> TargetRange = TargetSection->GetRange();
		TRange<FFrameNumber> RefRange = RefSection->GetRange();
		Overlap = TRange<FFrameNumber>::Intersection(TargetRange, RefRange);
	}

	auto AddMode = [&](EBoneMatchTimeMode Mode, const FString& Label)
	{
		FilteredMatchTimeModes.Add(Mode);
		MatchTimeModeNames.Add(MakeShared<FString>(Label));

		if ((uint8)Mode == MatchTimeMode)
		{
			SelectedMatchTimeModeIndex = FilteredMatchTimeModes.Num() - 1;
		}
	};

	// For each match time, check that it falls within the other section's range
	// so both sections are active at the match point.
	TRange<FFrameNumber> TargetRange = TargetSection ? TargetSection->GetRange() : TRange<FFrameNumber>::Empty();
	TRange<FFrameNumber> RefRange = RefSection ? RefSection->GetRange() : TRange<FFrameNumber>::Empty();

	if (Sequencer)
	{
		FFrameNumber CurrentFrame = Sequencer->GetLocalTime().Time.FloorToFrame();
		if (TargetRange.Contains(CurrentFrame) && RefRange.Contains(CurrentFrame))
		{
			AddMode(EBoneMatchTimeMode::AtCurrentTime, TEXT("At Current Time"));
		}
	}

	if (TargetSection && TargetSection->HasStartFrame() && RefRange.Contains(TargetSection->GetInclusiveStartFrame()))
	{
		AddMode(EBoneMatchTimeMode::AtStartOfSelectedSection, TEXT("At Start of This Section"));
	}
	if (TargetSection && TargetSection->HasEndFrame() && RefRange.Contains(TargetSection->GetExclusiveEndFrame() - 1))
	{
		AddMode(EBoneMatchTimeMode::AtEndOfSelectedSection, TEXT("At End of This Section"));
	}

	if (RefSection && RefSection->HasStartFrame() && TargetRange.Contains(RefSection->GetInclusiveStartFrame()))
	{
		AddMode(EBoneMatchTimeMode::AtStartOfReferenceSection,
			FString::Printf(TEXT("At Start of %s"), *RefName));
	}
	if (RefSection && RefSection->HasEndFrame() && TargetRange.Contains(RefSection->GetExclusiveEndFrame() - 1))
	{
		AddMode(EBoneMatchTimeMode::AtEndOfReferenceSection,
			FString::Printf(TEXT("At End of %s"), *RefName));
	}

	if (!Overlap.IsEmpty())
	{
		AddMode(EBoneMatchTimeMode::InBetween, TEXT("In Between (Midpoint)"));
	}

	// Sync the backing value with the displayed selection. When the previously
	// stored mode isn't in the filtered list, the UI defaults to index 0 but
	// MatchTimeMode still holds the old (now-invalid) enum value.
	if (SelectedMatchTimeModeIndex >= 0 && SelectedMatchTimeModeIndex < FilteredMatchTimeModes.Num())
	{
		MatchTimeMode = (uint8)FilteredMatchTimeModes[SelectedMatchTimeModeIndex];
	}
}

void SBoneMatchDialog::OnBoneSelectionChanged(FName NewBoneName)
{
	BoneName = NewBoneName;
}

void SBoneMatchDialog::OnReferenceSectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	int32 Index = ReferenceSectionNames.IndexOfByKey(NewValue);
	if (Index != INDEX_NONE)
	{
		SelectedReferenceSectionIndex = Index;
		ReferenceSection = ReferenceSections[Index];
		RebuildMatchTimeModeOptions();
	}
}

void SBoneMatchDialog::OnMatchTimeModeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	int32 Index = MatchTimeModeNames.IndexOfByKey(NewValue);
	if (Index != INDEX_NONE && Index < FilteredMatchTimeModes.Num())
	{
		SelectedMatchTimeModeIndex = Index;
		MatchTimeMode = (uint8)FilteredMatchTimeModes[Index];
	}
}

FReply SBoneMatchDialog::OnOKClicked()
{
	FMovieSceneBoneMatchData MatchData;
	MatchData.BoneName = BoneName;
	MatchData.ReferenceSection = ReferenceSection;
	MatchData.MatchTimeMode = (EBoneMatchTimeMode)MatchTimeMode;
	MatchData.bMatchLocationX = bMatchLocationX;
	MatchData.bMatchLocationY = bMatchLocationY;
	MatchData.bMatchLocationZ = bMatchLocationZ;
	MatchData.bMatchRotationX = bMatchRotationX;
	MatchData.bMatchRotationY = bMatchRotationY;
	MatchData.bMatchRotationZ = bMatchRotationZ;

	OnAccepted.ExecuteIfBound(MatchData);
	CloseDialog();
	return FReply::Handled();
}

FReply SBoneMatchDialog::OpenDialog(bool bModal)
{
	check(!DialogWindow.IsValid());

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("BoneMatchDialogTitle", "Bone Match Settings"))
		.CreateTitleBar(true)
		.Type(EWindowType::Normal)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			AsShared()
		];

	Window->SetWidgetToFocusOnActivate(AsShared());
	DialogWindow = Window;

	if (bModal)
	{
		GEditor->EditorAddModalWindow(Window);
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	return FReply::Handled();
}

void SBoneMatchDialog::CloseDialog()
{
	if (DialogWindow.IsValid())
	{
		DialogWindow.Pin()->RequestDestroyWindow();
		DialogWindow.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
