// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ISequencer;
class UMovieSceneSection;
class UMovieSceneAnimationMixerTrack;
class UMovieSceneRootMotionSettingsDecoration;
class SWindow;
struct FMovieSceneBoneMatchData;

DECLARE_DELEGATE_OneParam(FOnBoneMatchAccepted, const FMovieSceneBoneMatchData&);

// Modal dialog for configuring bone matching parameters.
class SBoneMatchDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SBoneMatchDialog)
		: _Sequencer(nullptr)
		, _TargetSection(nullptr)
		, _MixerTrack(nullptr)
		, _Decoration(nullptr)
		{}
		SLATE_ARGUMENT(ISequencer*, Sequencer)
		SLATE_ARGUMENT(UMovieSceneSection*, TargetSection)
		SLATE_ARGUMENT(UMovieSceneAnimationMixerTrack*, MixerTrack)
		SLATE_ARGUMENT(UMovieSceneRootMotionSettingsDecoration*, Decoration)
		SLATE_ARGUMENT(FGuid, ObjectBinding)
		SLATE_EVENT(FOnBoneMatchAccepted, OnAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FReply OpenDialog(bool bModal = true);
	void CloseDialog();

private:

	// Build the list of candidate reference sections
	void BuildReferenceSectionList();

	// Build the match time mode option list
	void RebuildMatchTimeModeOptions();

	// Callbacks
	void OnBoneSelectionChanged(FName NewBoneName);
	void OnReferenceSectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	void OnMatchTimeModeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	FReply OnOKClicked();

	ISequencer* Sequencer = nullptr;
	UMovieSceneSection* TargetSection = nullptr;
	UMovieSceneAnimationMixerTrack* MixerTrack = nullptr;
	UMovieSceneRootMotionSettingsDecoration* Decoration = nullptr;
	FGuid ObjectBinding;
	FOnBoneMatchAccepted OnAccepted;

	TWeakPtr<SWindow> DialogWindow;

	FName BoneName;
	TWeakObjectPtr<UMovieSceneSection> ReferenceSection;
	uint8 MatchTimeMode = 0;

	bool bMatchLocationX = true;
	bool bMatchLocationY = true;
	bool bMatchLocationZ = false;
	bool bMatchRotationX = false;
	bool bMatchRotationY = false;
	bool bMatchRotationZ = true;

	// Reference section options
	TArray<TWeakObjectPtr<UMovieSceneSection>> ReferenceSections;
	TArray<TSharedPtr<FString>> ReferenceSectionNames;
	int32 SelectedReferenceSectionIndex = INDEX_NONE;

	// Match time mode options (filtered to only include valid modes for current overlap)
	TArray<EBoneMatchTimeMode> FilteredMatchTimeModes;
	TArray<TSharedPtr<FString>> MatchTimeModeNames;
	int32 SelectedMatchTimeModeIndex = 0;
};
