// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimSubsystem_SequencerMixer.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_SequencerMixerTarget.generated.h"

/**
 * Extension for brokering Animation Mixer data between sequencer and animation blueprints.
 */
UCLASS()
class MOVIESCENEANIMMIXEREDITOR_API UAnimBlueprintExtension_SequencerMixerTarget : public UAnimBlueprintExtension
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FAnimSubsystem_SequencerMixer Subsystem;
};
