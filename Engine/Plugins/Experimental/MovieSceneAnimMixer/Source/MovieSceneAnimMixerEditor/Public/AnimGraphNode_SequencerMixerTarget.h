// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "AnimNode_SequencerMixerTarget.h"

#include "AnimGraphNode_SequencerMixerTarget.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_SequencerMixerTarget : public UAnimGraphNode_Base
{
public:

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_SequencerMixerTarget Node;

	//~ Begin UEdGraphNode Interface.
	MOVIESCENEANIMMIXEREDITOR_API virtual FLinearColor GetNodeTitleColor() const override;
	MOVIESCENEANIMMIXEREDITOR_API virtual FText GetTooltipText() const override;
	MOVIESCENEANIMMIXEREDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	MOVIESCENEANIMMIXEREDITOR_API virtual FString GetNodeCategory() const override;
	MOVIESCENEANIMMIXEREDITOR_API virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	MOVIESCENEANIMMIXEREDITOR_API virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;
	//~ End UAnimGraphNode_Base Interface
	
};
