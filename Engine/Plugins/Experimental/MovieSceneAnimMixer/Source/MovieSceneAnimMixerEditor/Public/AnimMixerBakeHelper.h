// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimMixerBakeEvaluation.h"
#include "Framework/Commands/UIAction.h"
#include "Internationalization/Text.h"
#include "Templates/SubclassOf.h"

class FMenuBuilder;
class ISequencer;
class UAnimSequence;
class UAnimSeqExportOption;
class UMovieSceneAnimationMixerTrack;
class USkeletalMeshComponent;
class USkeleton;
class UMovieScene;
class UMovieSceneTrack;

namespace UE::Sequencer::AnimMixerBake
{

/**
 * Fills a UAnimSequence from mixer EvaluateRange results.
 * Filter defaults to empty (evaluates entire track); populate IncludeOnlySections for per-layer bake.
 * Respects all UAnimSeqExportOption settings (transforms, curves, interpolation, name filters, etc.).
 */
MOVIESCENEANIMMIXEREDITOR_API bool ExportMixerToAnimSequence(
	const TSharedPtr<ISequencer>& Sequencer,
	UAnimSequence* AnimSequence,
	UAnimSeqExportOption* ExportOptions,
	USkeletalMeshComponent* SkelMeshComp,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	const UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter& Filter = {});

/**
 * Builds the full "Bake" menu section into MenuBuilder:
 *   - "{EntryLabelVerb} to Anim Sequence" entry (with asset creation dialog + notification)
 *   - "{EntryLabelVerb} to Control Rig" submenu (via IMovieSceneAnimMixerBakeProvider, if registered)
 *
 * Works for both whole-track and per-layer contexts. Callers provide:
 *   - Filter: empty for whole track, populated with priority range for layer(s)
 *   - EntryLabelVerb: prefix used in leaf entry labels (e.g. "Bake", "Bake Layer", "Bake Selected Layers")
 *   - ChildTrackFactory: creates a child track in the mixer (the provider passes the track class)
 *   - OnComplete: post-bake actions (muting source layers/tracks, etc.)
 *   - CanExecuteOverride (optional): controls enable/disable state of the leaf entries.
 *     Defaults to always-enabled.
 *   - DisabledTooltipOverride (optional): tooltip shown when CanExecuteOverride returns false.
 *     Empty falls back to the normal enabled tooltip.
 */
MOVIESCENEANIMMIXEREDITOR_API void BuildBakeMenuSection(
	FMenuBuilder& MenuBuilder,
	const TSharedPtr<ISequencer>& Sequencer,
	UMovieSceneAnimationMixerTrack* MixerTrack,
	USkeletalMeshComponent* SkelMeshComp,
	USkeleton* Skeleton,
	FGuid ObjectBinding,
	UObject* BoundObject,
	bool& bFilterAssetBySkeleton,
	const UE::MovieScene::AnimMixerBakeEvaluation::FBakeFilter& Filter,
	TFunction<UMovieSceneTrack*(UMovieScene*, TSubclassOf<UMovieSceneTrack>)> ChildTrackFactory,
	TFunction<void(UMovieSceneTrack*, UObject*, bool)> OnComplete,
	const FText& EntryLabelVerb = FText::GetEmpty(),
	FCanExecuteAction CanExecuteOverride = FCanExecuteAction(),
	const FText& DisabledTooltipOverride = FText::GetEmpty());

} // namespace UE::Sequencer::AnimMixerBake
