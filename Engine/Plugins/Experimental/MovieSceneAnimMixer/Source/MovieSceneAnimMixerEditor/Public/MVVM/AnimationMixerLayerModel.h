// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"

#include "UObject/WeakObjectPtr.h"

class UMovieSceneTrack;
class UMovieSceneAnimationMixerLayer;
class UMovieSceneAnimationMixerTrack;
class ISequencerTrackEditor;

namespace UE::Sequencer
{

/**
 * View model for an Animation Mixer Layer.
 *
 * A layer can contain either:
 * - Multiple sections (any combo of mixed types that produce anim poses)
 * - A single child track (e.g., Control Rig track) that occupies the entire layer
 *
 * Implements all the same extensions as FTrackModel and FTrackRowModel to support:
 * - Drag/drop reordering via ITrackExtension::SetRowIndex()
 * - Section management
 * - Channel display
 * - Locking, muting, soloing
 * - Context menus
 * - Being treated as a 'track' for child tracks
 *
 * 
 */
class FAnimationMixerLayerModel
	: public FEvaluableOutlinerItemModel
	, public IRenameableExtension
	, public IResizableExtension
	, public ITrackExtension
	, public ITrackRowExtension
	, public ISectionOwnerExtension
	, public ITopLevelChannelHolderExtension
	, public ITrackAreaExtension
	, public ILockableExtension
	, public IGroupableExtension
	, public ISortableExtension
	, public IDraggableOutlinerExtension
	, public IDeletableExtension
	, public IConditionableExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FAnimationMixerLayerModel
		, FEvaluableOutlinerItemModel
		, IRenameableExtension
		, IResizableExtension
		, ITrackExtension
		, ITrackRowExtension
		, ISectionOwnerExtension
		, ITopLevelChannelHolderExtension
		, ITrackAreaExtension
		, ILockableExtension
		, IGroupableExtension
		, ISortableExtension
		, IDraggableOutlinerExtension
		, IDeletableExtension
		, IConditionableExtension);

	/** Constructor for layer containing sections */
	explicit FAnimationMixerLayerModel(UMovieSceneAnimationMixerTrack* InParentTrack, UMovieSceneAnimationMixerLayer* InLayer);

	/** Constructor for layer containing a child track */
	explicit FAnimationMixerLayerModel(UMovieSceneAnimationMixerTrack* InParentTrack, UMovieSceneAnimationMixerLayer* InLayer, UMovieSceneTrack* InChildTrack);

	~FAnimationMixerLayerModel();

	/** Refreshes the layout of the layer model including all child sections/outliner items */
	void RefreshLayout(bool bChildrenNeedLayout = true);

	UMovieSceneAnimationMixerLayer* GetLayer() const { return WeakLayer.Get(); }

	UMovieSceneAnimationMixerTrack* GetParentMixerTrack() const { return WeakParentTrack.Get(); }

	bool HasChildTrack() const { return WeakChildTrack.IsValid(); }

	UMovieSceneTrack* GetChildTrack() const { return WeakChildTrack.Get(); }

	/*~ ITrackExtension */
	virtual UMovieSceneTrack* GetTrack() const override;
	virtual TSharedPtr<ISequencerTrackEditor> GetTrackEditor() const override;

	/*~ ITrackRowExtension */
	virtual int32 GetRowIndex() const override;
	virtual bool SetRowIndex(int32 InRowIndex) override;
	virtual UMovieSceneTrack* GetParentTrack() const override;

	/*~ ISectionOwnerExtension */
	virtual FViewModelChildren GetSectionModels() override;

	/*~ ITopLevelChannelHolderExtension */
	virtual FViewModelChildren GetTopLevelChannels() override;

	/*~ IOutlinerExtension */
	virtual FOutlinerSizing GetOutlinerSizing() const override;
	virtual TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	virtual FSlateFontInfo GetLabelFont() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetLabel() const override;
	virtual FSlateColor GetLabelColor() const override;
	virtual FText GetLabelToolTipText() const override;

	/*~ IDimmableExtension */
	virtual bool IsDimmed() const override;

	/*~ IResizableExtension */
	virtual bool IsResizable() const override;
	virtual void Resize(float NewSize) override;

	/*~ ILockableExtension Interface */
	virtual ELockableLockState GetLockState() const override;
	virtual void SetIsLocked(bool bIsLocked) override;

	/*~ IConditionableExtension Interface */
	virtual const UMovieSceneCondition* GetCondition() const override;
	virtual EConditionableConditionState GetConditionState() const override;
	virtual void SetConditionEditorForceTrue(bool bEditorForceTrue) override;

	/*~ ITrackAreaExtension */
	virtual FTrackAreaParameters GetTrackAreaParameters() const override;
	virtual FViewModelVariantIterator GetTrackAreaModelList() const override;
	virtual FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ ICurveEditorTreeItem */
	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;

	/*~ IGroupableIdentifier */
	virtual void GetIdentifierForGrouping(TStringBuilder<128>& OutString) const override;

	/*~ IRenameableExtension */
	virtual bool CanRename() const override;
	virtual void Rename(const FText& NewName) override;
	virtual bool IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const override;

	/*~ ISortableExtension */
	virtual void SortChildren() override;
	virtual FSortingKey GetSortingKey() const override;
	virtual void SetCustomOrder(int32 InCustomOrder) override;

	/*~ IDraggableOutlinerExtension */
	virtual bool CanDrag() const override;

	/*~ FEvaluableOutlinerItemModel */
	virtual bool IsDeactivated() const override;
	virtual void SetIsDeactivated(bool bInIsDeactivated) override;

	/*~ FOutlinerItemModel */
	virtual bool HasCurves() const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool GetDefaultExpansionState() const override;
	virtual void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;
	virtual void SetExpansion(bool bInIsExpanded) override;

	/*~ IDeletableExtension */
	virtual bool CanDelete(FText* OutErrorMessage) const override;
	virtual void Delete() override;

	/*~ IMutableExtension */
	virtual bool IsMuted() const override;
	virtual void SetIsMuted(bool bIsMuted) override;

	/*~ ISoloableExtension */
	virtual bool IsSolo() const override;
	virtual void SetIsSoloed(bool bIsSoloed) override;

	/*~ FViewModel interface */
	virtual void OnConstruct() override;

private:

	/**
	 * Query the saved expansion state from editor data without checking for children.
	 * This is needed during layout because IsExpanded() checks for children, but we're
	 * in the process of building them.
	 */
	bool GetSavedExpansionState() const;

	/** A second children list for the sections inside this layer */
	FViewModelListHead SectionList{EViewModelListType::TrackArea};
	FViewModelListHead TopLevelChannelList{FTrackModel::GetTopLevelChannelGroupType()};

	/** The parent animation mixer track */
	TWeakObjectPtr<UMovieSceneAnimationMixerTrack> WeakParentTrack;

	/** The layer data object */
	TWeakObjectPtr<UMovieSceneAnimationMixerLayer> WeakLayer;

	/** Child track if this layer contains one (e.g., Control Rig) */
	TWeakObjectPtr<UMovieSceneTrack> WeakChildTrack;

	/** Track editor for the child track or parent track */
	TSharedPtr<ISequencerTrackEditor> TrackEditor;

	TOptional<int32> PreviousLayoutNumSections = TOptional<int32>();

	/** Handles lightweight decoration sync when child track changes */
	void OnChildTrackChanged();
	FDelegateHandle ChildTrackChangedHandle;

	/** Handles decoration sync when layer changes */
	void OnLayerChanged();
	FDelegateHandle LayerChangedHandle;
};

} // namespace UE::Sequencer
