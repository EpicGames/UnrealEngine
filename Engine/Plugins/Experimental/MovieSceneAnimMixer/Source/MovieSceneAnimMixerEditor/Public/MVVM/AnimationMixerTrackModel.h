// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameNumber.h"
#include "Math/Range.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModelPtr.h"

class UMovieSceneAnimationMixerTrack;

namespace UE::Sequencer
{

class FLayerBarModel;

/**
 * Drop preview data for dragging sections in Animation Mixer track
 */
struct FAnimMixerDropPreview
{
	enum class EZone
	{
		AboveLayer,  // Insert new layer above target
		BelowLayer,  // Insert new layer below target
		OntoLayer    // Drop sections onto existing layer
	};

	EZone Zone = EZone::OntoLayer;
	int32 TargetRowIndex = INDEX_NONE;
	float VirtualTop = 0.0f;
	float VirtualBottom = 0.0f;
	TRange<FFrameNumber> SectionFrameRange; 

	bool operator==(const FAnimMixerDropPreview& Other) const
	{
		return Zone == Other.Zone && TargetRowIndex == Other.TargetRowIndex;
	}

	bool operator!=(const FAnimMixerDropPreview& Other) const
	{
		return !(*this == Other);
	}
};

/**
 * Custom track model for Animation Mixer tracks.
 * Supports hierarchical child tracks (e.g., control rig tracks) that render as nested tracks under the anim mixer track.
 * Uses preview-based dragging with outliner-style drop indicators.
 */
class FAnimationMixerTrackModel : public FTrackModel
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE(FAnimationMixerTrackModel, FTrackModel);

	explicit FAnimationMixerTrackModel(UMovieSceneTrack* Track);

	/*~ FViewModel interface */
	virtual void OnConstruct() override;

	/*~ ITrackAreaExtension */
	virtual FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ FEvaluableOutlinerItemModel */
	virtual bool IsDeactivated() const override;
	virtual void SetIsDeactivated(bool bInIsDeactivated) override;

	/*~ ISectionDragOwnerExtension - Preview-based drag with outliner-style drop indicators */
	virtual void OnBeginSectionVerticalDrag() override;
	virtual bool OnSectionVerticalDrag(const FSectionVerticalDragContext& Context) override;
	virtual void OnEndSectionVerticalDrag(const FSectionVerticalDragContext& Context) override;
	virtual int32 OnPaintSectionDragPreview(
		const FGeometry& TrackAreaGeometry,
		const class FVirtualTrackArea& VirtualTrackArea,
		const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled) const override;

protected:
	virtual void ForceUpdate() override;

	// Override section drag functions to consider both sections AND child tracks
	virtual void ShiftRowsDownFromIndex(int32 StartIndex, const TSet<UMovieSceneSection*>& ExcludeSections) override;

private:
	// Check if a row is occupied by a child track (prevents section drops on child track layers)
	bool IsRowOccupiedByChildTrack(int32 RowIndex) const;

	// Compute drop preview without modifying the track
	TOptional<FAnimMixerDropPreview> ComputeDropPreview(const FSectionVerticalDragContext& Context) const;

	// Apply the cached preview to actually move sections
	bool ApplyDropPreview(const FAnimMixerDropPreview& Preview, const FSectionVerticalDragContext& Context);

	// Track if we're currently in a drag operation (to prevent view mode changes during drag)
	bool bIsInDrag = false;

	// Cached drop preview (only valid during drag)
	mutable TOptional<FAnimMixerDropPreview> CachedPreview;

	// Layer bar shown when the track is collapsed
	FViewModelListHead LayerBarList;
	TSharedPtr<FLayerBarModel> LayerBar;
};

} // namespace UE::Sequencer
