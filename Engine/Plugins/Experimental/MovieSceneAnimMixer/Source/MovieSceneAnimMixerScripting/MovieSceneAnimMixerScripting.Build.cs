// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieSceneAnimMixerScripting : ModuleRules
	{
		public MovieSceneAnimMixerScripting(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"MovieScene",
					"MovieSceneTracks",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"MovieSceneAnimMixer",
				}
			);
		}
	}
}
