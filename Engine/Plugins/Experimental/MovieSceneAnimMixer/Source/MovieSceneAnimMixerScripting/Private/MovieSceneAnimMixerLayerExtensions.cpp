// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerLayerExtensions.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerLayerExtensions)

FText UMovieSceneAnimMixerLayerExtensions::GetDisplayName(UMovieSceneAnimationMixerLayer* Layer)
{
	if (!Layer)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetDisplayName on a null layer"), ELogVerbosity::Error);
		return FText::GetEmpty();
	}

#if WITH_EDITORONLY_DATA
	return Layer->GetDisplayName();
#else
	return FText::GetEmpty();
#endif
}

void UMovieSceneAnimMixerLayerExtensions::SetDisplayName(UMovieSceneAnimationMixerLayer* Layer, const FText& InName)
{
	if (!Layer)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetDisplayName on a null layer"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	Layer->SetDisplayName(InName);
#endif
}

int32 UMovieSceneAnimMixerLayerExtensions::GetLayerIndex(UMovieSceneAnimationMixerLayer* Layer)
{
	if (!Layer)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetLayerIndex on a null layer"), ELogVerbosity::Error);
		return INDEX_NONE;
	}

	return Layer->GetLayerIndex();
}

TArray<UMovieSceneSection*> UMovieSceneAnimMixerLayerExtensions::GetSections(UMovieSceneAnimationMixerLayer* Layer)
{
	if (!Layer)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetSections on a null layer"), ELogVerbosity::Error);
		return TArray<UMovieSceneSection*>();
	}

	TArray<UMovieSceneSection*> Result;
	for (const TObjectPtr<UMovieSceneSection>& Section : Layer->GetSections())
	{
		Result.Add(Section);
	}
	return Result;
}

bool UMovieSceneAnimMixerLayerExtensions::IsEmpty(UMovieSceneAnimationMixerLayer* Layer)
{
	if (!Layer)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call IsEmpty on a null layer"), ELogVerbosity::Error);
		return true;
	}

	return Layer->IsEmpty();
}
