// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerTrackExtensions.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "MovieSceneAnimationMixerLayer.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerTrackExtensions)

TArray<UMovieSceneAnimationMixerLayer*> UMovieSceneAnimMixerTrackExtensions::GetLayers(UMovieSceneAnimationMixerTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetLayers on a null track"), ELogVerbosity::Error);
		return TArray<UMovieSceneAnimationMixerLayer*>();
	}

	TArray<UMovieSceneAnimationMixerLayer*> Result;
	for (const TObjectPtr<UMovieSceneAnimationMixerLayer>& Layer : Track->GetLayers())
	{
		Result.Add(Layer);
	}
	return Result;
}

int32 UMovieSceneAnimMixerTrackExtensions::GetLayerCount(UMovieSceneAnimationMixerTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetLayerCount on a null track"), ELogVerbosity::Error);
		return 0;
	}

	return Track->GetLayers().Num();
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimMixerTrackExtensions::AddLayer(UMovieSceneAnimationMixerTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddLayer on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->InsertLayer(Track->GetLayers().Num());
}

UMovieSceneAnimationMixerLayer* UMovieSceneAnimMixerTrackExtensions::InsertLayer(UMovieSceneAnimationMixerTrack* Track, int32 Index)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call InsertLayer on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->InsertLayer(Index);
}

UMovieSceneSection* UMovieSceneAnimMixerTrackExtensions::AddAnimation(UMovieSceneAnimationMixerTrack* Track, int32 LayerIndex, FFrameNumber StartFrame, UAnimSequenceBase* AnimSequence)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddAnimation on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	if (!AnimSequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddAnimation with a null AnimSequence"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->AddNewAnimationOnRow(StartFrame, AnimSequence, LayerIndex);
}

TArray<UMovieSceneAnimTransitionSectionBase*> UMovieSceneAnimMixerTrackExtensions::GetTransitionsForSection(UMovieSceneAnimationMixerTrack* Track, UMovieSceneSection* Section)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTransitionsForSection on a null track"), ELogVerbosity::Error);
		return TArray<UMovieSceneAnimTransitionSectionBase*>();
	}

	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTransitionsForSection with a null section"), ELogVerbosity::Error);
		return TArray<UMovieSceneAnimTransitionSectionBase*>();
	}

	return Track->FindTransitionsForSection(Section);
}

UMovieSceneAnimTransitionSectionBase* UMovieSceneAnimMixerTrackExtensions::GetTransitionBetween(UMovieSceneAnimationMixerTrack* Track, UMovieSceneSection* FromSection, UMovieSceneSection* ToSection)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTransitionBetween on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	if (!FromSection || !ToSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTransitionBetween with null sections"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->FindTransitionForPair(FromSection, ToSection);
}

UMovieSceneTrack* UMovieSceneAnimMixerTrackExtensions::AddChildTrackToLayer(UMovieSceneAnimationMixerTrack* Track, FGuid ObjectBinding, TSubclassOf<UMovieSceneTrack> TrackClass, int32 LayerIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildTrackToLayer on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	if (TrackClass.Get() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildTrackToLayer with a null track class"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->AddChildTrack(ObjectBinding, TrackClass, LayerIndex);
}
