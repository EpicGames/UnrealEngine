// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAnimMixerTransitionExtensions.h"
#include "MovieSceneAnimTransitionSectionBase.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Channels/MovieSceneFloatChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAnimMixerTransitionExtensions)

FText UMovieSceneAnimMixerTransitionExtensions::GetTransitionDisplayName(UMovieSceneAnimTransitionSectionBase* Transition)
{
	if (!Transition)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTransitionDisplayName on a null transition"), ELogVerbosity::Error);
		return FText::GetEmpty();
	}

	return Transition->GetTransitionDisplayName();
}

bool UMovieSceneAnimMixerTransitionExtensions::IsTransitionValid(UMovieSceneAnimTransitionSectionBase* Transition)
{
	if (!Transition)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call IsTransitionValid on a null transition"), ELogVerbosity::Error);
		return false;
	}

	return Transition->IsValid();
}

UMovieSceneSection* UMovieSceneAnimMixerTransitionExtensions::GetFromSection(UMovieSceneAnimTransitionSectionBase* Transition)
{
	if (!Transition)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetFromSection on a null transition"), ELogVerbosity::Error);
		return nullptr;
	}

	return Transition->FromSection;
}

UMovieSceneSection* UMovieSceneAnimMixerTransitionExtensions::GetToSection(UMovieSceneAnimTransitionSectionBase* Transition)
{
	if (!Transition)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetToSection on a null transition"), ELogVerbosity::Error);
		return nullptr;
	}

	return Transition->ToSection;
}

UMovieSceneAnimTransitionSectionBase* UMovieSceneAnimMixerTransitionExtensions::ChangeTransitionType(
	UMovieSceneAnimTransitionSectionBase* Transition,
	TSubclassOf<UMovieSceneAnimTransitionSectionBase> NewTransitionClass)
{
	if (!Transition)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call ChangeTransitionType on a null transition"), ELogVerbosity::Error);
		return nullptr;
	}

	if (!NewTransitionClass)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call ChangeTransitionType with a null class"), ELogVerbosity::Error);
		return nullptr;
	}

	// No change needed if already the requested type
	if (Transition->GetClass() == NewTransitionClass)
	{
		return Transition;
	}

	UMovieSceneAnimationMixerTrack* MixerTrack = Cast<UMovieSceneAnimationMixerTrack>(Transition->GetTypedOuter<UMovieSceneAnimationMixerTrack>());
	if (!MixerTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("ChangeTransitionType: transition does not belong to a mixer track"), ELogVerbosity::Error);
		return nullptr;
	}

	// Preserve data from the old transition before removal
	UMovieSceneSection* FromSection = Transition->FromSection;
	UMovieSceneSection* ToSection = Transition->ToSection;

	// Copy blend channel data before RemoveSection potentially invalidates it
	TOptional<FMovieSceneFloatChannel> CopiedBlendChannel;
	if (const FMovieSceneFloatChannel* OldBlendChannel = Transition->GetBlendWeightChannel())
	{
		CopiedBlendChannel = *OldBlendChannel;
	}

	// Remove the old transition from the track
	MixerTrack->Modify();
	MixerTrack->RemoveSection(*Transition);

	// Create the replacement transition
	UMovieSceneAnimTransitionSectionBase* NewTransition = MixerTrack->CreateTransitionSectionOfType(FromSection, ToSection, NewTransitionClass);
	if (!NewTransition)
	{
		FFrame::KismetExecutionMessage(TEXT("ChangeTransitionType: failed to create new transition (sections may no longer overlap)"), ELogVerbosity::Error);
		return nullptr;
	}

	// Copy saved blend weight channel data to the new transition
	const FMovieSceneFloatChannel* NewBlendChannel = NewTransition->GetBlendWeightChannel();
	if (CopiedBlendChannel.IsSet() && NewBlendChannel)
	{
		FMovieSceneFloatChannel* MutableNewChannel = const_cast<FMovieSceneFloatChannel*>(NewBlendChannel);
		*MutableNewChannel = CopiedBlendChannel.GetValue();
	}
	else
	{
		NewTransition->InitializeDefaultCurve();
	}

	return NewTransition;
}
