// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDecorationContainerExtensions.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "IMovieSceneModule.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDecorationContainerExtensions)

TArray<UClass*> UMovieSceneDecorationContainerExtensions::GetCompatibleDecorations(UMovieSceneDecorationContainerObject* Container)
{
	if (!Container)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetCompatibleDecorations on a null container"), ELogVerbosity::Error);
		return TArray<UClass*>();
	}

	TSet<UClass*> ClassSet;
	IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(Container, ClassSet);
	return ClassSet.Array();
}

TArray<UObject*> UMovieSceneDecorationContainerExtensions::GetDecorations(UMovieSceneDecorationContainerObject* Container)
{
	if (!Container)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetDecorations on a null container"), ELogVerbosity::Error);
		return TArray<UObject*>();
	}

	TArray<UObject*> Result;
	for (const TObjectPtr<UObject>& Decoration : Container->GetDecorations())
	{
		Result.Add(Decoration);
	}
	return Result;
}

UObject* UMovieSceneDecorationContainerExtensions::FindDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass)
{
	if (!Container)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindDecoration on a null container"), ELogVerbosity::Error);
		return nullptr;
	}

	if (DecorationClass.Get() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindDecoration with a null class"), ELogVerbosity::Error);
		return nullptr;
	}

	return Container->FindDecoration(DecorationClass);
}

UObject* UMovieSceneDecorationContainerExtensions::AddDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass)
{
	if (!Container)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddDecoration on a null container"), ELogVerbosity::Error);
		return nullptr;
	}

	if (DecorationClass.Get() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddDecoration with a null class"), ELogVerbosity::Error);
		return nullptr;
	}

	return Container->GetOrCreateDecoration(DecorationClass);
}

void UMovieSceneDecorationContainerExtensions::RemoveDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass)
{
	if (!Container)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveDecoration on a null container"), ELogVerbosity::Error);
		return;
	}

	if (DecorationClass.Get() == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveDecoration with a null class"), ELogVerbosity::Error);
		return;
	}

	Container->RemoveDecoration(DecorationClass);
}
