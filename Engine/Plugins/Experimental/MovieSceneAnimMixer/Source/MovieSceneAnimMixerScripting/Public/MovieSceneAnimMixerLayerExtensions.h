// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneAnimMixerLayerExtensions.generated.h"

class UMovieSceneAnimationMixerLayer;
class UMovieSceneSection;

#define UE_API MOVIESCENEANIMMIXERSCRIPTING_API

/**
 * Function library containing methods that extend UMovieSceneAnimationMixerLayer for scripting.
 */
UCLASS()
class UMovieSceneAnimMixerLayerExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get the display name for this layer
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API FText GetDisplayName(UMovieSceneAnimationMixerLayer* Layer);

	// Set a custom display name for this layer
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetDisplayName(UMovieSceneAnimationMixerLayer* Layer, const FText& InName);

	// Get the index of this layer within its parent mixer track
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API int32 GetLayerIndex(UMovieSceneAnimationMixerLayer* Layer);

	// Get all sections on this layer
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneSection*> GetSections(UMovieSceneAnimationMixerLayer* Layer);

	// Check if this layer is empty (no sections and no child track)
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API bool IsEmpty(UMovieSceneAnimationMixerLayer* Layer);
};

#undef UE_API
