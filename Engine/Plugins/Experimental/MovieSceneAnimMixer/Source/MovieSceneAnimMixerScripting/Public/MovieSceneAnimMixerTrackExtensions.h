// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneAnimMixerTrackExtensions.generated.h"

class UAnimSequenceBase;
class UControlRig;
class UMovieSceneAnimationMixerLayer;
class UMovieSceneAnimationMixerTrack;
class UMovieSceneAnimTransitionSectionBase;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FFrameNumber;
struct FGuid;

#define UE_API MOVIESCENEANIMMIXERSCRIPTING_API

/**
 * Function library containing methods that extend UMovieSceneAnimationMixerTrack for scripting.
 */
UCLASS()
class UMovieSceneAnimMixerTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get all layers in this mixer track
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneAnimationMixerLayer*> GetLayers(UMovieSceneAnimationMixerTrack* Track);

	// Get the number of layers in this mixer track
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API int32 GetLayerCount(UMovieSceneAnimationMixerTrack* Track);

	// Add a new empty layer at the end
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API UMovieSceneAnimationMixerLayer* AddLayer(UMovieSceneAnimationMixerTrack* Track);

	// Insert a new empty layer at the specified index, shifting existing layers down
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API UMovieSceneAnimationMixerLayer* InsertLayer(UMovieSceneAnimationMixerTrack* Track, int32 Index);

	// Add a skeletal animation to a specific layer at the given start frame
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API UMovieSceneSection* AddAnimation(UMovieSceneAnimationMixerTrack* Track, int32 LayerIndex, FFrameNumber StartFrame, UAnimSequenceBase* AnimSequence);

	// Get all transition sections that reference the given section as either "from" or "to"
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneAnimTransitionSectionBase*> GetTransitionsForSection(UMovieSceneAnimationMixerTrack* Track, UMovieSceneSection* Section);

	// Get the transition between two specific sections, or nullptr if none exists
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API UMovieSceneAnimTransitionSectionBase* GetTransitionBetween(UMovieSceneAnimationMixerTrack* Track, UMovieSceneSection* FromSection, UMovieSceneSection* ToSection);

	// Add a child track (e.g. Control Rig) to a specific layer.
	// This is the UI operation of adding a track type to an empty mixer layer.
	// The track will occupy the entire layer (layers support either sections or a child track, not both).
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer", meta=(ScriptMethod))
	static UE_API UMovieSceneTrack* AddChildTrackToLayer(UMovieSceneAnimationMixerTrack* Track, FGuid ObjectBinding, TSubclassOf<UMovieSceneTrack> TrackClass, int32 LayerIndex);

};

#undef UE_API
