// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneAnimMixerTransitionExtensions.generated.h"

class UMovieSceneAnimTransitionSectionBase;
class UMovieSceneSection;

#define UE_API MOVIESCENEANIMMIXERSCRIPTING_API

/**
 * Function library containing methods that extend UMovieSceneAnimTransitionSectionBase for scripting.
 */
UCLASS()
class UMovieSceneAnimMixerTransitionExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get the display name for this transition type
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer|Transition", meta=(ScriptMethod))
	static UE_API FText GetTransitionDisplayName(UMovieSceneAnimTransitionSectionBase* Transition);

	// Check if this transition is structurally valid (both sections exist, same row, overlap exists)
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer|Transition", meta=(ScriptMethod))
	static UE_API bool IsTransitionValid(UMovieSceneAnimTransitionSectionBase* Transition);

	// Get the section this transition blends from
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer|Transition", meta=(ScriptMethod))
	static UE_API UMovieSceneSection* GetFromSection(UMovieSceneAnimTransitionSectionBase* Transition);

	// Get the section this transition blends to
	UFUNCTION(BlueprintPure, Category = "Sequencer|AnimMixer|Transition", meta=(ScriptMethod))
	static UE_API UMovieSceneSection* GetToSection(UMovieSceneAnimTransitionSectionBase* Transition);

	// Replace this transition with a new one of the given class, preserving blend data where possible.
	// Returns the new transition, or nullptr on failure.
	UFUNCTION(BlueprintCallable, Category = "Sequencer|AnimMixer|Transition", meta=(ScriptMethod))
	static UE_API UMovieSceneAnimTransitionSectionBase* ChangeTransitionType(UMovieSceneAnimTransitionSectionBase* Transition, TSubclassOf<UMovieSceneAnimTransitionSectionBase> NewTransitionClass);
};

#undef UE_API
