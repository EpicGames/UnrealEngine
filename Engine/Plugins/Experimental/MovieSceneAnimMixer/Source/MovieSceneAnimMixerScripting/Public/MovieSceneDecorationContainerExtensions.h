// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneDecorationContainerExtensions.generated.h"

class UMovieSceneDecorationContainerObject;

#define UE_API MOVIESCENEANIMMIXERSCRIPTING_API

/**
 * Function library containing methods that extend UMovieSceneDecorationContainerObject for scripting.
 * Exposes the generic decoration API for discovering, adding, and removing decorations on any container.
 */
UCLASS()
class UMovieSceneDecorationContainerExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get all decoration classes that can be added to this container
	UFUNCTION(BlueprintPure, Category = "Sequencer|Decorations", meta=(ScriptMethod))
	static UE_API TArray<UClass*> GetCompatibleDecorations(UMovieSceneDecorationContainerObject* Container);

	// Get all decorations currently on this container
	UFUNCTION(BlueprintPure, Category = "Sequencer|Decorations", meta=(ScriptMethod))
	static UE_API TArray<UObject*> GetDecorations(UMovieSceneDecorationContainerObject* Container);

	// Find a decoration of a specific class on this container, or nullptr if not present
	UFUNCTION(BlueprintPure, Category = "Sequencer|Decorations", meta=(ScriptMethod))
	static UE_API UObject* FindDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass);

	// Add a decoration by class. Returns the decoration (existing or newly created).
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Decorations", meta=(ScriptMethod))
	static UE_API UObject* AddDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass);

	// Remove a decoration by class
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Decorations", meta=(ScriptMethod))
	static UE_API void RemoveDecoration(UMovieSceneDecorationContainerObject* Container, TSubclassOf<UObject> DecorationClass);
};

#undef UE_API
